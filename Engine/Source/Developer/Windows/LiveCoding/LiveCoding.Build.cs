// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class LiveCoding : ModuleRules
{
	public const bool UseLiveCoding2 = false;

	public LiveCoding(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("CoreUObject");
		PrivateDependencyModuleNames.Add("Settings");
		PrivateDependencyModuleNames.Add("BuildSettings");
		PrivateDependencyModuleNames.Add("OodleDataCompression");

		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.Add("Engine");
		}

		if (Target.bBuildEditor == true)
        {
            PrivateDependencyModuleNames.Add("UnrealEd");
			PrivateDependencyModuleNames.Add("Slate");
		}

		if(Target.bUseDebugLiveCodingConsole)
        {
            PrivateDefinitions.Add("USE_DEBUG_LIVE_CODING_CONSOLE=1");
        }
		else
        {
            PrivateDefinitions.Add("USE_DEBUG_LIVE_CODING_CONSOLE=0");
        }

		bool UseLiveCoding2ToNotGetCompileWarning = UseLiveCoding2;

		if (!Target.Architecture.bIsX64)
		{
			PrivateDefinitions.Add("LC_VERSION=0");
		}
		else if (UseLiveCoding2ToNotGetCompileWarning)
		{
			PrivateDefinitions.Add("LC_VERSION=2");
			PrivateIncludePaths.Add("ThirdParty/LivePlusPlus");
		}
		else
		{
			PrivateDefinitions.Add("LC_VERSION=1");

			if (Target.Configuration == UnrealTargetConfiguration.Debug)
			{
				PrivateDefinitions.Add("LC_DEBUG=1");
			}
			else
			{
				PrivateDefinitions.Add("LC_DEBUG=0");
			}

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				PrivateIncludePaths.Add("Developer/Windows/LiveCodingServer/Private/External");
			}

			PublicAdditionalLibraries.Add(Path.Combine(ModuleDirectory, "Private", "External", "LC_JumpToSelf.lib"));
		}
	}
}
