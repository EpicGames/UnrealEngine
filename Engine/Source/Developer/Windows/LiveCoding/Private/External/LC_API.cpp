// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// EPIC BEGIN MOD
//#include PCH_INCLUDE
// EPIC END MOD
#include "LC_API.h"
#include "LC_ClientStartupThread.h"
#include "LC_CriticalSection.h"
#include "LC_SyncPoint.h"
#include "LC_Restart.h"
#include "LC_RunMode.h"
#include "LC_Memory.h"
#include "LPP_API.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
// END EPIC MOD

namespace
{
	// startup thread
	static ClientStartupThread* g_startupThread = nullptr;
	
	// critical section to ensure that startup thread is initialized only once
	static CriticalSection g_ensureOneTimeStartup;

	// default run mode
	static RunMode::Enum g_runMode = RunMode::DEFAULT;


	static bool CheckForStartup(void)
	{
		if (!g_startupThread)
		{
			LC_ERROR_USER("%s", "Live++ was not started properly. Call lppStartup() first.");
			return false;
		}

		return true;
	}
}


LPP_DLL_API(void) LppStartup(void)
{
	g_startupThread = new ClientStartupThread();

	restart::Startup();
}


LPP_DLL_API(void) LppShutdown(void)
{
	restart::Shutdown();

	// wait for the startup thread to finish its work and clean up
	g_startupThread->Join();
	memory::DeleteAndNull(g_startupThread);
}


LPP_DLL_API(const char*) LppGetVersion(void)
{
	return LPP_VERSION;
}


LPP_DLL_API(int) LppCheckVersion(const char* apiVersion)
{
	if (strcmp(apiVersion, LPP_VERSION) == 0)
	{
		return 1;
	}
	
	LC_ERROR_USER("Version mismatch detected. API version: %s, DLL version: %s", LPP_VERSION, apiVersion);
	return 0;
}


LPP_DLL_API(void) LppRegisterProcessGroup(const char* groupName)
{
	if (!CheckForStartup())
	{
		return;
	}

	// now that we have the process group name, start Live++.
	// ensure that initialization can happen only once, even if the user calls this more than once.
	{
		CriticalSection::ScopedLock lock(&g_ensureOneTimeStartup);

		static bool firstTime = true;
		if (!firstTime)
		{
			// this was already called once, bail out
			return;
		}

		firstTime = false;
	}

	g_startupThread->Start(groupName, g_runMode);
}


LPP_DLL_API(void) LppSyncPoint(void)
{
	if (!CheckForStartup())
	{
		return;
	}

	syncPoint::EnterTarget();
}


LPP_DLL_API(void) LppWaitForToken(void* token)
{
	if (!CheckForStartup())
	{
		return;
	}

	if (!token)
	{
		// nullptr tokens are returned by Live++ when trying to enable modules which are not loaded into the host process.
		// therefore, we need to handle this case gracefully.
		return;
	}

	g_startupThread->WaitForToken(token);
}


// BEGIN EPIC MOD - Adding LppTryWaitForToken
LPP_DLL_API(bool) LppTryWaitForToken(void* token)
{
	if (!CheckForStartup())
	{
		return false;
	}

	if (!token)
	{
		// nullptr tokens are returned by Live++ when trying to enable modules which are not loaded into the host process.
		// therefore, we need to handle this case gracefully.
		return true;
	}

	return g_startupThread->TryWaitForToken(token);
}
// END EPIC MOD


LPP_DLL_API(void) LppTriggerRecompile(void)
{
	if (!CheckForStartup())
	{
		return;
	}

	g_startupThread->TriggerRecompile();
}


LPP_DLL_API(void) LppLogMessage(const wchar_t* message)
{
	if (!CheckForStartup())
	{
		return;
	}

	g_startupThread->LogMessage(message);
}


LPP_DLL_API(void) LppBuildPatch(const wchar_t* moduleNames[], const wchar_t* objPaths[], const wchar_t* amalgamatedObjPaths[], unsigned int count)
{
	if (!CheckForStartup())
	{
		return;
	}

	g_startupThread->BuildPatch(moduleNames, objPaths, amalgamatedObjPaths, count);
}


LPP_DLL_API(void) LppInstallExceptionHandler(void)
{
	if (!CheckForStartup())
	{
		return;
	}

	g_startupThread->InstallExceptionHandler();
}


LPP_DLL_API(void) LppUseExternalBuildSystem(void)
{
	if (!CheckForStartup())
	{
		return;
	}

	g_runMode = RunMode::EXTERNAL_BUILD_SYSTEM;
}


LPP_DLL_API(void) LppTriggerRestart(void)
{
	if (!CheckForStartup())
	{
		return;
	}

	return g_startupThread->TriggerRestart();
}


LPP_DLL_API(int) LppWantsRestart(void)
{
	if (!CheckForStartup())
	{
		return 0;
	}

	return restart::WasRequested();
}


LPP_DLL_API(void) LppRestart(lpp::RestartBehaviour behaviour, unsigned int exitCode)
{
	if (!CheckForStartup())
	{
		return;
	}

	restart::Execute(behaviour, exitCode);
}


LPP_DLL_API(void*) LppEnableModule(const wchar_t* nameOfExeOrDll)
{
	if (!CheckForStartup())
	{
		return nullptr;
	}

	return g_startupThread->EnableModule(nameOfExeOrDll);
}


LPP_DLL_API(void*) LppEnableModules(const wchar_t* namesOfExeOrDll[], unsigned int count)
{
	if (!CheckForStartup())
	{
		return nullptr;
	}

	return g_startupThread->EnableModules(namesOfExeOrDll, count);
}


LPP_DLL_API(void*) LppEnableAllModules(const wchar_t* nameOfExeOrDll)
{
	if (!CheckForStartup())
	{
		return nullptr;
	}

	return g_startupThread->EnableAllModules(nameOfExeOrDll);
}


LPP_DLL_API(void*) LppDisableModule(const wchar_t* nameOfExeOrDll)
{
	if (!CheckForStartup())
	{
		return nullptr;
	}

	return g_startupThread->DisableModule(nameOfExeOrDll);
}


LPP_DLL_API(void*) LppDisableModules(const wchar_t* namesOfExeOrDll[], unsigned int count)
{
	if (!CheckForStartup())
	{
		return nullptr;
	}

	return g_startupThread->DisableModules(namesOfExeOrDll, count);
}


LPP_DLL_API(void*) LppDisableAllModules(const wchar_t* nameOfExeOrDll)
{
	if (!CheckForStartup())
	{
		return nullptr;
	}

	return g_startupThread->DisableAllModules(nameOfExeOrDll);
}


// BEGIN EPIC MOD - Adding ShowConsole command
LPP_DLL_API(void) LppShowConsole()
{
	g_startupThread->ShowConsole();
}
// END EPIC MOD


// BEGIN EPIC MOD - Adding SetVisible command
LPP_DLL_API(void) LppSetVisible(bool visible)
{
	g_startupThread->SetVisible(visible);
}
// END EPIC MOD



// BEGIN EPIC MOD - Adding SetActive command
LPP_DLL_API(void) LppSetActive(bool active)
{
	g_startupThread->SetActive(active);
}
// END EPIC MOD


// BEGIN EPIC MOD - Adding SetBuildArguments command
LPP_DLL_API(void) LppSetBuildArguments(const wchar_t* arguments)
{
	g_startupThread->SetBuildArguments(arguments);
}
// END EPIC MOD

// BEGIN EPIC MOD - Support for lazy-loading modules
LPP_DLL_API(void*) LppEnableLazyLoadedModule(const wchar_t* nameOfExeOrDll)
{
	HMODULE baseAddress = GetModuleHandle(nameOfExeOrDll);
	return g_startupThread->EnableLazyLoadedModule(nameOfExeOrDll, baseAddress);
}
// END EPIC MOD

// BEGIN EPIC MOD
LPP_DLL_API(void) LppSetReinstancingFlow(bool enable)
{
	g_startupThread->SetReinstancingFlow(enable);
}
// END EPIC MOD

// BEGIN EPIC MOD
LPP_DLL_API(void) LppDisableCompileFinishNotification()
{
	g_startupThread->DisableCompileFinishNotification();
}
// END EPIC MOD

// BEGIN EPIC MOD
LPP_DLL_API(void*) LppEnableModulesEx(const wchar_t* moduleNames[], unsigned int moduleCount, const wchar_t* lazyLoadModuleNames[], unsigned int lazyLoadModuleCount, const uintptr_t* reservedPages, unsigned int reservedPagesCount)
{
	if (!CheckForStartup())
	{
		return nullptr;
	}

	return g_startupThread->EnableModulesEx(moduleNames, moduleCount, lazyLoadModuleNames, lazyLoadModuleCount, reservedPages, reservedPagesCount);
}
// END EPIC MOD


LPP_DLL_API(void) LppApplySettingBool(const char* settingName, int value)
{
	if (!CheckForStartup())
	{
		return;
	}

	g_startupThread->ApplySettingBool(settingName, value);
}


LPP_DLL_API(void) LppApplySettingInt(const char* settingName, int value)
{
	if (!CheckForStartup())
	{
		return;
	}

	g_startupThread->ApplySettingInt(settingName, value);
}


LPP_DLL_API(void) LppApplySettingString(const char* settingName, const wchar_t* value)
{
	if (!CheckForStartup())
	{
		return;
	}

	g_startupThread->ApplySettingString(settingName, value);
}

#endif // LC_VERSION