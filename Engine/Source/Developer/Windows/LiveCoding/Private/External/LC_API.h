// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include <stddef.h>
#include <stdint.h>
// END EPIC MOD
#include "LPP_API.h"


// external Live++ API exported into DLL
// BEGIN EPIC MOD - Internalizing API
#define LPP_DLL_API(_rv) _rv __cdecl
//#define LPP_DLL_API(_rv) extern "C" __declspec(dllexport) _rv __cdecl
// END EPIC MOD - Internalizing API

LPP_DLL_API(void) LppStartup(void);
LPP_DLL_API(void) LppShutdown(void);
LPP_DLL_API(const char*) LppGetVersion(void);
LPP_DLL_API(int) LppCheckVersion(const char* apiVersion);
LPP_DLL_API(void) LppRegisterProcessGroup(const char* groupName);
LPP_DLL_API(void) LppSyncPoint(void);
LPP_DLL_API(void) LppWaitForToken(void* token);
// BEGIN EPIC MOD - Adding LppTryWaitForToken
LPP_DLL_API(bool) LppTryWaitForToken(void* token);
// END EPIC MOD
LPP_DLL_API(void) LppTriggerRecompile(void);
LPP_DLL_API(void) LppLogMessage(const wchar_t* message);
LPP_DLL_API(void) LppBuildPatch(const wchar_t* moduleNames[], const wchar_t* objPaths[], const wchar_t* amalgamatedObjPaths[], unsigned int count);
LPP_DLL_API(void) LppInstallExceptionHandler(void);
LPP_DLL_API(void) LppUseExternalBuildSystem(void);
LPP_DLL_API(void) LppTriggerRestart(void);
LPP_DLL_API(int) LppWantsRestart(void);
LPP_DLL_API(void) LppRestart(lpp::RestartBehaviour behaviour, unsigned int exitCode);
LPP_DLL_API(void*) LppEnableModule(const wchar_t* nameOfExeOrDll);
LPP_DLL_API(void*) LppEnableModules(const wchar_t* namesOfExeOrDll[], unsigned int count);
LPP_DLL_API(void*) LppEnableAllModules(const wchar_t* nameOfExeOrDll);
LPP_DLL_API(void*) LppDisableModule(const wchar_t* nameOfExeOrDll);
LPP_DLL_API(void*) LppDisableModules(const wchar_t* namesOfExeOrDll[], unsigned int count);
LPP_DLL_API(void*) LppDisableAllModules(const wchar_t* nameOfExeOrDll);
// BEGIN EPIC MOD - Additional functions
LPP_DLL_API(void) LppShowConsole();
LPP_DLL_API(void) LppSetVisible(bool visible);
LPP_DLL_API(void) LppSetActive(bool active);
LPP_DLL_API(void) LppSetBuildArguments(const wchar_t* arguments);
LPP_DLL_API(void*) LppEnableLazyLoadedModule(const wchar_t* nameOfExeOrDll);
LPP_DLL_API(void) LppSetReinstancingFlow(bool enable);
LPP_DLL_API(void) LppDisableCompileFinishNotification();
LPP_DLL_API(void*) LppEnableModulesEx(const wchar_t* moduleNames[], unsigned int moduleCount, const wchar_t* lazyLoadModuleNames[], unsigned int lazyLoadModuleCount, const uintptr_t* reservedPages, unsigned int reservedPagesCount);
// END EPIC MOD
LPP_DLL_API(void) LppApplySettingBool(const char* settingName, int value);
LPP_DLL_API(void) LppApplySettingInt(const char* settingName, int value);
LPP_DLL_API(void) LppApplySettingString(const char* settingName, const wchar_t* value);


#endif // LC_VERSION