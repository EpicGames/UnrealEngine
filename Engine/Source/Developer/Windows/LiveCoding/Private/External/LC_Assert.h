// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "LC_Platform.h"
#include "LC_Debugger_Windows.h"
#include "LC_Logging.h"
// END EPIC MOD


#if LC_DEBUG
#	define LC_ASSERT_BREAKPOINT()				(Debugger::IsConnected()) ? LC_DEBUGGER_BREAKPOINT() : (void)true
#	define LC_ASSERT(_condition, _msg)			(_condition) ? (void)true : (LC_ERROR_DEV("%s", _msg), LC_ASSERT_BREAKPOINT())
#else
#	define LC_BREAKPOINT()						(void)true
#	define LC_ASSERT(_condition, _msg)			LC_UNUSED(_condition)
#endif


#endif // LC_VERSION