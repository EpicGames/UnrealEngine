// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Atomic_Windows.h"
#include "LC_Foundation_Windows.h"


void Atomic::IncrementConsistent(volatile int32_t* value)
{
	LC_COMPILER_FENCE;
	LC_MEMORY_FENCE;
	LC_COMPILER_FENCE;
	::_InterlockedIncrement(reinterpret_cast<volatile long*>(value));
	LC_COMPILER_FENCE;
	LC_MEMORY_FENCE;
	LC_COMPILER_FENCE;
}


void Atomic::DecrementConsistent(volatile int32_t* value)
{
	LC_COMPILER_FENCE;
	LC_MEMORY_FENCE;
	LC_COMPILER_FENCE;
	::_InterlockedDecrement(reinterpret_cast<volatile long*>(value));
	LC_COMPILER_FENCE;
	LC_MEMORY_FENCE;
	LC_COMPILER_FENCE;
}


#endif // LC_VERSION