// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include <cinttypes>
// END EPIC MOD

namespace Atomic
{
	// Increments a value atomically, guaranteeing sequential consistency.
	void IncrementConsistent(volatile int32_t* value);

	// Decrements a value atomically, guaranteeing sequential consistency.
	void DecrementConsistent(volatile int32_t* value);
}


#endif // LC_VERSION