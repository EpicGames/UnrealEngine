// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD
#include <type_traits>
#pragma intrinsic(_BitScanReverse)

namespace bitUtil
{
	template <typename T>
	inline T RoundUpToMultiple(T numToRound, T multipleOf)
	{
		static_assert(std::is_unsigned<T>::value == true, "T must be an unsigned type.");
		return (numToRound + (multipleOf - 1u)) & ~(multipleOf - 1u);
	}


	// http://www.graphics.stanford.edu/~seander/bithacks.html#DetermineIfPowerOf2
	template <typename T>
	inline bool IsPowerOfTwo(T value)
	{
		static_assert(std::is_unsigned<T>::value == true, "T must be an unsigned type.");
		return value && !(value & (value - 1u));
	}

	// Finds the first set bit, searching from MSB to LSB.
	// Can be used to determine the log2 of integers that are powers of two.
	inline uint32_t BitScanReverse(uint32_t value)
	{
		DWORD index = 0u;
		::_BitScanReverse(&index, value);
		return index;
	}
}


#endif // LC_VERSION