// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_ClientCommandActions.h"
#include "LC_ClientUserCommandThread.h"
#include "LC_DuplexPipe.h"
#include "LC_SyncPoint.h"
#include "LC_Executable.h"
#include "LC_Event.h"
#include "LC_Process.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
// END EPIC MOD

bool actions::RegisterProcessFinished::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	bool* successfullyRegisteredProcess = static_cast<bool*>(context);
	*successfullyRegisteredProcess = command->success;

	pipe->SendAck();

	// don't continue execution
	return false;
}


bool actions::EnableModulesFinished::Execute(const CommandType* command, const DuplexPipe* pipe, void*, const void*, size_t)
{
	Event* event = static_cast<Event*>(command->token);
	event->Signal();
	pipe->SendAck();

	return false;
}


bool actions::DisableModulesFinished::Execute(const CommandType* command, const DuplexPipe* pipe, void*, const void*, size_t)
{
	Event* event = static_cast<Event*>(command->token);
	event->Signal();
	pipe->SendAck();

	return false;
}


bool actions::EnterSyncPoint::Execute(const CommandType*, const DuplexPipe* pipe, void*, const void*, size_t)
{
	syncPoint::Enter();
	pipe->SendAck();

	return true;
}


bool actions::LeaveSyncPoint::Execute(const CommandType*, const DuplexPipe* pipe, void*, const void*, size_t)
{
	syncPoint::Leave();
	pipe->SendAck();

	return true;
}


bool actions::CallHooks::Execute(const CommandType* command, const DuplexPipe* pipe, void*, const void* payload, size_t)
{
	switch (command->type)
	{
		case hook::Type::PREPATCH:
			hook::CallHooksInRange<hook::PrepatchFunction>(command->rangeBegin, command->rangeEnd);
			break;

		case hook::Type::POSTPATCH:
			hook::CallHooksInRange<hook::PostpatchFunction>(command->rangeBegin, command->rangeEnd);
			break;

		case hook::Type::COMPILE_START:
			hook::CallHooksInRange<hook::CompileStartFunction>(command->rangeBegin, command->rangeEnd);
			break;

		case hook::Type::COMPILE_SUCCESS:
			hook::CallHooksInRange<hook::CompileSuccessFunction>(command->rangeBegin, command->rangeEnd);
			break;

		case hook::Type::COMPILE_ERROR:
			hook::CallHooksInRange<hook::CompileErrorFunction>(command->rangeBegin, command->rangeEnd);
			break;

		case hook::Type::COMPILE_ERROR_MESSAGE:
			hook::CallHooksInRange<hook::CompileErrorMessageFunction>(command->rangeBegin, command->rangeEnd, static_cast<const wchar_t*>(payload));
			break;
	}

	pipe->SendAck();

	return true;
}


// BEGIN EPIC MOD - Support for UE debug visualizers
extern uint8** GNameBlocksDebug;

class FChunkedFixedUObjectArray;
extern FChunkedFixedUObjectArray*& GObjectArrayForDebugVisualizers;

namespace UE::Core{ struct FVisualizerDebuggingState; }
extern UE::Core::FVisualizerDebuggingState*& GDebuggingState;
// END EPIC MOD

// BEGIN EPIC MOD - Support for object reinstancing. We need to call a global post-patch handler, rather than just getting individual callbacks for modified modules.
extern void LiveCodingBeginPatch();
// END EPIC MOD

// BEGIN EPIC MOD - Notification that compilation has finished
extern void LiveCodingEndCompile();
// END EPIC MOD

// BEGIN EPIC MOD
extern void LiveCodingPreCompile();
extern void LiveCodingPostCompile(commands::PostCompileResult postCompileResult);
extern void LiveCodingTriggerReload();
// END EPIC MOD

bool actions::LoadPatch::Execute(const CommandType* command, const DuplexPipe* pipe, void*, const void*, size_t)
{
	// load library into this process
	HMODULE module = ::LoadLibraryW(command->path);

	// BEGIN EPIC MOD - Support for UE debug visualizers
	if (module != nullptr)
	{
		typedef void InitNatvisHelpersFunc(uint8** NameTable, FChunkedFixedUObjectArray* ObjectArray, UE::Core::FVisualizerDebuggingState* DebuggingState);

		InitNatvisHelpersFunc* InitNatvisHelpers = (InitNatvisHelpersFunc*)(void*)GetProcAddress(module, "InitNatvisHelpers");
		if (InitNatvisHelpers != nullptr)
		{
			(*InitNatvisHelpers)(GNameBlocksDebug, GObjectArrayForDebugVisualizers, GDebuggingState);
		}
	}
	// END EPIC MOD
	// BEGIN EPIC MOD - Support for object reinstancing. We need to call a global post-patch handler, rather than just getting individual callbacks for modified modules.
	LiveCodingBeginPatch();
	// END EPIC MOD

	pipe->SendAck();

	// send back command with module info
	pipe->SendCommandAndWaitForAck(commands::LoadPatchInfo { module }, nullptr, 0u);

	return true;
}


bool actions::UnloadPatch::Execute(const CommandType* command, const DuplexPipe* pipe, void*, const void*, size_t)
{
	// unload library from this process
	::FreeLibrary(command->module);
	pipe->SendAck();

	return true;
}


bool actions::CallEntryPoint::Execute(const CommandType* command, const DuplexPipe* pipe, void*, const void*, size_t)
{
	executable::CallDllEntryPoint(command->moduleBase, command->entryPointRva);
	pipe->SendAck();

	return true;
}


bool actions::LogOutput::Execute(const CommandType*, const DuplexPipe* pipe, void*, const void* payload, size_t)
{
	// BEGIN EPIC MOD
	Logging::LogNoFormat<Logging::Channel::USER>(static_cast<const wchar_t*>(payload));
	// END EPIC MOD
	pipe->SendAck();

	return true;
}

bool actions::CompilationFinished::Execute(const CommandType*, const DuplexPipe* pipe, void*, const void*, size_t)
{
	pipe->SendAck();

	// BEGIN EPIC MOD - Notification that compilation has finished
	LiveCodingEndCompile();
	// END EPIC MOD

	// don't continue execution
	return false;
}


bool actions::HandleExceptionFinished::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	ClientUserCommandThread::ExceptionResult* resultContext = static_cast<ClientUserCommandThread::ExceptionResult*>(context);
	resultContext->returnAddress = command->returnAddress;
	resultContext->framePointer = command->framePointer;
	resultContext->stackPointer = command->stackPointer;
	resultContext->continueExecution = command->continueExecution;

	pipe->SendAck();

	// don't continue execution
	return false;
}


// BEGIN EPIC MOD
bool actions::PreCompile::Execute(const CommandType*, const DuplexPipe* pipe, void*, const void*, size_t)
{
	LiveCodingPreCompile();
	pipe->SendAck();

	return true;
}


bool actions::PostCompile::Execute(const CommandType* command, const DuplexPipe* pipe, void*, const void*, size_t)
{
	LiveCodingPostCompile(command->postCompileResult);
	pipe->SendAck();

	return true;
}


bool actions::TriggerReload::Execute(const CommandType*, const DuplexPipe* pipe, void*, const void*, size_t)
{
	LiveCodingTriggerReload();
	syncPoint::Leave();
	syncPoint::Enter();
	pipe->SendAck();

	return true;
}
// END EPIC MOD

#endif // LC_VERSION