// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Commands.h"


class DuplexPipe;

namespace actions
{
	#define DECLARE_ACTION(_name)																													\
		struct _name																																\
		{																																			\
			typedef ::commands::_name CommandType;																									\
			static bool Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void* payload, size_t payloadSize);		\
		}

	DECLARE_ACTION(RegisterProcessFinished);
	DECLARE_ACTION(EnableModulesFinished);
	DECLARE_ACTION(DisableModulesFinished);
	DECLARE_ACTION(EnterSyncPoint);
	DECLARE_ACTION(LeaveSyncPoint);
	DECLARE_ACTION(CallHooks);
	DECLARE_ACTION(LoadPatch);
	DECLARE_ACTION(UnloadPatch);
	DECLARE_ACTION(CallEntryPoint);
	DECLARE_ACTION(LogOutput);
	DECLARE_ACTION(CompilationFinished);
	DECLARE_ACTION(HandleExceptionFinished);
	// BEGIN EPIC MOD
	DECLARE_ACTION(PreCompile);
	DECLARE_ACTION(PostCompile);
	DECLARE_ACTION(TriggerReload);
	// END EPIC MOD

	#undef DECLARE_ACTION
}


#endif // LC_VERSION