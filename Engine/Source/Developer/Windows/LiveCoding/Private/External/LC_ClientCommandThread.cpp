// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_ClientCommandThread.h"
#include "LC_Event.h"
#include "LC_CommandMap.h"
#include "LC_CriticalSection.h"
#include "LC_ClientCommandActions.h"
#include "LC_DuplexPipeClient.h"
#include "LC_Process.h"
#include "LC_HeartBeat.h"
#include "LC_Thread.h"


ClientCommandThread::ClientCommandThread(DuplexPipeClient* pipeClient)
	: m_thread(Thread::INVALID_HANDLE)
	, m_pipe(pipeClient)
{
}


ClientCommandThread::~ClientCommandThread(void)
{
}


Thread::Id ClientCommandThread::Start(const std::wstring& processGroupName, Event* compilationEvent, Event* waitForStartEvent, CriticalSection* pipeAccessCS)
{
	// spawn a thread that communicates with the server
	// BEGIN EPIC MOD
	m_thread = Thread::CreateFromMemberFunction("Live coding commands", 128u * 1024u, this, &ClientCommandThread::ThreadFunction, processGroupName, compilationEvent, waitForStartEvent, pipeAccessCS);
	// END EPIC MOD

	return Thread::GetId(m_thread);
}


void ClientCommandThread::Join(void)
{
	if (m_thread != Thread::INVALID_HANDLE)
	{
		Thread::Join(m_thread);
		Thread::Close(m_thread);
	}
}


Thread::ReturnValue ClientCommandThread::ThreadFunction(const std::wstring& processGroupName, Event* compilationEvent, Event* waitForStartEvent, CriticalSection* pipeAccessCS)
{
	waitForStartEvent->Wait();

	CommandMap commandMap;
	commandMap.RegisterAction<actions::LoadPatch>();
	commandMap.RegisterAction<actions::UnloadPatch>();
	commandMap.RegisterAction<actions::EnterSyncPoint>();
	commandMap.RegisterAction<actions::LeaveSyncPoint>();
	commandMap.RegisterAction<actions::CallEntryPoint>();
	commandMap.RegisterAction<actions::CallHooks>();
	commandMap.RegisterAction<actions::LogOutput>();
	commandMap.RegisterAction<actions::CompilationFinished>();
	// BEGIN EPIC MOD
	commandMap.RegisterAction<actions::PreCompile>();
	commandMap.RegisterAction<actions::PostCompile>();
	commandMap.RegisterAction<actions::TriggerReload>();
	// END EPIC MOD

	HeartBeat heartBeat(processGroupName.c_str(), Process::Current::GetId());

	for (;;)
	{
		// wait for compilation to start
		while (!compilationEvent->WaitTimeout(10))
		{
			if (!m_pipe->IsValid())
			{
				// pipe was closed or is broken, bail out
				return Thread::ReturnValue(1u);
			}

			heartBeat.Store();
		}

		if (!m_pipe->IsValid())
		{
			// pipe was closed or is broken, bail out
			return Thread::ReturnValue(1u);
		}

		// lock critical section for accessing the pipe.
		// we need to make sure that other threads talking through the pipe don't use it at the same time.
		CriticalSection::ScopedLock lock(pipeAccessCS);

		m_pipe->SendCommandAndWaitForAck(commands::ReadyForCompilation {}, nullptr, 0u);

		commandMap.HandleCommands(m_pipe, nullptr);
	}

	// BEGIN EPIC MOD
	return Thread::ReturnValue(0u);
	// END EPIC MOD
}


#endif // LC_VERSION