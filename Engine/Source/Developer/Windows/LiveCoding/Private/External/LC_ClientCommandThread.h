// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ThreadTypes.h"
// BEGIN EPIC MOD
#include <string>
// END EPIC MOD

class DuplexPipeClient;
class Event;
class CriticalSection;


// handles incoming commands from the Live++ server
class ClientCommandThread
{
public:
	explicit ClientCommandThread(DuplexPipeClient* pipeClient);
	~ClientCommandThread(void);

	// Starts the thread that takes care of handling incoming commands on the pipe.
	// Returns the thread ID.
	Thread::Id Start(const std::wstring& processGroupName, Event* compilationEvent, Event* waitForStartEvent, CriticalSection* pipeAccessCS);

	// Joins this thread.
	void Join(void);

private:
	Thread::ReturnValue ThreadFunction(const std::wstring& processGroupName, Event* compilationEvent, Event* waitForStartEvent, CriticalSection* pipeAccessCS);

	Thread::Handle m_thread;
	DuplexPipeClient* m_pipe;
};


#endif // LC_VERSION