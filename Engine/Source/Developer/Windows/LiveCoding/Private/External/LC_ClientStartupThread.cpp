// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_ClientStartupThread.h"
#include "LC_StringUtil.h"
#include "LC_NamedSharedMemory.h"
#include "LC_InterprocessMutex.h"
#include "LC_DuplexPipeClient.h"
#include "LC_CommandMap.h"
#include "LC_ClientCommandActions.h"
#include "LC_ClientCommandThread.h"
#include "LC_ClientUserCommandThread.h"
#include "LC_Event.h"
#include "LC_CriticalSection.h"
#include "LC_PrimitiveNames.h"
#include "LC_Environment.h"
#include "LC_MemoryStream.h"
#include "LC_Thread.h"
#include "LC_Process.h"
#include "LPP_API.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
// END EPIC MOD

// JumpToSelf is an extern function coming from assembler source
extern void JumpToSelf(void);

namespace
{
	template <typename T>
	static void DeleteAndNull(T*& instance)
	{
		delete instance;
		instance = nullptr;
	}
}


ClientStartupThread::ClientStartupThread(void)
	: m_thread(Thread::INVALID_HANDLE)
	, m_job(nullptr)
	, m_sharedMemory(nullptr)
	, m_mainProcessContext(nullptr)
	, m_processHandle(nullptr)
	, m_successfulInit(false)
	, m_pipeClient(nullptr)
	, m_exceptionPipeClient(nullptr)
	, m_pipeClientCS(nullptr)
	, m_commandThread(nullptr)
	, m_userCommandThread(nullptr)
	, m_startEvent(nullptr)
	, m_compilationEvent(nullptr)
{
	m_pipeClient = new DuplexPipeClient;
	m_exceptionPipeClient = new DuplexPipeClient;
	m_commandThread = new ClientCommandThread(m_pipeClient);
	m_userCommandThread = new ClientUserCommandThread(m_pipeClient, m_exceptionPipeClient);
}


ClientStartupThread::~ClientStartupThread(void)
{
	// close the pipe and then wait for the helper threads to finish.
	// closing the pipe bails out the helper threads.
	if (m_pipeClient)
	{
		// give the server a chance to deal with disconnected clients
		if (m_pipeClient->IsValid())
		{
			m_pipeClient->SendCommandAndWaitForAck(commands::DisconnectClient {}, nullptr, 0u);
		}
		m_pipeClient->Close();
	}

	if (m_exceptionPipeClient)
	{
		m_exceptionPipeClient->Close();
	}

	// wait for command thread to finish
	if (m_commandThread)
	{
		m_commandThread->Join();
	}

	// bail out user command thread and wait for it to finish
	if (m_userCommandThread)
	{
		m_userCommandThread->End();
		m_userCommandThread->Join();
	}

	DeleteAndNull(m_pipeClient);
	DeleteAndNull(m_exceptionPipeClient);
	DeleteAndNull(m_commandThread);
	DeleteAndNull(m_userCommandThread);

	DeleteAndNull(m_startEvent);
	DeleteAndNull(m_compilationEvent);
	DeleteAndNull(m_pipeClientCS);

	if (m_mainProcessContext)
	{
		Process::Destroy(m_mainProcessContext);
	}

	// close job object to make child processes close as well.
	// if this is the last handle we close, the Live++ process will be killed as well.
	::CloseHandle(m_job);

	// clean up interprocess objects
	if (m_sharedMemory)
	{
		Process::DestroyNamedSharedMemory(m_sharedMemory);
	}
}


void ClientStartupThread::Start(const char* const groupName, RunMode::Enum runMode)
{
	// spawn a thread that does all the initialization work.
	// in the context of mutexes, jobs, named shared memory, etc. object names behave similar to
	// file names and are not allowed to contain certain characters.
	std::wstring safeProcessGroupName = string::MakeSafeName(string::ToWideString(groupName));
	// BEGIN EPIC MOD
	m_thread = Thread::CreateFromMemberFunction("Live coding startup", 128u * 1024u, this, &ClientStartupThread::ThreadFunction, safeProcessGroupName, runMode);
	// END EPIC MOD
}


void ClientStartupThread::Join(void)
{
	if (m_thread != Thread::INVALID_HANDLE)
	{
		Thread::Join(m_thread);
		Thread::Close(m_thread);
	}
}


void* ClientStartupThread::EnableModule(const wchar_t* nameOfExeOrDll)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		return m_userCommandThread->EnableModule(nameOfExeOrDll);
	}

	return nullptr;
}


void* ClientStartupThread::EnableModules(const wchar_t* namesOfExeOrDll[], unsigned int count)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		return m_userCommandThread->EnableModules(namesOfExeOrDll, count);
	}

	return nullptr;
}


void* ClientStartupThread::EnableAllModules(const wchar_t* nameOfExeOrDll)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		return m_userCommandThread->EnableAllModules(nameOfExeOrDll);
	}

	return nullptr;
}


void* ClientStartupThread::DisableModule(const wchar_t* nameOfExeOrDll)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		return m_userCommandThread->DisableModule(nameOfExeOrDll);
	}

	return nullptr;
}


void* ClientStartupThread::DisableModules(const wchar_t* namesOfExeOrDll[], unsigned int count)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		return m_userCommandThread->DisableModules(namesOfExeOrDll, count);
	}

	return nullptr;
}


void* ClientStartupThread::DisableAllModules(const wchar_t* nameOfExeOrDll)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		return m_userCommandThread->DisableAllModules(nameOfExeOrDll);
	}

	return nullptr;
}

// BEGIN EPIC MOD - Adding TryWaitForToken
bool ClientStartupThread::TryWaitForToken(void* token)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		return m_userCommandThread->TryWaitForToken(token);
	}

	// If the command thread doesn't exist yet, return it's not ready yet.
	return false;
}
// END EPIC MOD

void ClientStartupThread::WaitForToken(void* token)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->WaitForToken(token);
	}
}


void ClientStartupThread::TriggerRecompile(void)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->TriggerRecompile();
	}
}


void ClientStartupThread::LogMessage(const wchar_t* message)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->LogMessage(message);
	}
}


void ClientStartupThread::BuildPatch(const wchar_t* moduleNames[], const wchar_t* objPaths[], const wchar_t* amalgamatedObjPaths[], unsigned int count)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->BuildPatch(moduleNames, objPaths, amalgamatedObjPaths, count);
	}
}


void ClientStartupThread::InstallExceptionHandler(void)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->InstallExceptionHandler();
	}
}


void ClientStartupThread::TriggerRestart(void)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->TriggerRestart();
	}
}


// BEGIN EPIC MOD - Adding ShowConsole command
void ClientStartupThread::ShowConsole(void)
{
	// we cannot wait for commands in the user command thread as long as startup hasn't finished
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->ShowConsole();
	}
}
// END EPIC MOD


// BEGIN EPIC MOD - Adding SetVisible command
void ClientStartupThread::SetVisible(bool visible)
{
	// we cannot wait for commands in the user command thread as long as startup hasn't finished
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->SetVisible(visible);
	}
}
// END EPIC MOD


// BEGIN EPIC MOD - Adding SetActive command
void ClientStartupThread::SetActive(bool active)
{
	// we cannot wait for commands in the user command thread as long as startup hasn't finished
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->SetActive(active);
	}
}
// END EPIC MOD


// BEGIN EPIC MOD - Adding SetBuildArguments command
void ClientStartupThread::SetBuildArguments(const wchar_t* arguments)
{
	// we cannot wait for commands in the user command thread as long as startup hasn't finished
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->SetBuildArguments(arguments);
	}
}
// END EPIC MOD

// BEGIN EPIC MOD - Support for lazy-loading modules
void* ClientStartupThread::EnableLazyLoadedModule(const wchar_t* fileName, Windows::HMODULE moduleBase)
{
	// we cannot wait for commands in the user command thread as long as startup hasn't finished
	Join();

	if (m_userCommandThread)
	{
		return m_userCommandThread->EnableLazyLoadedModule(fileName, moduleBase);
	}

	return nullptr;
}
// END EPIC MOD

// BEGIN EPIC MOD
void ClientStartupThread::SetReinstancingFlow(bool enable)
{
	// we cannot wait for commands in the user command thread as long as startup hasn't finished
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->SetReinstancingFlow(enable);
	}
}
// END EPIC MOD

// BEGIN EPIC MOD
void ClientStartupThread::DisableCompileFinishNotification()
{
	// we cannot wait for commands in the user command thread as long as startup hasn't finished
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->DisableCompileFinishNotification();
	}
}
// END EPIC MOD

// BEGIN EPIC MOD
void* ClientStartupThread::EnableModulesEx(const wchar_t* moduleNames[], unsigned int moduleCount, const wchar_t* lazyLoadModuleNames[], unsigned int lazyLoadModuleCount, const uintptr_t* reservedPages, unsigned int reservedPagesCount)
{
	// we cannot wait for commands in the user command thread as long as startup hasn't finished
	Join();

	if (m_userCommandThread)
	{
		return m_userCommandThread->EnableModulesEx(moduleNames, moduleCount, lazyLoadModuleNames, lazyLoadModuleCount, reservedPages, reservedPagesCount);
	}

	return nullptr;
}
// END EPIC MOD


void ClientStartupThread::ApplySettingBool(const char* settingName, int value)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->ApplySettingBool(settingName, value);
	}
}


void ClientStartupThread::ApplySettingInt(const char* settingName, int value)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->ApplySettingInt(settingName, value);
	}
}


void ClientStartupThread::ApplySettingString(const char* settingName, const wchar_t* value)
{
	// wait for the startup thread to finish initialization
	Join();

	if (m_userCommandThread)
	{
		m_userCommandThread->ApplySettingString(settingName, value);
	}
}


Thread::ReturnValue ClientStartupThread::ThreadFunction(const std::wstring& processGroupName, RunMode::Enum runMode)
{
	// configure all child processes associated with the job to terminate when the parent terminates.
	// we create (or open) a process-wide job per process group and register the spawned process with that job.
	// when the last handle to the job is closed, it will close the associated process automatically.
	// this nicely handles multi-process scenarios where applications can even be restarted and attach to the
	// same Live++ instance.
	m_job = ::CreateJobObjectW(NULL, primitiveNames::JobGroup(processGroupName).c_str());
	JOBOBJECT_EXTENDED_LIMIT_INFORMATION jobInfo = {};
	// BEGIN EPIC MOD
	// With UE, we can spawn a new editor while letting the existing editor close.  If the editor calling CreateProcess is
	// a child of the live coding console due to "Quick Restart" begin used, then the newly spawned process will be killed
	// when the first editor exits.  By adding the breakaway options (specifically silent), the second editor is 
	// no longer killed.
	jobInfo.BasicLimitInformation.LimitFlags = JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE | JOB_OBJECT_LIMIT_SILENT_BREAKAWAY_OK;
	// END EPIC MOD
	::SetInformationJobObject(m_job, JobObjectExtendedLimitInformation, &jobInfo, sizeof(jobInfo));

	// lock the interprocess mutex to ensure that only one process can run this code at any time.
	// the first one will spawn the Live++ process, all others will connect to the same process.
	{
		InterprocessMutex initProcessMutex(primitiveNames::StartupMutex(processGroupName).c_str());
		InterprocessMutex::ScopedLock mutexLock(&initProcessMutex);

		m_sharedMemory = Process::CreateNamedSharedMemory(primitiveNames::StartupNamedSharedMemory(processGroupName).c_str(), 4096u);
		if (Process::Current::DoesOwnNamedSharedMemory(m_sharedMemory))
		{
			// BEGIN EPIC MOD - Using LiveCodeConsole
			// we are the first DLL. spawn the console.
			LC_LOG_USER("First instance in process group \"%S\", spawning console", processGroupName.c_str());

			// get the path to the console application
			extern FString GLiveCodingConsolePath;
			const std::wstring& exePath = *GLiveCodingConsolePath;

			std::wstring commandLine;
			commandLine += L"-Group=";
			commandLine += processGroupName;

			extern FString GLiveCodingConsoleArguments;
			if(GLiveCodingConsoleArguments.Len() > 0)
			{
				commandLine += L" ";
				commandLine += *GLiveCodingConsoleArguments;
			}
			if (!FApp::IsProjectNameEmpty())
			{
				commandLine += L" -ProjectName=\"";
				commandLine += FApp::GetProjectName();
				commandLine += L"\"";
			}

			m_mainProcessContext = Process::Spawn(exePath.c_str(), nullptr, commandLine.c_str(), nullptr, Process::SpawnFlags::NONE);
			if (Process::GetId(m_mainProcessContext) != Process::Id(0u))
			{
				m_processHandle = Process::GetHandle(m_mainProcessContext);
				::AssignProcessToJobObject(m_job, +m_processHandle);

				// share Live++ process Id with other processes
				Process::WriteNamedSharedMemory(m_sharedMemory, +Process::GetId(m_mainProcessContext));
			}
			// END EPIC MOD - Using LiveCodeConsole
		}
		else
		{
			// the Live++ process is already running. fetch the process ID from shared memory.
			const Process::Id::Type processId = Process::ReadNamedSharedMemory<Process::Id::Type>(m_sharedMemory);
			// BEGIN EPIC MOD
			LC_LOG_USER("Detected running instance in process group \"%S\", connecting to console process (PID: %d)", processGroupName.c_str(), processId);
			// END EPIC MOD

			if (processId != 0u)
			{
				m_processHandle = Process::Open(Process::Id(processId));
				::AssignProcessToJobObject(m_job, +m_processHandle);
			}
		}
	}

	if (+m_processHandle == nullptr)
	{
		// we were unable to open the process, bail out
		// BEGIN EPIC MOD
		LC_ERROR_USER("%s", "Unable to attach to console process");
		// END EPIC MOD
		Process::DestroyNamedSharedMemory(m_sharedMemory);

		return Thread::ReturnValue(1u);
	}

	// wait for server to become ready
	{
		LC_LOG_USER("%s", "Waiting for server");

		Event serverReadyEvent(primitiveNames::ServerReadyEvent(processGroupName).c_str(), Event::Type::AUTO_RESET);
		serverReadyEvent.Wait();
	}

	// create a named duplex pipe for communicating between DLL and Live++ process
	if (!m_pipeClient->Connect(primitiveNames::Pipe(processGroupName).c_str()))
	{
		// could not connect to Live++ process
		// BEGIN EPIC MOD
		LC_ERROR_USER("%s", "Could not connect named pipe to console process");
		// END EPIC MOD

		return Thread::ReturnValue(2u);
	}

	// create a named duplex pipe for communicating exceptions between DLL and Live++ process
	if (!m_exceptionPipeClient->Connect(primitiveNames::ExceptionPipe(processGroupName).c_str()))
	{
		// could not connect to Live++ process
		// BEGIN EPIC MOD
		LC_ERROR_USER("%s", "Could not connect exception pipe to console process");
		// END EPIC MOD

		return Thread::ReturnValue(3u);
	}

	m_pipeClientCS = new CriticalSection;

	// the Live++ server must be ready. create the interprocess event used for signaling that compilation is about to start
	m_compilationEvent = new Event(primitiveNames::CompilationEvent(processGroupName).c_str(), Event::Type::MANUAL_RESET);

	// create helper threads responsible for handling commands from user calls as well as Live++.
	// both threads are not allowed to run until we send them a signal. this ensures that they don't use the
	// pipe for communicating as long as we aren't finished with it.
	m_startEvent = new Event(nullptr, Event::Type::MANUAL_RESET);

	const Thread::Id commandThreadId = m_commandThread->Start(processGroupName, m_compilationEvent, m_startEvent, m_pipeClientCS);
	m_userCommandThread->Start(processGroupName, m_startEvent, m_pipeClientCS);

	// register this process with Live++
	{
		// try getting the previous process ID from the environment in case the process was restarted
		Process::Id restartedProcessId(0u);
		const std::wstring& processIdStr = environment::GetVariable(L"LPP_PROCESS_RESTART_ID", nullptr);
		if (processIdStr.length() != 0u)
		{
			restartedProcessId = static_cast<unsigned int>(std::stoi(processIdStr));
			environment::RemoveVariable(L"LPP_PROCESS_RESTART_ID");
		}

		// store the current process ID in an environment variable.
		// upon restart, the environment block is inherited by the new process and can be used to map the process IDs of
		// restarted processes to their previous IDs.
		{
			const Process::Id processID = Process::Current::GetId();
			environment::SetVariable(L"LPP_PROCESS_RESTART_ID", std::to_wstring(+processID).c_str());
		}

		const std::wstring imagePath = Process::Current::GetImagePath().GetString();
		const std::wstring& commandLine = Process::Current::GetCommandLine();
		const std::wstring& workingDirectory = Process::Current::GetWorkingDirectory().GetString();
		Process::Environment environment = Process::CreateEnvironment(Process::Current::GetHandle());

		const commands::RegisterProcess command =
		{
			Process::Current::GetBase(), Process::Current::GetId(), restartedProcessId, commandThreadId, reinterpret_cast<void*>(&JumpToSelf),
			(imagePath.size() + 1u) * sizeof(wchar_t), 
			(commandLine.size() + 1u) * sizeof(wchar_t),
			(workingDirectory.size() + 1u) * sizeof(wchar_t),
			environment.size
		};

		memoryStream::Writer payload(command.imagePathSize + command.commandLineSize + command.workingDirectorySize + command.environmentSize);
		payload.Write(imagePath.data(), command.imagePathSize);
		payload.Write(commandLine.data(), command.commandLineSize);
		payload.Write(workingDirectory.data(), command.workingDirectorySize);
		payload.Write(environment.data, environment.size);

		m_pipeClient->SendCommandAndWaitForAck(command, payload.GetData(), payload.GetSize());

		Process::DestroyEnvironment(environment);
	}

	// handle commands until registration is finished
	{
		CommandMap commandMap;
		commandMap.RegisterAction<actions::RegisterProcessFinished>();
		commandMap.HandleCommands(m_pipeClient, &m_successfulInit);
	}

	if (!m_successfulInit)
	{
		// process could not be registered, bail out
		// BEGIN EPIC MOD
		LC_ERROR_USER("%s", "Could not register live coding process");
		// END EPIC MOD

		// close the pipe and then wait for the helper threads to finish.
		// closing the pipe bails out the helper threads.
		m_pipeClient->Close();
		m_exceptionPipeClient->Close();

		// let the threads run *after* we've closed the pipe, otherwise they could have tried communicating
		// with the server in the mean time.
		m_startEvent->Signal();

		// bail out command thread and wait for it
		m_compilationEvent->Signal();
		m_commandThread->Join();

		// bail out user command thread and wait for it
		m_userCommandThread->End();
		m_userCommandThread->Join();

		DeleteAndNull(m_pipeClient);
		DeleteAndNull(m_exceptionPipeClient);
		DeleteAndNull(m_commandThread);
		DeleteAndNull(m_userCommandThread);

		DeleteAndNull(m_startEvent);
		DeleteAndNull(m_compilationEvent);
		DeleteAndNull(m_pipeClientCS);

		return Thread::ReturnValue(3u);
	}

	LC_LOG_USER("%s", "Successfully initialized, removing startup thread");

	// helper threads are now allowed to run, we're finished with the pipe
	m_startEvent->Signal();

	return Thread::ReturnValue(0u);
}


#endif // LC_VERSION