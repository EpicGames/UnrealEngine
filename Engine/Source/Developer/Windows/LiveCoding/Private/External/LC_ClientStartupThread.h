// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ThreadTypes.h"
#include "LC_ProcessTypes.h"
#include "LC_NamedSharedMemoryTypes.h"
#include "LC_RunMode.h"
// BEGIN EPIC MOD
#include <string>
// END EPIC MOD

class ClientCommandThread;
class ClientUserCommandThread;
class DuplexPipeClient;
class CriticalSection;
class Event;


class ClientStartupThread
{
public:
	ClientStartupThread(void);
	~ClientStartupThread(void);

	// Spawns a thread that runs client initialization
	void Start(const char* const groupName, RunMode::Enum runMode);

	// Joins the thread, waiting for initialization to finish
	void Join(void);

	void* EnableModule(const wchar_t* nameOfExeOrDll);
	void* EnableModules(const wchar_t* namesOfExeOrDll[], unsigned int count);
	void* EnableAllModules(const wchar_t* nameOfExeOrDll);

	void* DisableModule(const wchar_t* nameOfExeOrDll);
	void* DisableModules(const wchar_t* namesOfExeOrDll[], unsigned int count);
	void* DisableAllModules(const wchar_t* nameOfExeOrDll);

	// BEGIN EPIC MOD - Adding TryWaitForToken
	bool TryWaitForToken(void* token);
	// END EPIC MOD

	void WaitForToken(void* token);
	void TriggerRecompile(void);
	void LogMessage(const wchar_t* message);
	void BuildPatch(const wchar_t* moduleNames[], const wchar_t* objPaths[], const wchar_t* amalgamatedObjPaths[], unsigned int count);

	void InstallExceptionHandler(void);

	void TriggerRestart(void);

	void ApplySettingBool(const char* settingName, int value);
	void ApplySettingInt(const char* settingName, int value);
	void ApplySettingString(const char* settingName, const wchar_t* value);

	// BEGIN EPIC MOD - Adding ShowConsole command
	void ShowConsole();
	// END EPIC MOD

	// BEGIN EPIC MOD - Adding SetVisible command
	void SetVisible(bool visible);
	// END EPIC MOD

	// BEGIN EPIC MOD - Adding SetActive command
	void SetActive(bool active);
	// END EPIC MOD

	// BEGIN EPIC MOD - Adding SetBuildArguments command
	void SetBuildArguments(const wchar_t* arguments);
	// END EPIC MOD

	// BEGIN EPIC MOD - Support for lazy-loading modules
	void* EnableLazyLoadedModule(const wchar_t* fileName, Windows::HMODULE moduleBase);
	// END EPIC MOD

	// BEGIN EPIC MOD
	void SetReinstancingFlow(bool Enable);
	// END EPIC MOD

	// BEGIN EPIC MOD
	void DisableCompileFinishNotification();
	// END EPIC MOD

	// BEGIN EPIC MOD
	void* EnableModulesEx(const wchar_t* moduleNames[], unsigned int moduleCount, const wchar_t* lazyLoadModuleNames[], unsigned int lazyLoadModuleCount, const uintptr_t* reservedPages, unsigned int reservedPagesCount);
	// END EPIC MOD

private:
	Thread::ReturnValue ThreadFunction(const std::wstring& groupName, RunMode::Enum runMode);

	Thread::Handle m_thread;

	// job object for associating spawned processes with main process the DLL is loaded into
	HANDLE m_job;

	// named shared memory for sharing the Live++ process ID between processes
	Process::NamedSharedMemory* m_sharedMemory;

	// main Live++ process. context may be empty in case we connected to an existing Live++ process
	Process::Context* m_mainProcessContext;
	Process::Handle m_processHandle;

	bool m_successfulInit;

	// pipe used for interprocess communication
	DuplexPipeClient* m_pipeClient;
	DuplexPipeClient* m_exceptionPipeClient;
	CriticalSection* m_pipeClientCS;

	// helper threads taking care of communication with the Live++ server and user code
	ClientCommandThread* m_commandThread;
	ClientUserCommandThread* m_userCommandThread;

	// manual-reset start event that signals to the helper threads that they can start talking to the pipe
	Event* m_startEvent;

	// process-wide event that is signaled by the Live++ server when compilation is about to begin
	Event* m_compilationEvent;
};


#endif // LC_VERSION