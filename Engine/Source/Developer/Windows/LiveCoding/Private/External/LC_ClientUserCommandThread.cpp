// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_ClientUserCommandThread.h"
#include "LC_CommandMap.h"
#include "LC_DuplexPipeClient.h"
#include "LC_ClientCommandActions.h"
#include "LC_Event.h"
#include "LC_Process.h"
#include "LC_CriticalSection.h"
// BEGIN EPIC MOD
//#include "LC_ClientExceptionHandler.h"
// END EPIC MOD
#include "LC_StringUtil.h"
#include "LC_Executable.h"
#include "LC_MemoryStream.h"
// BEGIN EPIC MOD
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
#include "LC_Thread.h"
#include "LC_Logging.h"
#include <deque>
#include <unordered_set>
// END EPIC MOD


namespace
{
	template <typename T>
	class ProxyCommand : public ClientUserCommandThread::BaseCommand
	{
	public:
		ProxyCommand(bool expectResponse, size_t payloadSize)
			: BaseCommand(expectResponse)
			, m_command()
			, m_payload(payloadSize)
		{
		}

		virtual void Execute(DuplexPipe* pipe) override
		{
			pipe->SendCommandAndWaitForAck(m_command, m_payload.GetData(), m_payload.GetSize());
		}

		T m_command;
		memoryStream::Writer m_payload;

		LC_DISABLE_COPY(ProxyCommand);
		LC_DISABLE_MOVE(ProxyCommand);
		LC_DISABLE_ASSIGNMENT(ProxyCommand);
		LC_DISABLE_MOVE_ASSIGNMENT(ProxyCommand);
	};


	// gathers module data for the given module, its import modules, the import's import modules, and so forth
	// BEGIN EPIC MOD
	static std::vector<commands::ModuleData> GatherImportModuleData(Windows::HMODULE mainModule)
	// END EPIC MOD
	{
		// BEGIN EPIC MOD
		std::vector<commands::ModuleData> moduleDatas;
		// END EPIC MOD
		moduleDatas.reserve(1024u);

		// BEGIN EPIC MOD
		std::unordered_set<std::wstring> loadedModules;
		// END EPIC MOD
		loadedModules.reserve(1024u);

		// BEGIN EPIC MOD
		std::vector<Windows::HMODULE> modules;
		// END EPIC MOD
		modules.reserve(1024u);

		modules.push_back(mainModule);
		while (!modules.empty())
		{
			const Windows::HMODULE module = modules.back();
			modules.pop_back();

			// get the absolute path of the module.
			// this automatically takes care of API sets used by Windows 7 and later. in a nutshell, these API sets
			// allow redirection of an API DLL to an underlying OS DLL, e.g. api-ms-win-core-apiquery-l1-1-0.dll redirects
			// to ntdll.dll.
			wchar_t fullPath[MAX_PATH];
			::GetModuleFileNameW(module, fullPath, MAX_PATH);

			// did we load the imports of this module already?
			auto findIt = loadedModules.find(fullPath);
			if (findIt == loadedModules.end())
			{
				loadedModules.insert(fullPath);

				// add data for this module
				{
					commands::ModuleData moduleData = {};
					moduleData.base = module;
					wcscpy_s(moduleData.path, fullPath);
					moduleDatas.emplace_back(moduleData);
				}

				executable::Image* image = executable::OpenImage(fullPath, Filesystem::OpenMode::READ);
				if (image)
				{
					executable::ImageSectionDB* imageSections = executable::GatherImageSectionDB(image);
					if (imageSections)
					{
						executable::ImportModuleDB* importModules = executable::GatherImportModuleDB(image, imageSections);
						if (importModules)
						{
							for (size_t i = 0; i < importModules->modules.size(); ++i)
							{
								const char* importModulePath = importModules->modules[i].path;

								// only add the import module if it is loaded into the process
								// BEGIN EPIC MOD
								Windows::HMODULE importModule = ::GetModuleHandleA(importModulePath);
								// END EPIC MOD
								if (importModule)
								{
									modules.push_back(importModule);
								}
								else
								{
									LC_ERROR_USER("Cannot enable module %s because it is not loaded by this process.", importModulePath);
								}
							}

							executable::DestroyImportModuleDB(importModules);
						}

						executable::DestroyImageSectionDB(imageSections);
					}

					executable::CloseImage(image);
				}
			}
		}

		return moduleDatas;
	}
}


ClientUserCommandThread::BaseCommand::BaseCommand(bool expectResponse)
	: m_expectResponse(expectResponse)
{
}


ClientUserCommandThread::BaseCommand::~BaseCommand(void)
{
}


bool ClientUserCommandThread::BaseCommand::ExpectsResponse(void) const
{
	return m_expectResponse;
}


ClientUserCommandThread::ClientUserCommandThread(DuplexPipeClient* pipeClient, DuplexPipeClient* exceptionPipeClient)
	: m_thread(Thread::INVALID_HANDLE)
	, m_processGroupName()
	, m_pipe(pipeClient)
	, m_exceptionPipe(exceptionPipeClient)
	, m_userCommandQueue()
	, m_userCommandQueueCS()
	, m_userCommandQueueSema(0, 65535u)
{
}


ClientUserCommandThread::~ClientUserCommandThread(void)
{
}


Thread::Id ClientUserCommandThread::Start(const std::wstring& processGroupName, Event* waitForStartEvent, CriticalSection* pipeAccessCS)
{
	m_processGroupName = processGroupName;

	// spawn a thread that does the work
	// BEGIN EPIC MOD
	m_thread = Thread::CreateFromMemberFunction("Live coding user commands", 128u * 1024u, this, &ClientUserCommandThread::ThreadFunction, waitForStartEvent, pipeAccessCS);
	// END EPIC MOD

	return Thread::GetId(m_thread);
}


void ClientUserCommandThread::Join(void)
{
	if (m_thread != Thread::INVALID_HANDLE)
	{
		Thread::Join(m_thread);
		Thread::Close(m_thread);
	}
}


void* ClientUserCommandThread::EnableModule(const wchar_t* nameOfExeOrDll)
{
	return EnableModules(&nameOfExeOrDll, 1u);
}


void* ClientUserCommandThread::EnableModules(const wchar_t* namesOfExeOrDll[], unsigned int count)
{
	// BEGIN EPIC MOD
	std::vector<Windows::HMODULE> loadedModules;
	// END EPIC MOD
	loadedModules.reserve(count);

	for (unsigned int i = 0u; i < count; ++i)
	{
		// BEGIN EPIC MOD
		Windows::HMODULE module = ::GetModuleHandleW(namesOfExeOrDll[i]);
		// END EPIC MOD
		if (module)
		{
			loadedModules.push_back(module);
		}
		else
		{
			LC_ERROR_USER("Cannot enable module %S because it is not loaded by this process.", namesOfExeOrDll[i]);
		}
	}

	const size_t loadedModuleCount = loadedModules.size();
	if (loadedModuleCount == 0u)
	{
		// nothing to load
		return nullptr;
	}

	ProxyCommand<commands::EnableModules>* proxy = new ProxyCommand<commands::EnableModules>(true, sizeof(commands::ModuleData) * loadedModuleCount);
	proxy->m_command.processId = Process::Current::GetId();
	proxy->m_command.moduleCount = static_cast<unsigned int>(loadedModuleCount);
	proxy->m_command.token = new Event(nullptr, Event::Type::AUTO_RESET);

	for (size_t i = 0u; i < loadedModuleCount; ++i)
	{
		// BEGIN EPIC MOD
		Windows::HMODULE module = loadedModules[i];
		// END EPIC MOD

		commands::ModuleData moduleData = {};
		moduleData.base = module;
		::GetModuleFileNameW(module, moduleData.path, MAX_PATH);

		proxy->m_payload.Write(moduleData);
	}

	PushUserCommand(proxy);

	return proxy->m_command.token;
}


void* ClientUserCommandThread::EnableAllModules(const wchar_t* nameOfExeOrDll)
{
	// BEGIN EPIC MOD
	Windows::HMODULE module = ::GetModuleHandleW(nameOfExeOrDll);
	// END EPIC MOD
	if (!module)
	{
		LC_ERROR_USER("Cannot enable module %S because it is not loaded by this process.", nameOfExeOrDll);
		return nullptr;
	}

	// BEGIN EPIC MOD
	const std::vector<commands::ModuleData>& allModuleData = GatherImportModuleData(module);
	// END EPIC MOD
	const size_t moduleCount = allModuleData.size();

	ProxyCommand<commands::EnableModules>* proxy = new ProxyCommand<commands::EnableModules>(true, sizeof(commands::ModuleData) * moduleCount);
	proxy->m_command.processId = Process::Current::GetId();
	proxy->m_command.moduleCount = static_cast<unsigned int>(moduleCount);
	proxy->m_command.token = new Event(nullptr, Event::Type::AUTO_RESET);

	for (size_t i = 0u; i < moduleCount; ++i)
	{
		const commands::ModuleData& moduleData = allModuleData[i];
		proxy->m_payload.Write(moduleData);
	}

	PushUserCommand(proxy);

	return proxy->m_command.token;
}


void* ClientUserCommandThread::DisableModule(const wchar_t* nameOfExeOrDll)
{
	return DisableModules(&nameOfExeOrDll, 1u);
}


void* ClientUserCommandThread::DisableModules(const wchar_t* namesOfExeOrDll[], unsigned int count)
{
	// BEGIN EPIC MOD
	std::vector<Windows::HMODULE> loadedModules;
	// END EPIC MOD
	loadedModules.reserve(count);

	for (unsigned int i = 0u; i < count; ++i)
	{
		// BEGIN EPIC MOD
		Windows::HMODULE module = ::GetModuleHandleW(namesOfExeOrDll[i]);
		// END EPIC MOD
		if (module)
		{
			loadedModules.push_back(module);
		}
		else
		{
			LC_ERROR_USER("Cannot disable module %S because it is not loaded by this process.", namesOfExeOrDll[i]);
		}
	}

	const size_t loadedModuleCount = loadedModules.size();
	if (loadedModuleCount == 0u)
	{
		// nothing to unload
		return nullptr;
	}

	ProxyCommand<commands::DisableModules>* proxy = new ProxyCommand<commands::DisableModules>(true, sizeof(commands::ModuleData) * loadedModuleCount);
	proxy->m_command.processId = Process::Current::GetId();
	proxy->m_command.moduleCount = static_cast<unsigned int>(loadedModuleCount);
	proxy->m_command.token = new Event(nullptr, Event::Type::AUTO_RESET);

	for (size_t i = 0u; i < loadedModuleCount; ++i)
	{
		// BEGIN EPIC MOD
		Windows::HMODULE module = loadedModules[i];
		// END EPIC MOD

		commands::ModuleData moduleData = {};
		moduleData.base = module;
		::GetModuleFileNameW(module, moduleData.path, MAX_PATH);

		proxy->m_payload.Write(moduleData);
	}

	PushUserCommand(proxy);

	return proxy->m_command.token;
}


void* ClientUserCommandThread::DisableAllModules(const wchar_t* nameOfExeOrDll)
{
	// BEGIN EPIC MOD
	Windows::HMODULE module = ::GetModuleHandleW(nameOfExeOrDll);
	// END EPIC MOD
	if (!module)
	{
		LC_ERROR_USER("Cannot disable module %S because it is not loaded by this process.", nameOfExeOrDll);
		return nullptr;
	}

	// BEGIN EPIC MOD
	const std::vector<commands::ModuleData>& allModuleData = GatherImportModuleData(module);
	// END EPIC MOD
	const size_t moduleCount = allModuleData.size();

	ProxyCommand<commands::DisableModules>* proxy = new ProxyCommand<commands::DisableModules>(true, sizeof(commands::ModuleData) * moduleCount);
	proxy->m_command.processId = Process::Current::GetId();
	proxy->m_command.moduleCount = static_cast<unsigned int>(moduleCount);
	proxy->m_command.token = new Event(nullptr, Event::Type::AUTO_RESET);

	for (size_t i = 0u; i < moduleCount; ++i)
	{
		const commands::ModuleData& moduleData = allModuleData[i];
		proxy->m_payload.Write(moduleData);
	}

	PushUserCommand(proxy);

	return proxy->m_command.token;
}

// BEGIN EPIC MOD - Adding TryWaitForToken
bool ClientUserCommandThread::TryWaitForToken(void* token)
{
	Event* event = static_cast<Event*>(token);

	if (m_thread != Thread::INVALID_HANDLE)
	{
		// thread was successfully initialized, try waiting until the command has been executed in the queue, non-blocking.
		if (event->TryWait())
		{
			delete event;
			return true;
		}
	}

	return false;
}
// END EPIC MOD

void ClientUserCommandThread::WaitForToken(void* token)
{
	Event* event = static_cast<Event*>(token);

	if (m_thread != Thread::INVALID_HANDLE)
	{
		// thread was successfully initialized, wait until the command has been executed in the queue
		event->Wait();
	}

	delete event;
}


void ClientUserCommandThread::TriggerRecompile(void)
{
	ProxyCommand<commands::TriggerRecompile>* proxy = new ProxyCommand<commands::TriggerRecompile>(false, 0u);

	PushUserCommand(proxy);
}


void ClientUserCommandThread::LogMessage(const wchar_t* message)
{
	const size_t lengthWithoutNull = wcslen(message);
	const size_t payloadSize = sizeof(wchar_t) * (lengthWithoutNull + 1u);

	ProxyCommand<commands::LogMessage>* proxy = new ProxyCommand<commands::LogMessage>(false, payloadSize);
	proxy->m_payload.Write(message, payloadSize);

	PushUserCommand(proxy);
}


void ClientUserCommandThread::BuildPatch(const wchar_t* moduleNames[], const wchar_t* objPaths[], const wchar_t* amalgamatedObjPaths[], unsigned int count)
{
	const size_t perFileSize = sizeof(wchar_t) * MAX_PATH * 3u;

	ProxyCommand<commands::BuildPatch>* proxy = new ProxyCommand<commands::BuildPatch>(false, perFileSize*count);
	proxy->m_command.fileCount = count;

	for (unsigned int i = 0u; i < count; ++i)
	{
		commands::BuildPatch::PatchData patchData = {};
		wcscpy_s(patchData.moduleName, moduleNames[i]);
		wcscpy_s(patchData.objPath, objPaths[i]);

		// the amalgamated object paths are optional
		if (amalgamatedObjPaths && amalgamatedObjPaths[i])
		{
			wcscpy_s(patchData.amalgamatedObjPath, amalgamatedObjPaths[i]);
		}

		proxy->m_payload.Write(patchData);
	}

	PushUserCommand(proxy);
}


void ClientUserCommandThread::TriggerRestart(void)
{
	ProxyCommand<commands::TriggerRestart>* proxy = new ProxyCommand<commands::TriggerRestart>(false, 0u);

	PushUserCommand(proxy);
}


void ClientUserCommandThread::ApplySettingBool(const char* settingName, int value)
{
	ProxyCommand<commands::ApplySettingBool>* proxy = new ProxyCommand<commands::ApplySettingBool>(false, 0u);
	strcpy_s(proxy->m_command.settingName, settingName);
	proxy->m_command.settingValue = value;

	PushUserCommand(proxy);
}


void ClientUserCommandThread::ApplySettingInt(const char* settingName, int value)
{
	ProxyCommand<commands::ApplySettingInt>* proxy = new ProxyCommand<commands::ApplySettingInt>(false, 0u);
	strcpy_s(proxy->m_command.settingName, settingName);
	proxy->m_command.settingValue = value;

	PushUserCommand(proxy);
}


void ClientUserCommandThread::ApplySettingString(const char* settingName, const wchar_t* value)
{
	ProxyCommand<commands::ApplySettingString>* proxy = new ProxyCommand<commands::ApplySettingString>(false, 0u);
	strcpy_s(proxy->m_command.settingName, settingName);
	wcscpy_s(proxy->m_command.settingValue, value);

	PushUserCommand(proxy);
}

// BEGIN EPIC MOD - Adding ShowConsole command
void ClientUserCommandThread::ShowConsole()
{
	ProxyCommand<commands::ShowConsole>* proxy = new ProxyCommand<commands::ShowConsole>(false, 0u);
	PushUserCommand(proxy);
}
// END EPIC MOD

// BEGIN EPIC MOD - Adding SetVisible command
void ClientUserCommandThread::SetVisible(bool visible)
{
	ProxyCommand<commands::SetVisible>* proxy = new ProxyCommand<commands::SetVisible>(false, 0u);
	proxy->m_command.visible = visible;

	PushUserCommand(proxy);
}
// END EPIC MOD

// BEGIN EPIC MOD - Adding SetActive command
void ClientUserCommandThread::SetActive(bool active)
{
	ProxyCommand<commands::SetActive>* proxy = new ProxyCommand<commands::SetActive>(false, 0u);
	proxy->m_command.active = active;

	PushUserCommand(proxy);
}
// END EPIC MOD

// BEGIN EPIC MOD - Adding SetBuildArguments command
void ClientUserCommandThread::SetBuildArguments(const wchar_t* arguments)
{
	ProxyCommand<commands::SetBuildArguments>* proxy = new ProxyCommand<commands::SetBuildArguments>(false, 0u);
	proxy->m_command.processId = Process::Current::GetId();
	wcscpy_s(proxy->m_command.arguments, arguments);

	PushUserCommand(proxy);
}
// END EPIC MOD

// BEGIN EPIC MOD - Adding support for lazy-loading modules
void* ClientUserCommandThread::EnableLazyLoadedModule(const wchar_t* fileName, Windows::HMODULE moduleBase)
{
	ProxyCommand<commands::EnableLazyLoadedModule>* proxy = new ProxyCommand<commands::EnableLazyLoadedModule>(true, 0u);
	proxy->m_command.processId = Process::Current::GetId();
	wcscpy_s(proxy->m_command.fileName, fileName);
	proxy->m_command.moduleBase = moduleBase;
	proxy->m_command.token = new Event(nullptr, Event::Type::AUTO_RESET);

	PushUserCommand(proxy);

	return proxy->m_command.token;
}
// END EPIC MOD

// BEGIN EPIC MOD
void ClientUserCommandThread::SetReinstancingFlow(bool enable)
{
	ProxyCommand<commands::SetReinstancingFlow>* proxy = new ProxyCommand<commands::SetReinstancingFlow>(false, 0u);
	proxy->m_command.processId = Process::Current::GetId();
	proxy->m_command.enable = enable;

	PushUserCommand(proxy);
}
// END EPIC MOD

// BEGIN EPIC MOD
void ClientUserCommandThread::DisableCompileFinishNotification()
{
	ProxyCommand<commands::DisableCompileFinishNotification>* proxy = new ProxyCommand<commands::DisableCompileFinishNotification>(false, 0u);
	proxy->m_command.processId = Process::Current::GetId();

	PushUserCommand(proxy);
}
// END EPIC MOD

// BEGIN EPIC MOD
namespace
{
	std::vector<Windows::HMODULE> GatherModuleHandles(const wchar_t* moduleNames[], unsigned int moduleCount)
	{
		std::vector<Windows::HMODULE> moduleHandles;
		moduleHandles.reserve(moduleCount);

		for (unsigned int i = 0u; i < moduleCount; ++i)
		{
			Windows::HMODULE module = ::GetModuleHandleW(moduleNames[i]);
			if (module)
			{
				moduleHandles.push_back(module);
			}
			else
			{
				LC_ERROR_USER("Cannot enable module %S because it is not loaded by this process.", moduleNames[i]);
			}
		}
		return moduleHandles;
	}

	void AppendModuleHandles(ProxyCommand<commands::EnableModulesEx>* proxy, const std::vector<Windows::HMODULE>& moduleHandles)
	{
		for (Windows::HMODULE moduleHandle : moduleHandles)
		{
			commands::ModuleData moduleData = {};
			moduleData.base = moduleHandle;
			::GetModuleFileNameW(moduleHandle, moduleData.path, MAX_PATH);
			proxy->m_payload.Write(moduleData);
		}
	}
}

void* ClientUserCommandThread::EnableModulesEx(const wchar_t* moduleNames[], unsigned int moduleCount, const wchar_t* lazyLoadModuleNames[], unsigned int lazyLoadModuleCount, const uintptr_t* reservedPages, unsigned int reservedPagesCount)
{
	std::vector<Windows::HMODULE> moduleHandles = GatherModuleHandles(moduleNames, moduleCount);
	std::vector<Windows::HMODULE> lazyLoadModuleHandles = GatherModuleHandles(lazyLoadModuleNames, lazyLoadModuleCount);
	if (moduleHandles.size() == 0 && lazyLoadModuleHandles.size() == 0)
	{
		return nullptr;
	}

	ProxyCommand<commands::EnableModulesEx>* proxy = new ProxyCommand<commands::EnableModulesEx>(true,
		sizeof(commands::ModuleData) * moduleHandles.size() +
		sizeof(commands::ModuleData) * lazyLoadModuleHandles.size() +
		sizeof(uintptr_t) * reservedPagesCount);

	proxy->m_command.processId = Process::Current::GetId();
	proxy->m_command.moduleCount = static_cast<unsigned int>(moduleHandles.size());
	proxy->m_command.lazyLoadModuleCount = static_cast<unsigned int>(lazyLoadModuleHandles.size());
	proxy->m_command.reservedPagesCount = reservedPagesCount;
	proxy->m_command.token = new Event(nullptr, Event::Type::AUTO_RESET);
	AppendModuleHandles(proxy, moduleHandles);
	AppendModuleHandles(proxy, lazyLoadModuleHandles);
	for (unsigned int i = 0; i < reservedPagesCount; ++i)
	{
		proxy->m_payload.Write(reservedPages[i]);
	}

	PushUserCommand(proxy);

	return proxy->m_command.token;

}
// END EPIC MOD

void ClientUserCommandThread::InstallExceptionHandler(void)
{
	// BEGIN EPIC MOD - Using internal CrashReporter
	// exceptionHandler::Register(this);
	// END EPIC MOD
}


ClientUserCommandThread::ExceptionResult ClientUserCommandThread::HandleException(EXCEPTION_RECORD* exception, CONTEXT* context, Thread::Id threadId)
{
	commands::HandleException serverCommand;
	serverCommand.processId = Process::Current::GetId();
	serverCommand.threadId = threadId;
	serverCommand.exception = *exception;
	serverCommand.context = *context;
	serverCommand.clientContextPtr = context;

	m_exceptionPipe->SendCommandAndWaitForAck(serverCommand, nullptr, 0u);

	ExceptionResult result = {};

	CommandMap commandMap;
	commandMap.RegisterAction<actions::HandleExceptionFinished>();
	commandMap.HandleCommands(m_exceptionPipe, &result);

	return result;
}


void ClientUserCommandThread::End(void)
{
	// signal to the thread that a new item is in the queue to make it break out of its main loop
	PushUserCommand(nullptr);
}


void ClientUserCommandThread::PushUserCommand(BaseCommand* command)
{
	{
		CriticalSection::ScopedLock lock(&m_userCommandQueueCS);
		m_userCommandQueue.push_front(command);
	}

	// signal to the thread that a new item is in the queue
	m_userCommandQueueSema.Signal();
}


ClientUserCommandThread::BaseCommand* ClientUserCommandThread::PopUserCommand(void)
{
	m_userCommandQueueSema.Wait();

	CriticalSection::ScopedLock lock(&m_userCommandQueueCS);
	BaseCommand* command = m_userCommandQueue.back();
	m_userCommandQueue.pop_back();

	return command;
}


Thread::ReturnValue ClientUserCommandThread::ThreadFunction(Event* waitForStartEvent, CriticalSection* pipeAccessCS)
{
	// wait until we get the signal that the thread can start
	waitForStartEvent->Wait();

	CommandMap moduleCommandMap;
	moduleCommandMap.RegisterAction<actions::EnableModulesFinished>();
	moduleCommandMap.RegisterAction<actions::DisableModulesFinished>();

	// those commands are needed when loading compiled patches into spawned executables
	moduleCommandMap.RegisterAction<actions::LoadPatch>();
	moduleCommandMap.RegisterAction<actions::UnloadPatch>();
	moduleCommandMap.RegisterAction<actions::EnterSyncPoint>();
	moduleCommandMap.RegisterAction<actions::LeaveSyncPoint>();
	moduleCommandMap.RegisterAction<actions::CallEntryPoint>();
	moduleCommandMap.RegisterAction<actions::CallHooks>();
	// BEGIN EPIC MOD
	moduleCommandMap.RegisterAction<actions::PreCompile>();
	moduleCommandMap.RegisterAction<actions::PostCompile>();
	moduleCommandMap.RegisterAction<actions::TriggerReload>();
	// END EPIC MOD

	for (;;)
	{
		// wait until a command becomes available in the queue
		BaseCommand* command = PopUserCommand();
		if (command == nullptr)
		{
			// BEGIN EPIC MOD - Using internal CrashReporter
			// // no new item available, bail out
			// exceptionHandler::Unregister();
			// END EPIC MOD - Using internal CrashReporter
			return Thread::ReturnValue(2u);
		}

		if (!m_pipe->IsValid())
		{
			// BEGIN EPIC MOD - Using internal CrashReporter
			// // pipe was closed or is broken, bail out
			// exceptionHandler::Unregister();
			// END EPIC MOD - Using internal CrashReporter
			return Thread::ReturnValue(1u);
		}

		// lock critical section for accessing the pipe.
		// we need to make sure that other threads talking through the pipe don't use it at the same time.
		{
			CriticalSection::ScopedLock pipeLock(pipeAccessCS);

			command->Execute(m_pipe);
			if (command->ExpectsResponse())
			{
				moduleCommandMap.HandleCommands(m_pipe, nullptr);
			}
		}

		delete command;
	}
}


#endif // LC_VERSION