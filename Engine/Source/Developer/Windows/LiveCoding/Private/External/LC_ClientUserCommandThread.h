// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ThreadTypes.h"
#include "LC_CriticalSection.h"
#include "LC_Semaphore.h"
// BEGIN EPIC MOD
#include <string>
#include <deque>
// END EPIC MOD

class DuplexPipe;
class DuplexPipeClient;
class Event;


// handles incoming commands from the host (the executable that loaded the Live++ DLL)
class ClientUserCommandThread
{
public:
	class BaseCommand
	{
	public:
		explicit BaseCommand(bool expectResponse);
		virtual ~BaseCommand(void);

		virtual void Execute(DuplexPipe* pipe) = 0;

		bool ExpectsResponse(void) const;

	private:
		bool m_expectResponse;
	};

	struct ExceptionResult
	{
		const void* returnAddress;
		const void* framePointer;
		const void* stackPointer;
		bool continueExecution;
	};

	ClientUserCommandThread(DuplexPipeClient* pipeClient, DuplexPipeClient* exceptionPipeClient);
	~ClientUserCommandThread(void);

	// Starts the thread that takes care of handling incoming commands on the pipe.
	Thread::Id Start(const std::wstring& processGroupName, Event* waitForStartEvent, CriticalSection* pipeAccessCS);

	// Joins this thread.
	void Join(void);

	void* EnableModule(const wchar_t* nameOfExeOrDll);
	void* EnableModules(const wchar_t* namesOfExeOrDll[], unsigned int count);
	void* EnableAllModules(const wchar_t* nameOfExeOrDll);

	void* DisableModule(const wchar_t* nameOfExeOrDll);
	void* DisableModules(const wchar_t* namesOfExeOrDll[], unsigned int count);
	void* DisableAllModules(const wchar_t* nameOfExeOrDll);

	// BEGIN EPIC MOD - Adding TryWaitForToken
	bool TryWaitForToken(void* token);
	// END EPIC MOD

	void WaitForToken(void* token);
	void TriggerRecompile(void);
	void LogMessage(const wchar_t* message);
	void BuildPatch(const wchar_t* moduleNames[], const wchar_t* objPaths[], const wchar_t* amalgamatedObjPaths[], unsigned int count);

	void TriggerRestart(void);

	// BEGIN EPIC MOD - Adding ShowConsole command
	void ShowConsole();
	// END EPIC MOD

	// BEGIN EPIC MOD - Adding SetVisible command
	void SetVisible(bool visible);
	// END EPIC MOD

	// BEGIN EPIC MOD - Adding SetActive command
	void SetActive(bool active);
	// END EPIC MOD

	// BEGIN EPIC MOD - Adding SetBuildArguments command
	void SetBuildArguments(const wchar_t* arguments);
	// END EPIC MOD

	// BEGIN EPIC MOD - Support for lazy-loading modules
	void* EnableLazyLoadedModule(const wchar_t* fileName, Windows::HMODULE moduleBase);
	// END EPIC MOD

	// BEGIN EPIC MOD
	void SetReinstancingFlow(bool enable);
	// END EPIC MOD

	// BEGIN EPIC MOD
	void DisableCompileFinishNotification();
	// END EPIC MOD

	// BEGIN EPIC MOD
	void* EnableModulesEx(const wchar_t* moduleNames[], unsigned int moduleCount, const wchar_t* lazyLoadModuleNames[], unsigned int lazyLoadModuleCount, const uintptr_t* reservedPages, unsigned int reservedPagesCount);
	// END EPIC MOD

	void ApplySettingBool(const char* settingName, int value);
	void ApplySettingInt(const char* settingName, int value);
	void ApplySettingString(const char* settingName, const wchar_t* value);

	void InstallExceptionHandler(void);
	ExceptionResult HandleException(EXCEPTION_RECORD* exception, CONTEXT* context, Thread::Id threadId);
	void End(void);

private:
	// pushes a user command into the command queue
	void PushUserCommand(BaseCommand* command);

	// pops a user command from the command queue.
	// blocks until a command becomes available.
	BaseCommand* PopUserCommand(void);

	Thread::ReturnValue ThreadFunction(Event* waitForStartEvent, CriticalSection* pipeAccessCS);

	Thread::Handle m_thread;
	std::wstring m_processGroupName;
	DuplexPipeClient* m_pipe;
	DuplexPipeClient* m_exceptionPipe;

	// queue for working on commands received by user code
	std::deque<BaseCommand*> m_userCommandQueue;
	CriticalSection m_userCommandQueueCS;
	Semaphore m_userCommandQueueSema;
};


#endif // LC_VERSION