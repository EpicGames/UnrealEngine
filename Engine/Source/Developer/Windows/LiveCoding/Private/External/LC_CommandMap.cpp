// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_CommandMap.h"
#include "LC_DuplexPipe.h"


namespace
{
	template <class T>
	static bool DefaultAction(const DuplexPipe* pipe, void*, void* payload, size_t payloadSize)
	{
		// receive command and continue execution
		T command = {};
		const bool success = pipe->ReceiveCommand(&command, payload, payloadSize);
		pipe->SendAck();

		if (!success)
		{
			return false;
		}

		return true;
	}

	template <typename T>
	static void RegisterDefaultAction(CommandMap::Action* actions)
	{
		actions[T::ID] = &DefaultAction<T>;
	}
}


CommandMap::CommandMap(void)
	: m_actions()
{
	// register default handlers that receive command data and continue execution
	RegisterDefaultAction<commands::Acknowledge>(m_actions);
	RegisterDefaultAction<commands::RegisterProcess>(m_actions);
	RegisterDefaultAction<commands::RegisterProcessFinished>(m_actions);

	RegisterDefaultAction<commands::EnableModules>(m_actions);
	RegisterDefaultAction<commands::EnableModulesFinished>(m_actions);
	RegisterDefaultAction<commands::DisableModules>(m_actions);
	RegisterDefaultAction<commands::DisableModulesFinished>(m_actions);

	RegisterDefaultAction<commands::EnterSyncPoint>(m_actions);
	RegisterDefaultAction<commands::LeaveSyncPoint>(m_actions);
	RegisterDefaultAction<commands::CallHooks>(m_actions);
	RegisterDefaultAction<commands::LoadPatch>(m_actions);
	RegisterDefaultAction<commands::LoadPatchInfo>(m_actions);
	RegisterDefaultAction<commands::UnloadPatch>(m_actions);
	RegisterDefaultAction<commands::CallEntryPoint>(m_actions);
	RegisterDefaultAction<commands::LogOutput>(m_actions);
	RegisterDefaultAction<commands::ReadyForCompilation>(m_actions);
	RegisterDefaultAction<commands::CompilationFinished>(m_actions);
	RegisterDefaultAction<commands::DisconnectClient>(m_actions);
	RegisterDefaultAction<commands::TriggerRecompile>(m_actions);	
	RegisterDefaultAction<commands::TriggerRestart>(m_actions);
	RegisterDefaultAction<commands::LogMessage>(m_actions);
	RegisterDefaultAction<commands::BuildPatch>(m_actions);
	RegisterDefaultAction<commands::HandleException>(m_actions);
	RegisterDefaultAction<commands::HandleExceptionFinished>(m_actions);

	RegisterDefaultAction<commands::ApplySettingBool>(m_actions);
	RegisterDefaultAction<commands::ApplySettingInt>(m_actions);
	RegisterDefaultAction<commands::ApplySettingString>(m_actions);

	// BEGIN EPIC MOD - Adding ShowConsole command
	RegisterDefaultAction<commands::ShowConsole>(m_actions);
	// END EPIC MOD

	// BEGIN EPIC MOD - Adding SetVisible command
	RegisterDefaultAction<commands::SetVisible>(m_actions);
	// END EPIC MOD

	// BEGIN EPIC MOD - Adding SetActive command
	RegisterDefaultAction<commands::SetActive>(m_actions);
	// END EPIC MOD

	// BEGIN EPIC MOD - Adding SetBuildArguments command
	RegisterDefaultAction<commands::SetBuildArguments>(m_actions);
	// END EPIC MOD

	// BEGIN EPIC MOD
	RegisterDefaultAction<commands::PreCompile>(m_actions);
	RegisterDefaultAction<commands::PostCompile>(m_actions);
	RegisterDefaultAction<commands::TriggerReload>(m_actions);
	RegisterDefaultAction<commands::SetReinstancingFlow>(m_actions);
	RegisterDefaultAction<commands::DisableCompileFinishNotification>(m_actions);
	// END EPIC MOD
}


CommandMap::~CommandMap(void)
{
}


bool CommandMap::HandleCommands(const DuplexPipe* pipe, void* context)
{
	for (;;)
	{
		// fetch incoming command header
		commands::Header header = {};
		{
			const bool success = pipe->ReceiveHeader(&header);
			if (!success)
			{
				return false;
			}
		}

		// BEGIN EPIC MOD
		if (header.commandId >= commands::COUNT)
		{
			return false;
		}
		// END EPIC MOD

		const Action action = m_actions[header.commandId];

		// BEGIN EPIC MOD
		if (action == nullptr)
		{
			return false;
		}
		// END EPIC MOD

		// make space for optional payload
		void* payload = nullptr;
		if (header.payloadSize != 0u)
		{
			payload = malloc(header.payloadSize);
		}

		// call handler for this command
		const bool continueExecution = action(pipe, context, payload, header.payloadSize);

		// free payload
		free(payload);

		if (!continueExecution)
		{
			return true;
		}
	}
}


#endif // LC_VERSION