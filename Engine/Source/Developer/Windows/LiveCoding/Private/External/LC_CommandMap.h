// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Commands.h"
// BEGIN EPIC MOD
#include "LC_DuplexPipe.h"
// END EPIC MOD

class DuplexPipe;

class CommandMap
{
	template <class T>
	static bool ReceiveAndCallAction(const DuplexPipe* pipe, void* context, void* payload, size_t payloadSize)
	{
		// receive command and call user action
		typename T::CommandType command = {};
		const bool success = pipe->ReceiveCommand(&command, payload, payloadSize);
		if (!success)
		{
			pipe->SendAck();
			return false;
		}

		return T::Execute(&command, pipe, context, payload, payloadSize);
	}

public:
	// an action returns whether execution should continue
	typedef bool (*Action)(const DuplexPipe*, void*, void*, size_t);

	CommandMap(void);
	~CommandMap(void);

	template <class T>
	void RegisterAction(void)
	{
		m_actions[T::CommandType::ID] = &ReceiveAndCallAction<T>;
	}

	bool HandleCommands(const DuplexPipe* pipe, void* context);

private:
	Action m_actions[commands::COUNT];
};


#endif // LC_VERSION