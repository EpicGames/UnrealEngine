// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "LC_Hook.h"
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD
#include "LC_ProcessTypes.h"
#include "LC_ThreadTypes.h"


namespace commands
{
	struct Header
	{
		uint32_t commandId;
		uint32_t payloadSize;
	};

	struct ModuleData
	{
		void* base;
		// BEGIN EPIC MOD
		wchar_t path[WINDOWS_MAX_PATH];
		// END EPIC MOD
	};

	// acknowledge that a command has been received
	struct Acknowledge
	{
		// BEGIN EPIC MOD
		static const uint32_t ID = 100;
		// END EPIC MOD
	};

	// register a process with Live++
	struct RegisterProcess
	{
		static const uint32_t ID = Acknowledge::ID + 1u;

		void* processBase;
		Process::Id processId;				// current process ID
		Process::Id restartedProcessId;		// process ID of the previous, restarted process. 0 if non-existent
		Thread::Id threadId;				// thread ID of Live++ thread running in host
		const void* jumpToSelf;				// address of jump-to-self instruction in host

		size_t imagePathSize;
		size_t commandLineSize;
		size_t workingDirectorySize;
		size_t environmentSize;

		// image path, command line, working directory and environment follow as payload
	};

	// tell the DLL that registration has finished
	struct RegisterProcessFinished
	{
		static const uint32_t ID = RegisterProcess::ID + 1u;

		bool success;
	};

	// tell Live++ to enable modules for live coding
	struct EnableModules
	{
		static const uint32_t ID = RegisterProcessFinished::ID + 1u;

		Process::Id processId;
		unsigned int moduleCount;
		void* token;

		// this command always contains an array of 'ModuleData x moduleCount' as payload
	};

	// tell the DLL that enabling modules has finished
	struct EnableModulesFinished
	{
		static const uint32_t ID = EnableModules::ID + 1u;

		void* token;
	};

	// tell Live++ to disable modules for live coding
	struct DisableModules
	{
		static const uint32_t ID = EnableModulesFinished::ID + 1u;

		Process::Id processId;
		unsigned int moduleCount;
		void* token;

		// this command always contains an array of 'ModuleData x moduleCount' as payload
	};

	// tell the DLL that disabling a module has finished
	struct DisableModulesFinished
	{
		static const uint32_t ID = DisableModules::ID + 1u;

		void* token;
	};

	// tell the DLL to enter the synchronization point
	struct EnterSyncPoint
	{
		static const uint32_t ID = DisableModulesFinished::ID + 1u;
	};

	// tell the DLL to leave the synchronization point
	struct LeaveSyncPoint
	{
		static const uint32_t ID = EnterSyncPoint::ID + 1u;
	};

	// tell the DLL to call hooks
	struct CallHooks
	{
		static const uint32_t ID = LeaveSyncPoint::ID + 1u;

		hook::Type::Enum type;
		const void* rangeBegin;
		const void* rangeEnd;
	};

	// tell the DLL to load a DLL
	struct LoadPatch
	{
		static const uint32_t ID = CallHooks::ID + 1u;

		// BEGIN EPIC MOD
		wchar_t path[WINDOWS_MAX_PATH];
		// END EPIC MOD
	};

	// returns info about a loaded DLL to Live++
	struct LoadPatchInfo
	{
		static const uint32_t ID = LoadPatch::ID + 1u;

		// BEGIN EPIC MOD
		Windows::HMODULE module;
		// END EPIC MOD
	};

	// tell the DLL to unload a DLL
	struct UnloadPatch
	{
		static const uint32_t ID = LoadPatchInfo::ID + 1u;

		// BEGIN EPIC MOD
		Windows::HMODULE module;
		// END EPIC MOD
	};

	// tell the DLL to call the entry point of a DLL
	struct CallEntryPoint
	{
		static const uint32_t ID = UnloadPatch::ID + 1u;

		void* moduleBase;
		uint32_t entryPointRva;
	};

	// tell the DLL to log output
	struct LogOutput
	{
		static const uint32_t ID = CallEntryPoint::ID + 1u;
	};

	// tell Live++ server we're ready for compilation
	struct ReadyForCompilation
	{
		static const uint32_t ID = LogOutput::ID + 1u;
	};

	// tell the DLL that compilation has finished
	struct CompilationFinished
	{
		static const uint32_t ID = ReadyForCompilation::ID + 1u;
	};

	// tell Live++ server that a client is about to disconnect
	struct DisconnectClient
	{
		static const uint32_t ID = CompilationFinished::ID + 1u;
	};

	// tell Live++ to trigger a recompile
	struct TriggerRecompile
	{
		static const uint32_t ID = DisconnectClient::ID + 1u;
	};

	// tell Live++ to trigger a restart
	struct TriggerRestart
	{
		static const uint32_t ID = TriggerRecompile::ID + 1u;
	};

	// tell Live++ to log a message
	struct LogMessage
	{
		static const uint32_t ID = TriggerRestart::ID + 1u;
	};

	// tell Live++ to build a patch using an array of object files
	struct BuildPatch
	{
		static const uint32_t ID = LogMessage::ID + 1u;

		unsigned int fileCount;

		// this command always contains an array of 'PatchData x fileCount' as payload
		struct PatchData
		{
			wchar_t moduleName[MAX_PATH];
			wchar_t objPath[MAX_PATH];
			wchar_t amalgamatedObjPath[MAX_PATH];
		};
	};

	// tell Live++ to handle an exception
	struct HandleException
	{
		static const uint32_t ID = BuildPatch::ID + 1u;

		Process::Id processId;
		Thread::Id threadId;
		EXCEPTION_RECORD exception;
		CONTEXT context;
		CONTEXT* clientContextPtr;
	};

	// tell the DLL that handling an exception has finished
	struct HandleExceptionFinished
	{
		static const uint32_t ID = HandleException::ID + 1u;

		const void* returnAddress;
		const void* framePointer;
		const void* stackPointer;
		bool continueExecution;
	};

	// tell the EXE that a bool setting needs to be changed
	struct ApplySettingBool
	{
		static const uint32_t ID = HandleExceptionFinished::ID + 1u;

		char settingName[256];
		int settingValue;
	};

	// tell the EXE that a int setting needs to be changed
	struct ApplySettingInt
	{
		static const uint32_t ID = ApplySettingBool::ID + 1u;

		char settingName[256];
		int settingValue;
	};

	// tell the EXE that a bool setting needs to be changed
	struct ApplySettingString
	{
		static const uint32_t ID = ApplySettingInt::ID + 1u;

		char settingName[256];
		wchar_t settingValue[256];
	};

	// BEGIN EPIC MOD
	struct ShowConsole
	{
		static const uint32_t ID = ApplySettingString::ID + 1u;
	};

	struct SetVisible
	{
		static const uint32_t ID = ShowConsole::ID + 1u;

		bool visible;
	};

	struct SetActive
	{
		static const uint32_t ID = SetVisible::ID + 1u;

		bool active;
	};

	struct SetBuildArguments
	{
		static const uint32_t ID = SetActive::ID + 1u;

		Process::Id processId;
		wchar_t arguments[1024];
	};

	struct EnableLazyLoadedModule
	{
		static const uint32_t ID = SetBuildArguments::ID + 1u;

		Process::Id processId;
		wchar_t fileName[260];
		Windows::HMODULE moduleBase;
		void* token;
	};

	struct FinishedLazyLoadingModules
	{
		static const uint32_t ID = EnableLazyLoadedModule::ID + 1u;
	};

	struct PreCompile
	{
		static const uint32_t ID = FinishedLazyLoadingModules::ID + 1u;
	};

	enum class PostCompileResult : unsigned char
	{
		Success,
		NoChanges,
		Failure,
		Cancelled,
	};

	struct PostCompile
	{
		static const uint32_t ID = PreCompile::ID + 1u;
		PostCompileResult postCompileResult = PostCompileResult::Success;
	};

	struct TriggerReload
	{
		static const uint32_t ID = PostCompile::ID + 1u;
	};

	struct SetReinstancingFlow
	{
		static const uint32_t ID = TriggerReload::ID + 1u;

		Process::Id processId;
		bool enable;
	};

	struct DisableCompileFinishNotification
	{
		static const uint32_t ID = SetReinstancingFlow::ID + 1u;

		Process::Id processId;
	};

	// This is a version of EnableModules that includes support for lazy modules and pre-reserved page addresses.
	struct EnableModulesEx
	{
		static const uint32_t ID = DisableCompileFinishNotification::ID + 1u;

		Process::Id processId;
		unsigned int moduleCount;
		unsigned int lazyLoadModuleCount;
		unsigned int reservedPagesCount;
		void* token;

		// this command always contains an array of 'ModuleData x moduleCount + ModuleData x lazyLoadModuleCount + sizeof(uintptr_t) * reservedPagesCount' as payload
	};

	static const uint32_t COUNT = EnableModulesEx::ID + 1u;
	// END EPIC MOD
}


#endif // LC_VERSION