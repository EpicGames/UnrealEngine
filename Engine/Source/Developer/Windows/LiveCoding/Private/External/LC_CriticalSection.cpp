// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_CriticalSection.h"


CriticalSection::CriticalSection(void)
{
	// BEGIN EPIC MOD
	Windows::InitializeCriticalSection(&m_cs);
	// END EPIC MOD
}


CriticalSection::~CriticalSection(void)
{
	// BEGIN EPIC MOD
	Windows::DeleteCriticalSection(&m_cs);
	// END EPIC MOD
}


void CriticalSection::Enter(void)
{
	// BEGIN EPIC MOD
	Windows::EnterCriticalSection(&m_cs);
	// END EPIC MOD
}


void CriticalSection::Leave(void)
{
	// BEGIN EPIC MOD
	Windows::LeaveCriticalSection(&m_cs);
	// END EPIC MOD
}


#endif // LC_VERSION