// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "Windows/MinimalWindowsApi.h"
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
// END EPIC MOD

class CriticalSection
{
public:
	CriticalSection(void);
	~CriticalSection(void);

	void Enter(void);
	void Leave(void);

	class ScopedLock
	{
	public:
		explicit ScopedLock(CriticalSection* cs)
			: m_cs(cs)
		{
			cs->Enter();
		}

		~ScopedLock(void)
		{
			m_cs->Leave();
		}

	private:
		LC_DISABLE_COPY(ScopedLock);
		LC_DISABLE_MOVE(ScopedLock);
		LC_DISABLE_ASSIGNMENT(ScopedLock);
		LC_DISABLE_MOVE_ASSIGNMENT(ScopedLock);

		CriticalSection* m_cs;
	};

private:
	// BEGIN EPIC MOD
	Windows::CRITICAL_SECTION m_cs;
	// END EPIC MOD
};


#endif // LC_VERSION