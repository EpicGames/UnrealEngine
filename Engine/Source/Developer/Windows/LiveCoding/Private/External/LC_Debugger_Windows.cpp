// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Debugger_Windows.h"
// BEGIN EPIC MOD
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD


bool Debugger::IsConnected(void)
{
	return (::IsDebuggerPresent() != 0);
}


#endif // LC_VERSION