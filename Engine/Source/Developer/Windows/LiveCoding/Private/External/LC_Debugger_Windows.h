// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD

namespace Debugger
{
	bool IsConnected(void);
}


#define LC_DEBUGGER_BREAKPOINT() __debugbreak()


#endif // LC_VERSION