// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_DuplexPipe.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
// END EPIC MOD

DuplexPipe::DuplexPipe(void)
	: m_pipe(INVALID_HANDLE_VALUE)
{
}


DuplexPipe::~DuplexPipe(void)
{
	Close();
}


void DuplexPipe::Close(void)
{
	::CloseHandle(m_pipe);
	m_pipe = INVALID_HANDLE_VALUE;
}


bool DuplexPipe::IsValid(void) const volatile
{
	return (m_pipe != INVALID_HANDLE_VALUE);
}


bool DuplexPipe::ReceiveHeader(commands::Header* header) const
{
	const bool success = Read(header, sizeof(commands::Header));
	return success;
}


void DuplexPipe::Send(const void* buffer, size_t size) const
{
	size_t writtenSoFar = 0u;
	do
	{
		DWORD bytesWritten = 0u;
		const BOOL success = ::WriteFile(m_pipe, static_cast<const char*>(buffer) + writtenSoFar, static_cast<DWORD>(size - writtenSoFar), &bytesWritten, NULL);
		if (success == 0)
		{
			// error while trying to write to the pipe, process has probably ended and closed its end of the pipe
			const DWORD error = ::GetLastError();
			if (error == ERROR_NO_DATA)
			{
				// this is expected, pipe has disconnected
				return;
			}

			LC_ERROR_USER("Error 0x%X while writing to pipe: Size: %zu, written: %d", error, size, bytesWritten);
			return;
		}

		writtenSoFar += bytesWritten;
	}
	while (writtenSoFar != size);
}


bool DuplexPipe::Read(void* buffer, size_t size) const
{
	size_t readSoFar = 0u;
	do
	{
		DWORD bytesRead = 0u;
		const BOOL success = ::ReadFile(m_pipe, static_cast<char*>(buffer) + readSoFar, static_cast<DWORD>(size - readSoFar), &bytesRead, NULL);
		if (success == 0)
		{
			// error while trying to read from the pipe, process has probably ended and closed its end of the pipe
			const DWORD error = ::GetLastError();
			if ((error == ERROR_BROKEN_PIPE) || (error == ERROR_PIPE_NOT_CONNECTED) || (error == ERROR_OPERATION_ABORTED))
			{
				// this is expected, pipe has disconnected
				return false;
			}

			LC_ERROR_USER("Error 0x%X while reading from pipe. Size: %zu, read: %d", error, size, bytesRead);
			return false;
		}

		readSoFar += bytesRead;
	}
	while (readSoFar != size);

	return true;
}


#endif // LC_VERSION