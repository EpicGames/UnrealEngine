// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Commands.h"
// BEGIN EPIC MOD
#include "LC_Assert.h"
#include "LC_Platform.h"
// END EPIC MOD

class DuplexPipe
{
public:
	DuplexPipe(void);
	~DuplexPipe(void);

	void Close(void);
	bool IsValid(void) const volatile;

	// send command, synchronous
	template <typename T>
	void SendCommandAndWaitForAck(const T& command, const void* payload, size_t payloadSize) const
	{
		// send the header and command, and wait for the ACK to be received
		const commands::Header header = { T::ID, static_cast<uint32_t>(payloadSize) };
		Send(&header, sizeof(commands::Header));
		Send(&command, sizeof(T));

		if (payloadSize != 0u)
		{
			LC_ASSERT(payload != nullptr, "Cannot send command payload.");
			Send(payload, payloadSize);
		}

		commands::Acknowledge ack = {};
		Read(&ack, sizeof(commands::Acknowledge));
	}

	// receive command header
	bool ReceiveHeader(commands::Header* header) const;

	// receive any command
	template <typename T>
	bool ReceiveCommand(T* command, void* payload, size_t payloadSize) const
	{
		// receive command and optional payload
		bool success = Read(command, sizeof(T));

		if (payloadSize != 0u)
		{
			LC_ASSERT(payload != nullptr, "Cannot receive command payload.");
			success &= Read(payload, payloadSize);
		}

		return success;
	}

	// send acknowledge command
	void SendAck(void) const
	{
		commands::Acknowledge ack = {};
		Send(&ack, sizeof(commands::Acknowledge));
	}

protected:
	HANDLE m_pipe;

private:
	void Send(const void* buffer, size_t size) const;
	bool Read(void* buffer, size_t size) const;
};


#endif // LC_VERSION