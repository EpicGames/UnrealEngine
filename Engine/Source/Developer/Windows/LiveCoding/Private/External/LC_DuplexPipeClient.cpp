// Copyright 2011-2019 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_DuplexPipeClient.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
// END EPIC MOD

bool DuplexPipeClient::Connect(const wchar_t* name)
{
	m_pipe = ::CreateFile(
		name,
		GENERIC_READ | GENERIC_WRITE,
		0,
		NULL,
		OPEN_EXISTING,
		0,
		NULL);

	if (m_pipe == INVALID_HANDLE_VALUE)
	{
		const DWORD error = ::GetLastError();
		LC_ERROR_USER("Error 0x%X while trying to connect to named pipe", error);
		return false;
	}

	DWORD mode = PIPE_READMODE_MESSAGE;
	const BOOL success = ::SetNamedPipeHandleState(m_pipe, &mode, NULL, NULL);
	if (!success)
	{
		const DWORD error = ::GetLastError();
		LC_ERROR_USER("Error 0x%X while trying to set named pipe state", error);
		return false;
	}

	return true;
}


#endif // LC_VERSION