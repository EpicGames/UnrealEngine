// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "LC_EntryPoint.h"
// END EPIC MOD

#endif // LC_VERSION