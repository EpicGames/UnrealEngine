// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if LC_VERSION == 1

#include "CoreTypes.h"
#include "Windows/MinimalWindowsApi.h"

void Startup(Windows::HINSTANCE hInstance);
void Shutdown();


#endif // LC_VERSION