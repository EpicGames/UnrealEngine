// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Environment.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

namespace
{
	// BEGIN EPIC MOD
	static const DWORD MAX_ENVIRONMENT_VARIABLE_SIZE = 1024u;
	// END EPIC MOD
}


void environment::RemoveVariable(const wchar_t* variable)
{
	const BOOL result = ::SetEnvironmentVariableW(variable, NULL);
	if (result == 0)
	{
		LC_LOG_DEV("Could not remove environment variable %S (Error: %d)", variable, ::GetLastError());
	}
}


void environment::SetVariable(const wchar_t* variable, const wchar_t* value)
{
	const BOOL result = ::SetEnvironmentVariableW(variable, value);
	if (result == 0)
	{
		LC_LOG_DEV("Could not set environment variable %S to value %S (Error: %d)", variable, value, ::GetLastError());
	}
}


std::wstring environment::GetVariable(const wchar_t* variable, const wchar_t* defaultValue)
{
	std::wstring value;
	// BEGIN EPIC MOD
	value.resize(MAX_ENVIRONMENT_VARIABLE_SIZE);
	// END EPIC MOD

	// BEGIN EPIC MOD
	const DWORD result = ::GetEnvironmentVariableW(variable, &value[0], MAX_ENVIRONMENT_VARIABLE_SIZE);
	// END EPIC _MOD
	if (result == 0)
	{
		// if a default value was provided, return that instead
		if (defaultValue)
		{
			return std::wstring(defaultValue);
		}
		else
		{
			return std::wstring();
		}
	}

	return value;
}


#endif // LC_VERSION