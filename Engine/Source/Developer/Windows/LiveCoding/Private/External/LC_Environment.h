// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include <string>
// END EPIC MOD

namespace environment
{
	// Removes a variable from the environment of the calling process.
	void RemoveVariable(const wchar_t* variable);

	// Sets a variable in the environment of the calling process.
	void SetVariable(const wchar_t* variable, const wchar_t* value);

	// Gets a variable from the environment of the calling process.
	std::wstring GetVariable(const wchar_t* variable, const wchar_t* defaultValue);
}


#endif // LC_VERSION