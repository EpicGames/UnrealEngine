// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Event.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD


Event::Event(const wchar_t* name, Type::Enum type)
	// BEGIN EPIC MOD
	: m_event(::CreateEventW(NULL, (type == Type::MANUAL_RESET) ? Windows::TRUE : Windows::FALSE, Windows::FALSE, name))
	// END EPIC MOD
{
	const DWORD error = ::GetLastError();
	if (m_event == NULL)
	{
		LC_ERROR_USER("Cannot create event %S. Error: 0x%X", name ? name : L"(unnamed)", error);
	}
	else if (error == ERROR_ALREADY_EXISTS)
	{
		// another process already created this event, this is to be expected
	}
}


Event::~Event(void)
{
	::CloseHandle(m_event);
}


void Event::Reset(void)
{
	::ResetEvent(m_event);
}


void Event::Signal(void)
{
	::SetEvent(m_event);
}


bool Event::Wait(void)
{
	return WaitTimeout(INFINITE);
}


bool Event::WaitTimeout(unsigned int milliSeconds)
{
	const DWORD result = ::WaitForSingleObject(m_event, milliSeconds);
	switch (result)
	{
		case WAIT_OBJECT_0:
			// event was successfully signaled
			return true;

		case WAIT_TIMEOUT:
			// the operation timed out, which should never happen with a timeout of INFINITE
			if (milliSeconds == INFINITE)
			{
				LC_ERROR_DEV("%s", "Event timed out.");
			}
			return false;

		case WAIT_ABANDONED:
			LC_ERROR_DEV("%s", "Wait() was called on a stale event which was not released by the owning thread.");
			return false;

		case WAIT_FAILED:
			LC_ERROR_DEV("%s", "Failed to Wait() on an event.");
			return false;

		default:
			return false;
	}
}


bool Event::TryWait(void)
{
	return WaitTimeout(0u);
}


#endif // LC_VERSION