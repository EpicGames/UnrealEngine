// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "Windows/MinimalWindowsApi.h"
// END EPIC MOD

// named/unnamed event.
// acts process-wide if given a name.
class Event
{
public:
	struct Type
	{
		enum Enum
		{
			MANUAL_RESET,
			AUTO_RESET
		};
	};

	Event(const wchar_t* name, Type::Enum type);
	~Event(void);

	// Resets the event.
	void Reset(void);

	// Signals the event.
	void Signal(void);

	// Waits until the event becomes signaled, blocking.
	bool Wait(void);

	// Waits until the event becomes signaled, blocking until the timeout is reached.
	// Returns whether the event was signaled.
	bool WaitTimeout(unsigned int milliSeconds);

	// Returns whether the event was signaled, non-blocking.
	bool TryWait(void);

private:
	// BEGIN EPIC MOD
	Windows::HANDLE m_event;
	// END EPIC MOD
};


#endif // LC_VERSION