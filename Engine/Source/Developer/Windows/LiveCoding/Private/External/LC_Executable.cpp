// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Executable.h"
#include "LC_MemoryMappedFile.h"
// BEGIN EPIC MOD
#include "LC_Assert.h"
#include "LC_Logging.h"
#include <algorithm>
// END EPIC MOD

// this is the default DLL entry point, taken from the CRT. we don't need it, but can extract its signature.
extern "C" BOOL WINAPI _DllMainCRTStartup(
	HINSTANCE const instance,
	DWORD     const reason,
	LPVOID    const reserved
);


namespace detail
{
	static inline bool SortSectionByAscendingRVA(const executable::ImageSection& lhs, const executable::ImageSection& rhs)
	{
		return lhs.rva < rhs.rva;
	}


	static const IMAGE_NT_HEADERS* GetNtHeader(const executable::Image* image)
	{
		const void* base = Filesystem::GetMemoryMappedFileData(image);

		// PE image start with a DOS header
		const IMAGE_DOS_HEADER* dosHeader = static_cast<const IMAGE_DOS_HEADER*>(base);
		if (dosHeader->e_magic != IMAGE_DOS_SIGNATURE)
		{
			LC_ERROR_USER("%s", "Image has unknown file format");
			return nullptr;
		}

		const IMAGE_NT_HEADERS* ntHeader = pointer::Offset<const IMAGE_NT_HEADERS*>(dosHeader, dosHeader->e_lfanew);
		if (ntHeader->Signature != IMAGE_NT_SIGNATURE)
		{
			LC_ERROR_USER("%s", "Invalid .exe file");
			return nullptr;
		}

		return ntHeader;
	}


	static IMAGE_NT_HEADERS* GetNtHeader(executable::Image* image)
	{
		return const_cast<IMAGE_NT_HEADERS*>(GetNtHeader(const_cast<const executable::Image*>(image)));
	}


	static const IMAGE_SECTION_HEADER* GetSectionHeader(const IMAGE_NT_HEADERS* ntHeader)
	{
		return IMAGE_FIRST_SECTION(ntHeader);
	}
}


uint32_t executable::GetEntryPointRva(const Image* image)
{
	const IMAGE_NT_HEADERS* ntHeader = detail::GetNtHeader(image);
	if (!ntHeader)
	{
		return 0u;
	}

	return ntHeader->OptionalHeader.AddressOfEntryPoint;
}


executable::PreferredBase executable::GetPreferredBase(const Image* image)
{
	const IMAGE_NT_HEADERS* ntHeader = detail::GetNtHeader(image);
	if (!ntHeader)
	{
		return 0ull;
	}

	return ntHeader->OptionalHeader.ImageBase;
}


executable::Header executable::GetHeader(const Image* image)
{
	const IMAGE_NT_HEADERS* ntHeader = detail::GetNtHeader(image);
	if (!ntHeader)
	{
		return executable::Header {};
	}

	const uint64_t sizeOnDisk = Filesystem::GetMemoryMappedFileSizeOnDisk(image);
	return executable::Header { sizeOnDisk, ntHeader->FileHeader, ntHeader->OptionalHeader };
}


bool executable::IsValidHeader(const Header& header)
{
	return header.imageHeader.NumberOfSections != 0;
}


uint32_t executable::GetSize(const Image* image)
{
	const IMAGE_NT_HEADERS* ntHeader = detail::GetNtHeader(image);
	if (!ntHeader)
	{
		return 0ull;
	}

	return ntHeader->OptionalHeader.SizeOfImage;
}


uint32_t executable::RvaToFileOffset(const ImageSectionDB* database, uint32_t rva)
{
	LC_ASSERT(rva != 0u, "RVA cannot be mapped to image.");

	const size_t count = database->sections.size();
	for (size_t i = 0u; i < count; ++i)
	{
		const ImageSection& section = database->sections[i];
		if ((rva >= section.rva) && (rva < section.rva + section.size))
		{
			const uint32_t sectionOffset = rva - section.rva;
			if (sectionOffset >= section.rawDataSize)
			{
				// the offset relative to the section lies outside the section data stored in the image.
				// this can happen for sections like .bss/.data which don't store uninitialized data for
				// the symbols.
				return 0u;
			}

			return section.rawDataRva + sectionOffset;
		}
	}

	LC_ERROR_DEV("Cannot map RVA 0x%X to executable image file offset", rva);
	return 0u;
}


void executable::ReadFromFileOffset(const Image* image, uint32_t offset, void* destination, size_t byteCount)
{
	const void* address = pointer::Offset<const void*>(Filesystem::GetMemoryMappedFileData(image), offset);
	memcpy(destination, address, byteCount);
}


void executable::WriteToFileOffset(Image* image, uint32_t offset, const void* source, size_t byteCount)
{
	void* address = pointer::Offset<void*>(Filesystem::GetMemoryMappedFileData(image), offset);
	memcpy(address, source, byteCount);
}


executable::Image* executable::OpenImage(const wchar_t* filename, Filesystem::OpenMode::Enum openMode)
{
	return Filesystem::OpenMemoryMappedFile(filename, openMode);
}


void executable::CloseImage(Image*& image)
{
	Filesystem::CloseMemoryMappedFile(image);
}


void executable::RebaseImage(Image* image, PreferredBase preferredBase)
{
	void* base = Filesystem::GetMemoryMappedFileData(image);

	IMAGE_NT_HEADERS* ntHeader = detail::GetNtHeader(image);
	if (!ntHeader)
	{
		return;
	}

	const IMAGE_SECTION_HEADER* sectionHeader = detail::GetSectionHeader(ntHeader);
	if (!sectionHeader)
	{
		return;
	}

	ImageSectionDB* database = GatherImageSectionDB(image);

	// the image has been linked against a certain base address, namely ntHeader->OptionalHeader.ImageBase.
	// work out by how much all relocations need to be shifted if basing the image against the new
	// preferred base.
	const int64_t baseDelta = static_cast<int64_t>(preferredBase - ntHeader->OptionalHeader.ImageBase);

	// this is the easy part: simply set the new preferred base address in the image
	ntHeader->OptionalHeader.ImageBase = preferredBase;

	// now comes the hard part: patch all relocation entries in the image
	const DWORD relocSectionSize = ntHeader->OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_BASERELOC].Size;
	if (relocSectionSize != 0u)
	{
		// .reloc section exists, patch it
		const DWORD baseRelocationOffset = RvaToFileOffset(database, ntHeader->OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_BASERELOC].VirtualAddress);
		const IMAGE_BASE_RELOCATION* baseRelocations = pointer::Offset<const IMAGE_BASE_RELOCATION*>(base, baseRelocationOffset);

		DWORD blockSizeLeft = relocSectionSize;
		while (blockSizeLeft > 0u)
		{
			const DWORD pageRVA = baseRelocations->VirtualAddress;
			const DWORD blockSize = baseRelocations->SizeOfBlock;
			const DWORD blockOffset = RvaToFileOffset(database, pageRVA);

			// PE spec: Block size: The total number of bytes in the base relocation block, *including* the Page RVA and
			// Block Size fields and the Type/Offset fields that follow.
			const DWORD numberOfEntriesInThisBlock = (blockSize - sizeof(IMAGE_BASE_RELOCATION)) / sizeof(WORD);
			const WORD* entries = pointer::Offset<const WORD*>(baseRelocations, sizeof(IMAGE_BASE_RELOCATION));
			for (DWORD i = 0u; i < numberOfEntriesInThisBlock; ++i)
			{
				// PE spec: Type: Stored in the high 4 bits of the WORD
				//			Offset: Stored in the remaining 12 bits of the WORD
				const WORD low12BitMask = 0x0FFF;
				const WORD type = static_cast<WORD>(entries[i] >> 12u);
				const WORD offset = static_cast<WORD>(entries[i] & low12BitMask);

				// PE spec:
				// IMAGE_REL_BASED_ABSOLUTE: The base relocation is skipped. This type can be used to pad a block.
				// IMAGE_REL_BASED_HIGH: The base relocation adds the high 16 bits of the difference to the 16-bit field
				// at offset. The 16-bit field represents the high value of a 32-bit word.
				// IMAGE_REL_BASED_LOW: The base relocation adds the low 16 bits of the difference to the 16-bit field
				// at offset. The 16-bit field represents the low half of a 32-bit word.
				// IMAGE_REL_BASED_HIGHLOW: The base relocation applies all 32 bits of the difference to the 32-bit field at
				// offset.
				// IMAGE_REL_BASED_HIGHADJ: The base relocation adds the high 16 bits of the difference to the 16-bit field
				// at offset. The 16-bit field represents the high value of a 32-bit word. The low 16 bits of the 32-bit value
				// are stored in the 16-bit word that follows this base relocation. This means that this base relocation
				// occupies two slots.
				// IMAGE_REL_BASED_DIR64: The base relocation applies the difference to the 64-bit field at offset.
				if (type == IMAGE_REL_BASED_ABSOLUTE)
				{
					continue;
				}
				else if (type == IMAGE_REL_BASED_HIGHLOW)
				{
					uint32_t* relocation = pointer::Offset<uint32_t*>(base, blockOffset + offset);
					*relocation += static_cast<uint32_t>(baseDelta);
				}
				else if (type == IMAGE_REL_BASED_DIR64)
				{
					uint64_t* relocation = pointer::Offset<uint64_t*>(base, blockOffset + offset);
					*relocation += static_cast<uint64_t>(baseDelta);
				}
			}

			baseRelocations = pointer::Offset<const IMAGE_BASE_RELOCATION*>(baseRelocations, baseRelocations->SizeOfBlock);
			LC_ASSERT(blockSizeLeft >= blockSize, "Underflow while reading image relocations");
			blockSizeLeft -= blockSize;
		}
	}

	DestroyImageSectionDB(database);
}


executable::ImageSectionDB* executable::GatherImageSectionDB(const Image* image)
{
	const IMAGE_NT_HEADERS* ntHeader = detail::GetNtHeader(image);
	if (!ntHeader)
	{
		return nullptr;
	}

	const IMAGE_SECTION_HEADER* sectionHeader = detail::GetSectionHeader(ntHeader);
	if (!sectionHeader)
	{
		return nullptr;
	}

	ImageSectionDB* database = new ImageSectionDB;
	const size_t sectionCount = ntHeader->FileHeader.NumberOfSections;
	database->sections.reserve(sectionCount);

	for (size_t i = 0u; i < sectionCount; ++i)
	{
		database->sections.emplace_back(ImageSection { sectionHeader->VirtualAddress, sectionHeader->Misc.VirtualSize, sectionHeader->PointerToRawData, sectionHeader->SizeOfRawData });
		++sectionHeader;
	}

	std::sort(database->sections.begin(), database->sections.end(), &detail::SortSectionByAscendingRVA);

	return database;
}


void executable::DestroyImageSectionDB(ImageSectionDB* database)
{
	delete database;
}


executable::ImportModuleDB* executable::GatherImportModuleDB(const Image* image, const ImageSectionDB* imageSections)
{
	const IMAGE_NT_HEADERS* ntHeader = detail::GetNtHeader(image);
	if (!ntHeader)
	{
		return nullptr;
	}

	const IMAGE_SECTION_HEADER* sectionHeader = detail::GetSectionHeader(ntHeader);
	if (!sectionHeader)
	{
		return nullptr;
	}

	ImportModuleDB* database = new ImportModuleDB;

	// the import directory stores an array of IMAGE_IMPORT_DESCRIPTOR entries
	const size_t count = ntHeader->OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_IMPORT].Size / sizeof(IMAGE_IMPORT_DESCRIPTOR);
	if (count == 0u)
	{
		// no import modules
		return database;
	}

	const DWORD baseImportModule = RvaToFileOffset(imageSections, ntHeader->OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_IMPORT].VirtualAddress);
	if (baseImportModule == 0u)
	{
		// no import modules
		return database;
	}

	database->modules.reserve(count);

	const IMAGE_IMPORT_DESCRIPTOR* importModules = pointer::Offset<const IMAGE_IMPORT_DESCRIPTOR*>(Filesystem::GetMemoryMappedFileData(image), baseImportModule);
	while (importModules->Name != 0)
	{
		const DWORD fileOffset = RvaToFileOffset(imageSections, importModules->Name);
		const char* importModuleName = pointer::Offset<const char*>(Filesystem::GetMemoryMappedFileData(image), fileOffset);

		ImportModule module;
		strcpy_s(module.path, importModuleName);
		database->modules.emplace_back(module);

		++importModules;
	}

	return database;
}


void executable::DestroyImportModuleDB(ImportModuleDB* database)
{
	delete database;
}


executable::PdbInfo* executable::GatherPdbInfo(const Image* image, const ImageSectionDB* imageSections)
{
	const IMAGE_NT_HEADERS* ntHeader = detail::GetNtHeader(image);
	if (!ntHeader)
	{
		return nullptr;
	}

	const IMAGE_SECTION_HEADER* sectionHeader = detail::GetSectionHeader(ntHeader);
	if (!sectionHeader)
	{
		return nullptr;
	}

	// the debug directory stores an array of IMAGE_DEBUG_DIRECTORY entries
	const size_t count = ntHeader->OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_DEBUG].Size / sizeof(IMAGE_DEBUG_DIRECTORY);
	if (count == 0u)
	{
		// no debug directories
		return nullptr;
	}

	const DWORD baseDebugDirectory = RvaToFileOffset(imageSections, ntHeader->OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_DEBUG].VirtualAddress);
	if (baseDebugDirectory == 0u)
	{
		// no debug directories
		return nullptr;
	}

	const IMAGE_DEBUG_DIRECTORY* debugDirectory = pointer::Offset<const IMAGE_DEBUG_DIRECTORY*>(Filesystem::GetMemoryMappedFileData(image), baseDebugDirectory);
	for (size_t i=0u; i < count; ++i)
	{
		// we are only interested in PDB files
		if (debugDirectory->Type == IMAGE_DEBUG_TYPE_CODEVIEW)
		{
			// try interpreting the raw data as PDB 7.0 info. if it belongs to PDB 7.0 data, the signature matches "RSDS"
			// http://www.debuginfo.com/articles/debuginfomatch.html
			struct PDB70Header
			{
				DWORD signature;	// 'RSDS'
				GUID guid;
				DWORD age;
			};

			const DWORD RSDS = 0x53445352u;
			const PDB70Header* pdbHeader = pointer::Offset<const PDB70Header*>(Filesystem::GetMemoryMappedFileData(image), debugDirectory->PointerToRawData);
			if (pdbHeader->signature == RSDS)
			{
				// PDB filename follows right after the header data
				const char* pdbPath = pointer::Offset<const char*>(pdbHeader, sizeof(PDB70Header));

				PdbInfo* pdbInfo = new PdbInfo;
				pdbInfo->guid = pdbHeader->guid;
				pdbInfo->age = pdbHeader->age;
				strcpy_s(pdbInfo->path, pdbPath);

				return pdbInfo;
			}
		}
	}

	return nullptr;
}


void executable::DestroyPdbInfo(PdbInfo* pdbInfo)
{
	delete pdbInfo;
}


void executable::CallDllEntryPoint(void* moduleBase, uint32_t entryPointRva)
{
	typedef decltype(_DllMainCRTStartup) DllEntryPoint;

	DllEntryPoint* entryPoint = pointer::Offset<DllEntryPoint*>(moduleBase, entryPointRva);
	entryPoint(static_cast<HINSTANCE>(moduleBase), DLL_PROCESS_ATTACH, NULL);
}


#endif // LC_VERSION