// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_PointerUtil.h"
#include "LC_MemoryMappedFile.h"
#include "LC_Hashing.h"
// BEGIN EPIC MOD
#include <vector>
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

namespace executable
{
#if LC_64_BIT
	typedef uint64_t PreferredBase;
#else
	typedef uint32_t PreferredBase;
#endif

	struct Header
	{
		uint64_t sizeOnDisk;
		IMAGE_FILE_HEADER imageHeader;
		IMAGE_OPTIONAL_HEADER optionalHeader;
	};

	typedef Filesystem::MemoryMappedFile Image;

	struct ImageSection
	{
		uint32_t rva;
		uint32_t size;
		uint32_t rawDataRva;
		uint32_t rawDataSize;
	};

	struct ImageSectionDB
	{
		std::vector<ImageSection> sections;
	};

	struct ImportModule
	{
		char path[MAX_PATH];
	};

	struct ImportModuleDB
	{
		std::vector<ImportModule> modules;
	};

	struct PdbInfo
	{
		char path[MAX_PATH];	// can be an absolute or relative path
		GUID guid;
		uint32_t age;
	};


	Image* OpenImage(const wchar_t* filename, Filesystem::OpenMode::Enum openMode);
	void CloseImage(Image*& image);

	void RebaseImage(Image* image, PreferredBase preferredBase);


	ImageSectionDB* GatherImageSectionDB(const Image* image);
	void DestroyImageSectionDB(ImageSectionDB* database);

	ImportModuleDB* GatherImportModuleDB(const Image* image, const ImageSectionDB* imageSections);
	void DestroyImportModuleDB(ImportModuleDB* database);

	// reads the PDB info stored in the image's debug directories (if available)
	PdbInfo* GatherPdbInfo(const Image* image, const ImageSectionDB* imageSections);
	void DestroyPdbInfo(PdbInfo* pdbInfo);



	// returns the RVA of the entry point of the image when loaded into memory
	uint32_t GetEntryPointRva(const Image* image);

	// returns the preferred address at which the image is to be loaded into memory
	PreferredBase GetPreferredBase(const Image* image);

	// returns the image's header
	Header GetHeader(const Image* image);

	// checks whether a header is valid
	bool IsValidHeader(const Header& header);

	// returns the size of the image when loaded into memory
	uint32_t GetSize(const Image* image);

	// helper functions
	uint32_t RvaToFileOffset(const ImageSectionDB* database, uint32_t rva);

	void ReadFromFileOffset(const Image* file, uint32_t offset, void* destination, size_t byteCount);
	void WriteToFileOffset(Image* file, uint32_t offset, const void* source, size_t byteCount);


	// helper function to directly read from an RVA in the given image
	template <typename T>
	inline T ReadFromImage(const Image* image, const ImageSectionDB* database, uint32_t rva)
	{
		const uint32_t fileOffset = RvaToFileOffset(database, rva);
		if (fileOffset == 0u)
		{
			// don't try to read from sections without data in the image, e.g. .bss
			return T(0);
		}

		const T* address = pointer::Offset<const T*>(Filesystem::GetMemoryMappedFileData(image), fileOffset);
		return *address;
	}

	void CallDllEntryPoint(void* moduleBase, uint32_t entryPointRva);
}


// specializations to allow executable::Header to be used as key in maps and sets
namespace std
{
	template <>
	struct hash<executable::Header>
	{
		inline std::size_t operator()(const executable::Header& header) const
		{
			const uint32_t hash1 = Hashing::Hash32(&header.imageHeader, sizeof(executable::Header::imageHeader), 0u);
			const uint32_t hash2 = Hashing::Hash32(&header.optionalHeader, sizeof(executable::Header::optionalHeader), hash1);

			return hash2;
		}
	};

	template <>
	struct equal_to<executable::Header>
	{
		inline bool operator()(const executable::Header& lhs, const executable::Header& rhs) const
		{
			// comparing just the size on disk is fastest
			if (lhs.sizeOnDisk != rhs.sizeOnDisk)
			{
				return false;
			}
			// comparing the image header is the second-best option
			else if (::memcmp(&lhs.imageHeader, &rhs.imageHeader, sizeof(executable::Header::imageHeader)) != 0)
			{
				return false;
			}
			// comparing the optional header is slowest
			else if (::memcmp(&lhs.optionalHeader, &rhs.optionalHeader, sizeof(executable::Header::optionalHeader)) != 0)
			{
				return false;
			}

			return true;
		}
	};
}


#endif // LC_VERSION