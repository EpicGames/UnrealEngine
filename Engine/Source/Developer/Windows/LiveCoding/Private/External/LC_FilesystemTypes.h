// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

#include "LC_FixedSizeString.h"


namespace Filesystem
{
	struct OpenMode
	{
		enum Enum
		{
			READ = 0,
			READ_WRITE = 1
		};
	};

	struct DriveType
	{
		enum Enum
		{
			UNKNOWN = 0,
			REMOVABLE,
			FIXED,
			REMOTE,
			OPTICAL,
			RAMDISK
		};
	};

	struct PathAttributes
	{
		uint64_t size;
		uint64_t lastModificationTime;
		uint32_t flags;
	};

	typedef FixedSizeString<wchar_t, MAX_PATH> Path;
}


#endif // LC_VERSION