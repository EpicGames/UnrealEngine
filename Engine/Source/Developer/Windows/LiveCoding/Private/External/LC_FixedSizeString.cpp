// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_FixedSizeString.h"
#include "LC_StringUtil.h"

// BEGIN EPIC MOD
// This is really treated as an include file.
//#pragma once 
#ifndef LC_FIXEDSIZESTRING_CPP
#define LC_FIXEDSIZESTRING_CPP
// END EPIC MOD

template <typename T, unsigned int N>
FixedSizeString<T, N>::FixedSizeString(void)
	: m_length(0u)
{
	m_data[0] = '\0';
}


template <typename T, unsigned int N>
FixedSizeString<T, N>::FixedSizeString(const T* other)
	: m_length(static_cast<unsigned int>(string::GetLength(other)))
{
	memcpy(m_data, other, sizeof(T)*(m_length + 1u));
}


template <typename T, unsigned int N>
FixedSizeString<T, N>::FixedSizeString(const FixedSizeString<T, N>& other)
	: m_length(other.m_length)
{
	memcpy(m_data, other.m_data, sizeof(T)*(other.m_length + 1u));
}


template <typename T, unsigned int N>
FixedSizeString<T, N>::FixedSizeString(const T* other, unsigned int length)
	: m_length(length)
{
	memcpy(m_data, other, sizeof(T)*length);
	m_data[length] = '\0';
}


template <typename T, unsigned int N>
FixedSizeString<T, N>& FixedSizeString<T, N>::operator=(const FixedSizeString<T, N>& other)
{
	if (this != &other)
	{
		memcpy(m_data, other.m_data, sizeof(T)*(other.m_length + 1u));
		m_length = other.m_length;
	}

	return *this;
}


template <typename T, unsigned int N>
FixedSizeString<T, N>& FixedSizeString<T, N>::operator=(const T* other)
{
	const size_t length = string::GetLength(other);
	memcpy(m_data, other, sizeof(T)*(length + 1u));
	m_length = static_cast<unsigned int>(length);

	return *this;
}


template <typename T, unsigned int N>
FixedSizeString<T, N>& FixedSizeString<T, N>::operator+=(const FixedSizeString<T, N>& other)
{
	memcpy(m_data + m_length, other.m_data, sizeof(T)*(other.m_length + 1u));
	m_length += other.m_length;

	return *this;
}


template <typename T, unsigned int N>
FixedSizeString<T, N>& FixedSizeString<T, N>::operator+=(const T* other)
{
	const size_t length = string::GetLength(other);
	memcpy(m_data + m_length, other, sizeof(T)*(length + 1u));
	m_length += static_cast<unsigned int>(length);

	return *this;
}


template <typename T, unsigned int N>
FixedSizeString<T, N> FixedSizeString<T, N>::ToLower(void) const LC_RESTRICT
{
	FixedSizeString<T, N> result;
	result.m_length = m_length;

	for (unsigned int i = 0u; i < m_length; ++i)
	{
		result.m_data[i] = string::ToLower(m_data[i]);
	}
	result.m_data[m_length] = '\0';

	return result;
}


template <typename T, unsigned int N>
FixedSizeString<T, N> FixedSizeString<T, N>::ToUpper(void) const LC_RESTRICT
{
	FixedSizeString<T, N> result;
	result.m_length = m_length;

	for (unsigned int i = 0u; i < m_length; ++i)
	{
		result.m_data[i] = string::ToUpper(m_data[i]);
	}
	result.m_data[m_length] = '\0';

	return result;
}

#endif // LC_FIXEDSIZESTRING_CPP

#endif // LC_VERSION