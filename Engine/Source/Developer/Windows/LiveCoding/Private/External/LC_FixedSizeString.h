// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
// END EPIC MOD

template <typename T, unsigned int N>
class FixedSizeString
{
public:
	static const unsigned int CAPACITY = N;

	FixedSizeString(void);
	explicit FixedSizeString(const T* other);
	FixedSizeString(const FixedSizeString<T, N>& other);
	FixedSizeString(const T* other, unsigned int length);

	FixedSizeString<T, N>& operator=(const FixedSizeString<T, N>& other);
	FixedSizeString<T, N>& operator=(const T* other);

	FixedSizeString<T, N>& operator+=(const FixedSizeString<T, N>& other);
	FixedSizeString<T, N>& operator+=(const T* other);

	FixedSizeString<T, N> ToLower(void) const LC_RESTRICT;
	FixedSizeString<T, N> ToUpper(void) const LC_RESTRICT;

	inline unsigned int GetLength(void) const
	{
		return m_length;
	}

	inline const T* GetString(void) const
	{
		return m_data;
	}

private:
	unsigned int m_length;
	T m_data[N];
};


#endif // LC_VERSION