// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_FixedSizeString.cpp"
// BEGIN EPIC MOD
#include "Windows/WindowsHWrapper.h"
#include <ws2tcpip.h> // for INET_ADDRSTRLEN
// END EPIC MOD

// explicit template instantiation of all used types
template class FixedSizeString<wchar_t, MAX_PATH>;
template class FixedSizeString<char, INET_ADDRSTRLEN>;


#endif // LC_VERSION