// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "Windows/WindowsHWrapper.h"
#include <atomic>
// END EPIC MOD

#define LC_FILE									__FILE__
#define LC_LINE									__LINE__
#define LC_FUNCTION_NAME						__FUNCTION__
#define LC_FUNCTION_SIGNATURE					__FUNCSIG__

#define LC_DISABLE_COPY(_name)					_name(const _name&) = delete
#define LC_DISABLE_MOVE(_name)					_name(_name&&) = delete
#define LC_DISABLE_ASSIGNMENT(_name)			_name& operator=(const _name&) = delete
#define LC_DISABLE_MOVE_ASSIGNMENT(_name)		_name& operator=(_name&&) = delete

#define LC_ALWAYS_INLINE						__forceinline
#define LC_NEVER_INLINE							__declspec(noinline)

#define LC_RESTRICT								__restrict
#define LC_UNUSED(_value)						(void)(_value)

// BEGIN EPIC MOD
#define LC_COMPILER_FENCE						std::atomic_signal_fence(std::memory_order_acq_rel)
#define LC_MEMORY_FENCE							std::atomic_thread_fence(std::memory_order_acq_rel)
// END EPIC MOD

#endif // LC_VERSION