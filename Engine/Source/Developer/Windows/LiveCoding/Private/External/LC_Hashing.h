// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

#include "xxhash.h"
// BEGIN EPIC MOD
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
// END EPIC MOD

namespace Hashing
{
	LC_ALWAYS_INLINE uint32_t Hash32(const void* input, size_t length, uint32_t seed)
	{
		return XXH32(input, length, seed);
	}
}


#endif // LC_VERSION