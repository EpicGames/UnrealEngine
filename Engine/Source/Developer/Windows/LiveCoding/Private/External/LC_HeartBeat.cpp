// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_HeartBeat.h"
#include "LC_InterprocessMutex.h"
#include "LC_NamedSharedMemory.h"
#include "LC_UtcTime.h"
#include "LC_PrimitiveNames.h"


HeartBeat::HeartBeat(const wchar_t* const processGroupName, Process::Id processId)
	: m_mutex(nullptr)
	, m_memory(nullptr)
{
	m_mutex = new InterprocessMutex(primitiveNames::HeartBeatMutex(processGroupName, processId).c_str());
	m_memory = Process::CreateNamedSharedMemory(primitiveNames::HeartBeatNamedSharedMemory(processGroupName, processId).c_str(), 4096u);
}


HeartBeat::~HeartBeat(void)
{
	Process::DestroyNamedSharedMemory(m_memory);
	delete m_mutex;
}


void HeartBeat::Store(void)
{
	const uint64_t currentTime = utcTime::GetCurrent();

	InterprocessMutex::ScopedLock lock(m_mutex);
	Process::WriteNamedSharedMemory(m_memory, currentTime);
}


uint64_t HeartBeat::ReadBeatDelta(void) const
{
	const uint64_t currentTime = utcTime::GetCurrent();
	const uint64_t heartBeat = ReadBeat();

	if (currentTime >= heartBeat)
	{
		return currentTime - heartBeat;
	}
	else
	{
		return heartBeat - currentTime;
	}
}


uint64_t HeartBeat::ReadBeat(void) const
{
	InterprocessMutex::ScopedLock lock(m_mutex);
	return Process::ReadNamedSharedMemory<uint64_t>(m_memory);
}


#endif // LC_VERSION