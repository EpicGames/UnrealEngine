// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ProcessTypes.h"
#include "LC_NamedSharedMemoryTypes.h"


class InterprocessMutex;

// used for communicating heart beats between client and server.
// can be used by several processes.
class HeartBeat
{
public:
	HeartBeat(const wchar_t* const processGroupName, Process::Id processId);
	~HeartBeat(void);

	// stores the current UTC time as heart beat
	void Store(void);

	// reads the last stored heart beat and compares it against the current UTC time
	uint64_t ReadBeatDelta(void) const;

private:
	uint64_t ReadBeat(void) const;

	InterprocessMutex* m_mutex;
	Process::NamedSharedMemory* m_memory;
};


#endif // LC_VERSION