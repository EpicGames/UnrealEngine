// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_InterprocessMutex.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

InterprocessMutex::InterprocessMutex(const wchar_t* name)
	// BEGIN EPIC MOD
	: m_mutex(::CreateMutexW(NULL, Windows::FALSE, name))
	// END EPIC MOD
{
}


InterprocessMutex::~InterprocessMutex(void)
{
	::CloseHandle(m_mutex);
}


void InterprocessMutex::Lock(void)
{
	const DWORD result = ::WaitForSingleObject(m_mutex, INFINITE);
	switch (result)
	{
		case WAIT_OBJECT_0:
			// mutex was successfully signaled
			break;

		case WAIT_TIMEOUT:
			// the operation timed out, which should never happen with a timeout of INFINITE
			LC_ERROR_DEV("%s", "Mutex timed out.");
			break;

		case WAIT_ABANDONED:
			LC_ERROR_DEV("%s", "Wait() was called on a stale mutex which was not released by the owning thread.");
			break;

		case WAIT_FAILED:
			LC_ERROR_DEV("%s", "Failed to Wait() on a mutex.");
			break;

		default:
			break;
	}
}


void InterprocessMutex::Unlock(void)
{
	::ReleaseMutex(m_mutex);
}


#endif // LC_VERSION