// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "Windows/MinimalWindowsApi.h"
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
// END EPIC MOD

class InterprocessMutex
{
public:
	explicit InterprocessMutex(const wchar_t* name);
	~InterprocessMutex(void);

	void Lock(void);
	void Unlock(void);

	class ScopedLock
	{
	public:
		explicit ScopedLock(InterprocessMutex* mutex)
			: m_mutex(mutex)
		{
			mutex->Lock();
		}

		~ScopedLock(void)
		{
			m_mutex->Unlock();
		}

	private:
		LC_DISABLE_COPY(ScopedLock);
		LC_DISABLE_MOVE(ScopedLock);
		LC_DISABLE_ASSIGNMENT(ScopedLock);
		LC_DISABLE_MOVE_ASSIGNMENT(ScopedLock);

		InterprocessMutex* m_mutex;
	};

private:
	// BEGIN EPIC MOD
	Windows::HANDLE m_mutex;
	// END EPIC MOD
};


#endif // LC_VERSION