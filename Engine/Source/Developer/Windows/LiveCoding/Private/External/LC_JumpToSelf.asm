; Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

PUBLIC ?JumpToSelf@@YAXXZ

_TEXT SEGMENT

?JumpToSelf@@YAXXZ PROC

Self:
    jmp Self

?JumpToSelf@@YAXXZ ENDP

_TEXT ENDS

END
