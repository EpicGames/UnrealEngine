// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Logging.h"
// BEGIN EPIC MOD
//#include "LC_App.h"
//#include "LC_TimeDate.h"
// END EPIC MOD
#include "LC_Atomic_Windows.h"
// BEGIN EPIC MOD
#include "Logging/LogMacros.h"
#include "LiveCodingLog.h"
// END EPIC MOD

namespace
{
	// we want to be able to dump large things, including environment variable blocks, but 32k on the stack is too much
	static const size_t PER_THREAD_BUFFER_SIZE = 32u * 1024u;
	static thread_local char gtls_logBuffer[PER_THREAD_BUFFER_SIZE] = {};

	// keeps track of the current indentation
	static volatile int32_t g_indentationLevel[Logging::Channel::COUNT] = {};

	// small lookup table for different levels of indentation
	static const int MAX_INDENTATION_LEVELS = 7;
	static const char* const INDENTATION_STRINGS[MAX_INDENTATION_LEVELS] =
	{
		"",						// level 0
		"  o ",					// level 1
		"    - ",				// level 2
		"      * ",				// level 3
		"        o ",			// level 4
		"          - ",			// level 5
		"            * "		// level 6
	};


	// BEGIN EPIC MOD - Redirecting log output
	static void DefaultOutputHandler(Logging::Channel::Enum channel, Logging::Type::Enum type, const wchar_t* const Message)
	{
		FString MessageWithoutNewline = FString(Message).TrimEnd();
		switch (type)
		{
		case Logging::Type::LOG_WARNING:
			UE_LOGF(LogLiveCoding, Warning, "%ls", *MessageWithoutNewline);
			break;
		case Logging::Type::LOG_ERROR:
			UE_LOGF(LogLiveCoding, Error, "%ls", *MessageWithoutNewline);
			break;
		default:
			UE_LOGF(LogLiveCoding, Display, "%ls", *MessageWithoutNewline);
			break;
		}
	}

	static Logging::OutputHandlerType OutputHandler = &DefaultOutputHandler;
	// END EPIC MOD - Redirecting log output

	// BEGIN EPIC MOD - Explicit API for enabling log channels
	static bool ChannelEnabled[Logging::Channel::COUNT] = { };
	// END EPIC MOD - Explicit API for enabling log channels

	static bool IsChannelEnabled(Logging::Channel::Enum channel, Logging::Type::Enum type)
	{
		// warnings, errors and success logs should always be output to all channels
		if (type != Logging::Type::LOG_INFO)
		{
			return true;
		}

		// BEGIN EPIC MOD - Explicit API for enabling log channels
		switch (channel)
		{
			case Logging::Channel::USER:
				// user-visible logs are *always* logged
				return true;

			default:
				// disabled or unknown channel
				return ChannelEnabled[channel];
		}
		// END EPIC MOD - Explicit API for enabling log channels
	}
}


Logging::Indent::Indent(Channel::Enum channel)
	: m_channel(channel)
{
	IncrementIndentation(channel);
}


Logging::Indent::~Indent(void)
{
	DecrementIndentation(m_channel);
}


void Logging::IncrementIndentation(Channel::Enum channel)
{
	Atomic::IncrementConsistent(&g_indentationLevel[channel]);
}


void Logging::DecrementIndentation(Channel::Enum channel)
{
	Atomic::DecrementConsistent(&g_indentationLevel[channel]);
}


const char* Logging::GetIndentation(Logging::Channel::Enum channel)
{
	int indent = g_indentationLevel[channel];
	if (indent >= MAX_INDENTATION_LEVELS)
	{
		return INDENTATION_STRINGS[MAX_INDENTATION_LEVELS - 1];
	}

	return INDENTATION_STRINGS[indent];
}


template <> void Logging::LogNoFormat<Logging::Channel::Enum::USER>(const wchar_t* const buffer)
{
	// BEGIN EPIC MOD - Redirecting log output
	OutputHandler(Logging::Channel::Enum::USER, Type::LOG_INFO, buffer);
	// END EPIC MOD - Redirecting log output
}


template <> void Logging::LogNoFormat<Logging::Channel::Enum::DEV>(const char* const buffer)
{
	// BEGIN EPIC MOD - Redirecting log output
	OutputHandler(Logging::Channel::Enum::DEV, Type::LOG_INFO, ANSI_TO_TCHAR(buffer));
	// END EPIC MOD - Redirecting log output
}


void Logging::Log(Channel::Enum channel, Type::Enum type, const char* const format, ...)
{
	if (!IsChannelEnabled(channel, type))
	{
		return;
	}

	va_list argptr;
	va_start(argptr, format);

	char (&buffer)[PER_THREAD_BUFFER_SIZE] = gtls_logBuffer;
	_vsnprintf_s(buffer, _TRUNCATE, format, argptr);

	// BEGIN EPIC MOD - Redirecting log output
	OutputHandler(channel, type, ANSI_TO_TCHAR(buffer));
	// END EPIC MOD - Redirecting log output

	va_end(argptr);
}

// BEGIN EPIC MOD - Redirecting log output
void Logging::EnableChannel(Channel::Enum channel, bool enabled)
{
	check(channel != Logging::Channel::USER || enabled);
	ChannelEnabled[(int)channel] = enabled;
}

void Logging::SetOutputHandler(Logging::OutputHandlerType handler)
{
	if (handler == nullptr)
	{
		OutputHandler = &DefaultOutputHandler;
	}
	else
	{
		OutputHandler = handler;
	}
}
// END EPIC MOD - Redirecting log output


#endif // LC_VERSION