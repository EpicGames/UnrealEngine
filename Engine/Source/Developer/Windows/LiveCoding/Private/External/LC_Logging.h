// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Preprocessor.h"


namespace Logging
{
	struct Channel
	{
		enum Enum
		{
			USER,
			DEV,
			TELEMETRY,
		};

		static const unsigned int COUNT = 3u;
	};

	struct Type
	{
		enum Enum
		{
			LOG_INFO,
			LOG_WARNING,
			LOG_ERROR,
			LOG_SUCCESS
		};
	};


	// RAII helper class for indentation
	class Indent
	{
	public:
		explicit Indent(Channel::Enum channel);
		~Indent(void);

	private:
		Channel::Enum m_channel;
	};

	void IncrementIndentation(Channel::Enum channel);
	void DecrementIndentation(Channel::Enum channel);
	const char* GetIndentation(Channel::Enum channel);

	// logs without any formatting.
	template <int Channel, typename T> void LogNoFormat(const T* const buffer);

	// template specializations.
	// USER channel takes wchar_t*, DEV takes char*.
	template <> void LogNoFormat<Channel::Enum::USER>(const wchar_t* const buffer);
	template <> void LogNoFormat<Channel::Enum::DEV>(const char* const buffer);

	void Log(Channel::Enum channel, Type::Enum type, const char* const format, ...);

	// BEGIN EPIC MOD - Redirecting log output
	typedef void (*OutputHandlerType)(Channel::Enum channel, Type::Enum type, const wchar_t* const text);
	void SetOutputHandler(OutputHandlerType handler);
	// END EPIC MOD - Redirecting log output

	// BEGIN EPIC MOD - Explicit API for log channels
	void EnableChannel(Channel::Enum channel, bool enabled);
	// END EPIC MOD - Explicit API for log channels
}

#define LC_LOG_INDENT_USER						const Logging::Indent LC_PP_UNIQUE_NAME(autoIndentUser)(Logging::Channel::USER)
#define LC_LOG_INDENT_DEV						const Logging::Indent LC_PP_UNIQUE_NAME(autoIndentDev)(Logging::Channel::DEV)
#define LC_LOG_INDENT_TELEMETRY					const Logging::Indent LC_PP_UNIQUE_NAME(autoIndentTelemetry)(Logging::Channel::TELEMETRY)

// BEGIN EPIC MOD - __VA_ARGS__ -> ##__VA_ARGS__
#define LC_LOG_USER(_format, ...)				Logging::Log(Logging::Channel::USER, Logging::Type::LOG_INFO, "%s" _format "\n", Logging::GetIndentation(Logging::Channel::USER), ##__VA_ARGS__)
#define LC_LOG_DEV(_format, ...)				Logging::Log(Logging::Channel::DEV, Logging::Type::LOG_INFO, "%s" _format "\n", Logging::GetIndentation(Logging::Channel::DEV), ##__VA_ARGS__)
#define LC_LOG_TELEMETRY(_format, ...)			Logging::Log(Logging::Channel::TELEMETRY, Logging::Type::LOG_INFO, "%s" _format "\n", Logging::GetIndentation(Logging::Channel::TELEMETRY), ##__VA_ARGS__)

#define LC_WARNING_USER(_format, ...)			Logging::Log(Logging::Channel::USER, Logging::Type::LOG_WARNING, "%s" _format "\n", Logging::GetIndentation(Logging::Channel::USER), ##__VA_ARGS__)
#define LC_WARNING_DEV(_format, ...)			Logging::Log(Logging::Channel::DEV, Logging::Type::LOG_WARNING, "WARNING: %s" _format "\n", Logging::GetIndentation(Logging::Channel::DEV), ##__VA_ARGS__)
#define LC_WARNING_TELEMETRY(_format, ...)		Logging::Log(Logging::Channel::TELEMETRY, Logging::Type::LOG_WARNING, "WARNING: %s" _format "\n", Logging::GetIndentation(Logging::Channel::TELEMETRY), ##__VA_ARGS__)

#define LC_ERROR_USER(_format, ...)				Logging::Log(Logging::Channel::USER, Logging::Type::LOG_ERROR, "%s" _format "\n", Logging::GetIndentation(Logging::Channel::USER), ##__VA_ARGS__)
#define LC_ERROR_DEV(_format, ...)				Logging::Log(Logging::Channel::DEV, Logging::Type::LOG_ERROR, "ERROR: %s" _format "\n", Logging::GetIndentation(Logging::Channel::DEV), ##__VA_ARGS__)
#define LC_ERROR_TELEMETRY(_format, ...)		Logging::Log(Logging::Channel::TELEMETRY, Logging::Type::LOG_ERROR, "ERROR: %s" _format "\n", Logging::GetIndentation(Logging::Channel::TELEMETRY), ##__VA_ARGS__)

#define LC_SUCCESS_USER(_format, ...)			Logging::Log(Logging::Channel::USER, Logging::Type::LOG_SUCCESS, "%s" _format "\n", Logging::GetIndentation(Logging::Channel::USER), ##__VA_ARGS__)
// END EPIC MOD 


#endif // LC_VERSION