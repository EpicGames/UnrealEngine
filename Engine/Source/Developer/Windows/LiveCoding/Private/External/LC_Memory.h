// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD

namespace memory
{
	template <typename T>
	inline void ReleaseAndNull(T*& instance)
	{
		if (instance)
		{
			instance->Release();
			instance = nullptr;
		}
	}


	template <typename T>
	inline void DeleteAndNull(T*& instance)
	{
		delete instance;
		instance = nullptr;
	}


	template <typename T>
	inline void DeleteAndNullArray(T*& instance)
	{
		delete[] instance;
		instance = nullptr;
	}


	template <typename T, size_t N>
	bool Matches(const T(&mem1)[N], const T(&mem2)[N])
	{
		for (size_t i = 0u; i < N; ++i)
		{
			if (mem1[i] != mem2[i])
			{
				return false;
			}
		}

		return true;
	}
}


#endif // LC_VERSION