// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// BEGIN EPIC MOD
#include "LC_MemoryMappedFile.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include "LC_Vfs.h"
#include <oodle2.h>
// END EPIC MOD

namespace detail
{
	static inline DWORD GetDesiredAccess(Filesystem::OpenMode::Enum openMode)
	{
		switch (openMode)
		{
			case Filesystem::OpenMode::READ:
				return GENERIC_READ;

			case Filesystem::OpenMode::READ_WRITE:
				return GENERIC_READ | GENERIC_WRITE;
		}

		return 0u;
	}


	static inline DWORD GetShareMode(Filesystem::OpenMode::Enum openMode)
	{
		switch (openMode)
		{
			case Filesystem::OpenMode::READ:
				return FILE_SHARE_READ;

			case Filesystem::OpenMode::READ_WRITE:
				return FILE_SHARE_READ | FILE_SHARE_WRITE;
		}

		return 0u;
	}


	static inline DWORD GetFileMappingPageProtection(Filesystem::OpenMode::Enum openMode)
	{
		switch (openMode)
		{
			case Filesystem::OpenMode::READ:
				return PAGE_READONLY;

			case Filesystem::OpenMode::READ_WRITE:
				return PAGE_READWRITE;
		}

		return 0u;
	}


	static inline DWORD GetFileMappingDesiredAccess(Filesystem::OpenMode::Enum openMode)
	{
		switch (openMode)
		{
			case Filesystem::OpenMode::READ:
				return FILE_MAP_READ;

			case Filesystem::OpenMode::READ_WRITE:
				return FILE_MAP_READ | FILE_MAP_WRITE;
		}

		return 0u;
	}
}


namespace Filesystem
{
	struct MemoryMappedFile
	{
		HANDLE file;
		HANDLE fileMapping;
		void* baseAddress;
		uint64_t size;

		LC_DISABLE_ASSIGNMENT(MemoryMappedFile);
	};
}


Filesystem::MemoryMappedFile* Filesystem::OpenMemoryMappedFile(const wchar_t* path, OpenMode::Enum openMode, bool checkIfCompressed)
{
	Path temp;
	path = Devirtualize(path, temp);

	HANDLE file = ::CreateFileW(path, detail::GetDesiredAccess(openMode), detail::GetShareMode(openMode), NULL, OPEN_EXISTING, FILE_ATTRIBUTE_NORMAL, NULL);
	if (file == INVALID_HANDLE_VALUE)
	{
		LC_ERROR_USER("Cannot open file %S. Error: 0x%X", path, ::GetLastError());
		return nullptr;
	}

	// create memory-mapped file
	HANDLE fileMapping = ::CreateFileMappingW(file, NULL, detail::GetFileMappingPageProtection(openMode), 0, 0, NULL);
	if (fileMapping == NULL)
	{
		LC_ERROR_USER("Cannot create mapped file %S. Error: 0x%X", path, ::GetLastError());
		::CloseHandle(file);
		return nullptr;
	}

	void* baseAddress = ::MapViewOfFile(fileMapping, detail::GetFileMappingDesiredAccess(openMode), 0, 0, 0);
	if (baseAddress == NULL)
	{
		LC_ERROR_USER("Cannot map file %S. Error: 0x%X", path, ::GetLastError());
		::CloseHandle(fileMapping);
		::CloseHandle(file);
		return nullptr;
	}

	uint64_t fileSize = 0;

	if (checkIfCompressed)
	{
		auto data = (const BYTE*)baseAddress;
		static constexpr char UbaMagic[] = { 'U', 'B', 'A', 0x01 };
		if (memcmp(data, UbaMagic, sizeof(UbaMagic)) == 0)
		{
			data += 24;
			fileSize = *(uint64_t*)data;
			data += sizeof(uint64_t);

			::ULARGE_INTEGER integer = {};
			integer.QuadPart = fileSize;
			HANDLE memoryMapping = ::CreateFileMappingW(INVALID_HANDLE_VALUE, NULL, PAGE_READWRITE, integer.HighPart, integer.LowPart, NULL);
			if (memoryMapping == NULL)
			{
				LC_ERROR_USER("Cannot create named shared memory. Error: 0x%X", ::GetLastError());
				return nullptr;
			}
			void* memoryView = ::MapViewOfFile(memoryMapping, FILE_MAP_ALL_ACCESS, 0, 0, fileSize);

			BYTE* destMem = (BYTE*)memoryView;
			uint64_t left = fileSize;
			while (left)
			{
				uint32_t compressedBlockSize = *(uint32_t*)data;
				data += 4;
				uint32_t decompressedBlockSize = *(uint32_t*)data;
				data += 4;
				OO_SINTa decompLen = OodleLZ_Decompress(data, (OO_SINTa)compressedBlockSize, destMem, (OO_SINTa)decompressedBlockSize, OodleLZ_FuzzSafe_Yes, OodleLZ_CheckCRC_No, OodleLZ_Verbosity_None, NULL, 0, NULL, NULL, NULL, 0);
				data += compressedBlockSize;
				destMem += decompressedBlockSize;
				left -= decompressedBlockSize;
			}

			::UnmapViewOfFile(baseAddress);
			::CloseHandle(fileMapping);

			baseAddress = memoryView;
			fileMapping = memoryMapping;
		}
	}


	return new MemoryMappedFile { file, fileMapping, baseAddress, fileSize };
}


void Filesystem::CloseMemoryMappedFile(MemoryMappedFile*& file)
{
	::UnmapViewOfFile(file->baseAddress);
	::CloseHandle(file->fileMapping);
	::CloseHandle(file->file);

	delete file;
	file = nullptr;
}


void* Filesystem::GetMemoryMappedFileData(MemoryMappedFile* file)
{
	return file->baseAddress;
}


const void* Filesystem::GetMemoryMappedFileData(const MemoryMappedFile* file)
{
	return file->baseAddress;
}

uint64_t Filesystem::GetMemoryMappedSize(const MemoryMappedFile* file)
{
	if (file->size)
		return file->size;
	else
		return GetMemoryMappedFileSizeOnDisk(file);
}

uint64_t Filesystem::GetMemoryMappedFileSizeOnDisk(const MemoryMappedFile* file)
{
	::BY_HANDLE_FILE_INFORMATION info = {};
	::GetFileInformationByHandle(file->file, &info);

	::ULARGE_INTEGER integer = {};
	integer.LowPart = info.nFileSizeLow;
	integer.HighPart = info.nFileSizeHigh;

	return static_cast<uint64_t>(integer.QuadPart);
}


#endif // LC_VERSION