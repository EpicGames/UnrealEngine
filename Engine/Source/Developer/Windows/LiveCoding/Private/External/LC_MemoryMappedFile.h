// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

#include "LC_FilesystemTypes.h"


namespace Filesystem
{
	// opaque type
	struct MemoryMappedFile;

	MemoryMappedFile* OpenMemoryMappedFile(const wchar_t* path, OpenMode::Enum openMode, bool checkIfCompressed = false);
	void CloseMemoryMappedFile(MemoryMappedFile*& file);

	void* GetMemoryMappedFileData(MemoryMappedFile* file);
	const void* GetMemoryMappedFileData(const MemoryMappedFile* file);

	uint64_t GetMemoryMappedSize(const MemoryMappedFile* file);
	uint64_t GetMemoryMappedFileSizeOnDisk(const MemoryMappedFile* file);
}


#endif // LC_VERSION