// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_MemoryStream.h"
#include "LC_PointerUtil.h"
// BEGIN EPIC MOD
#include "LC_Assert.h"
// END EPIC MOD


memoryStream::Reader::Reader(const void* data, size_t capacity)
	: m_data(data)
	, m_capacity(capacity)
	, m_offset(0u)
{
}


void memoryStream::Reader::Read(void* data, size_t size)
{
	LC_ASSERT(m_offset + size <= m_capacity, "Not enough data left to read.");

	memcpy(data, pointer::Offset<const void*>(m_data, m_offset), size);
	m_offset += size;
}


void memoryStream::Reader::Seek(size_t offset)
{
	m_offset = offset;
}


memoryStream::Writer::Writer(size_t capacity)
	: m_data(new char[capacity])
	, m_capacity(capacity)
	, m_offset(0u)
{
}


memoryStream::Writer::~Writer(void)
{
	delete[] m_data;
}


void memoryStream::Writer::Write(const void* data, size_t size)
{
	LC_ASSERT(m_offset + size <= m_capacity, "Not enough space to write data.");

	memcpy(m_data + m_offset, data, size);
	m_offset += size;
}


#endif // LC_VERSION