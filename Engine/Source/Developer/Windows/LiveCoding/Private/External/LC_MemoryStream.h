// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
// END EPIC MOD

namespace memoryStream
{
	class Reader
	{
	public:
		Reader(const void* data, size_t capacity);

		void Read(void* data, size_t size);

		template <typename T>
		inline T Read(void)
		{
			T data;
			Read(&data, sizeof(T));
			return data;
		}

		void Seek(size_t offset);

	private:
		const void* m_data;
		size_t m_capacity;
		size_t m_offset;

		LC_DISABLE_COPY(Reader);
		LC_DISABLE_MOVE(Reader);
		LC_DISABLE_ASSIGNMENT(Reader);
		LC_DISABLE_MOVE_ASSIGNMENT(Reader);
	};


	class Writer
	{
	public:
		explicit Writer(size_t capacity);
		~Writer(void);

		void Write(const void* data, size_t size);

		template <typename T>
		inline void Write(const T& data)
		{
			Write(&data, sizeof(T));
		}

		inline const void* GetData(void) const
		{
			return m_data;
		}

		inline size_t GetSize(void) const
		{
			return m_offset;
		}

	private:
		char* m_data;
		size_t m_capacity;
		size_t m_offset;

		LC_DISABLE_COPY(Writer);
		LC_DISABLE_MOVE(Writer);
		LC_DISABLE_ASSIGNMENT(Writer);
		LC_DISABLE_MOVE_ASSIGNMENT(Writer);
	};
}


#endif // LC_VERSION