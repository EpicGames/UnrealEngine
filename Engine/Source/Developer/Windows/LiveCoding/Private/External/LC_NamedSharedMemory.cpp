// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_NamedSharedMemory.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include "Windows/WindowsHWrapper.h"
//END EPIC MOD

namespace Process
{
	struct NamedSharedMemory
	{
		// BEGIN EPIC MOD
		Windows::HANDLE memoryMapping;
		// END EPIC MOD
		void* memoryView;
		bool isOwned;
	};
}


bool Process::Current::DoesOwnNamedSharedMemory(const NamedSharedMemory* memory)
{
	return memory->isOwned;
}


Process::NamedSharedMemory* Process::CreateNamedSharedMemory(const wchar_t* name, size_t size)
{
	::ULARGE_INTEGER integer = {};
	integer.QuadPart = size;

	// BEGIN EPIC MOD
	Windows::HANDLE memoryMapping = ::CreateFileMappingW(INVALID_HANDLE_VALUE, NULL, PAGE_READWRITE, integer.HighPart, integer.LowPart, name);
	// END EPIC MOD
	const DWORD error = ::GetLastError();
	if (memoryMapping == NULL)
	{
		LC_ERROR_USER("Cannot create named shared memory. Error: 0x%X", error);
		return nullptr;
	}

	// check if another process already created this file mapping.
	// the handle then points to the already existing object.
	const bool isOwned = (error != ERROR_ALREADY_EXISTS);

	void* memoryView = ::MapViewOfFile(memoryMapping, FILE_MAP_ALL_ACCESS, 0, 0, size);
	if (memoryView == NULL)
	{
		LC_ERROR_USER("Cannot map view for named shared memory. Error: 0x%X", ::GetLastError());

		::CloseHandle(memoryMapping);
		return nullptr;
	}

	return new NamedSharedMemory { memoryMapping, memoryView, isOwned };
}


void Process::DestroyNamedSharedMemory(NamedSharedMemory*& memory)
{
	// BEGIN EPIC MOD
	if (memory == nullptr)
	{
		return;
	}
	// END EPIC MOD
	::UnmapViewOfFile(memory->memoryView);
	::CloseHandle(memory->memoryMapping);
	delete memory;
	memory = nullptr;
}


void Process::ReadNamedSharedMemory(const NamedSharedMemory* memory, void* buffer, size_t size)
{
	memcpy(buffer, memory->memoryView, size);
}


void Process::WriteNamedSharedMemory(NamedSharedMemory* memory, const void* buffer, size_t size)
{
	memcpy(memory->memoryView, buffer, size);
}


#endif // LC_VERSION