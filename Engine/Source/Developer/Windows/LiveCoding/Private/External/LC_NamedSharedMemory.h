// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "Windows/MinimalWindowsApi.h"
// END EPIC MOD

namespace Process
{
	// opaque type
	struct NamedSharedMemory;

	// Current/calling process.
	namespace Current
	{
		// Returns whether the named shared memory is owned by the calling process.
		bool DoesOwnNamedSharedMemory(const NamedSharedMemory* memory);
	}

	// Creates named shared memory.
	NamedSharedMemory* CreateNamedSharedMemory(const wchar_t* name, size_t size);

	// Destroys named shared memory.
	void DestroyNamedSharedMemory(NamedSharedMemory*& memory);

	// Reads from named shared memory into the given buffer.
	void ReadNamedSharedMemory(const NamedSharedMemory* memory, void* buffer, size_t size);

	// Writes from the given buffer into named shared memory.
	void WriteNamedSharedMemory(NamedSharedMemory* memory, const void* buffer, size_t size);

	// Convenience function for reading a value of a certain type from named shared memory.
	template <typename T>
	T ReadNamedSharedMemory(const NamedSharedMemory* memory)
	{
		T value = {};
		ReadNamedSharedMemory(memory, &value, sizeof(T));

		return value;
	}

	// Convenience function for writing a value of a certain type to named shared memory.
	template <typename T>
	void WriteNamedSharedMemory(NamedSharedMemory* memory, const T& value)
	{
		WriteNamedSharedMemory(memory, &value, sizeof(T));
	}
}


#endif // LC_VERSION