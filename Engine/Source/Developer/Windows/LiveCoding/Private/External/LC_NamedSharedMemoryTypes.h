// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

namespace Process
{
	// opaque type
	struct NamedSharedMemory;
}


#endif // LC_VERSION