// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
// END EPIC MOD
#include "LC_Preprocessor.h"


// determine the platform we're compiling for
#if defined(_WIN32)
	// Windows
#	if defined(_WIN64)
		// 64-bit
#		define LC_64_BIT	1
#		define LC_32_BIT	0
#	else
		// 32-bit
#		define LC_64_BIT	0
#		define LC_32_BIT	1
#	endif
#	define LC_PLATFORM_SUFFIX _Windows.h
#elif defined(__ORBIS__)
	// PlayStation 4
#	define LC_64_BIT	1
#	define LC_32_BIT	0
#	define LC_PLATFORM_SUFFIX _Orbis.h
#else
	#pragma error "Unknown platform"
#endif

// BEGIN EPIC MOD
//#define LC_PLATFORM_INCLUDE(_header) LC_PP_STRINGIFY(LC_PP_JOIN(_header, LC_PLATFORM_SUFFIX))
// END EPIC MOD

// BEGIN EPIC MOD
// convenience macro for referring to symbol identifiers.
// some identifiers contain an extra leading underscore in 32-bit builds.
#if LC_64_BIT
#	define LC_IDENTIFIER(_name)		_name
#else
#	define LC_IDENTIFIER(_name)		"_" _name
#endif
// END EPIC MOD

#endif // LC_VERSION