// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include <type_traits>


namespace pointer
{
	// BEGIN EPIC MOD
	template <typename T>
	T As(void* anyPointer);

	template <typename T>
	T As(const void* anyPointer);
	// END EPIC MOD

	template <typename T>
	T AsInteger(const void* anyPointer)
	{
		// unfortunately, conversion from pointers to integers is implementation-defined, which is sign-extended in MSVC and GCC.
		// this means that e.g. converting directly from void* to uint64_t would sign-extend on 32-bit, which is not what we want.
		// in order to convert without sign-extension on both 32-bit and 64-bit targets, we need to go via uintptr_t first.
		return static_cast<T>(reinterpret_cast<const uintptr_t>(anyPointer));
	}


	template <typename To, typename From>
	To FromInteger(From integer)
	{
		// widen integer to width of pointer type first before converting to pointer
		return reinterpret_cast<To>(static_cast<uintptr_t>(integer));
	}


	template <typename T>
	T AlignBottom(void* anyPointer, size_t alignment)
	{
		union
		{
			void* as_void;
			uintptr_t as_uintptr_t;
		};

		as_void = anyPointer;
		const size_t mask = alignment - 1u;
		as_uintptr_t &= ~mask;

		return As<T>(as_void);
	}


	template <typename T>
	T AlignBottom(const void* anyPointer, size_t alignment)
	{
		union
		{
			const void* as_void;
			uintptr_t as_uintptr_t;
		};

		as_void = anyPointer;
		const size_t mask = alignment - 1u;
		as_uintptr_t &= ~mask;

		return As<T>(as_void);
	}


	template <typename T>
	T AlignTop(void* anyPointer, size_t alignment)
	{
		union
		{
			void* as_void;
			uintptr_t as_uintptr_t;
		};

		as_void = anyPointer;
		const size_t mask = alignment - 1u;
		as_uintptr_t += mask;
		as_uintptr_t &= ~mask;

		return As<T>(as_void);
	}


	template <typename T>
	T AlignTop(const void* anyPointer, size_t alignment)
	{
		union
		{
			const void* as_void;
			uintptr_t as_uintptr_t;
		};

		as_void = anyPointer;
		const size_t mask = alignment - 1u;
		as_uintptr_t += mask;
		as_uintptr_t &= ~mask;

		return As<T>(as_void);
	}


	template <typename T>
	T As(void* anyPointer)
	{
		// only a check for pointer-type needed here.
		// T can point to const or non-const type.
		static_assert(std::is_pointer<T>::value == true, "Expected pointer type");

		union
		{
			void* as_void;
			T as_T;
		};

		as_void = anyPointer;
		return as_T;
	}


	template <typename T>
	T As(const void* anyPointer)
	{
		static_assert(std::is_pointer<T>::value == true, "Expected pointer type");

		// enforce T being a pointer to const elements
		static_assert(std::is_const<typename std::remove_pointer<T>::type>::value == true, "Wrong cv-qualifiers");

		union
		{
			const void* as_void;
			T as_T;
		};

		as_void = anyPointer;
		return as_T;
	}


	template <typename T, typename U>
	T Offset(void* anyPointer, U howManyBytes)
	{
		static_assert(std::is_pointer<T>::value == true, "Expected pointer type");

		return As<T>((As<char*>(anyPointer) + howManyBytes));
	}


	template <typename T, typename U>
	T Offset(const void* anyPointer, U howManyBytes)
	{
		static_assert(std::is_pointer<T>::value == true, "Expected pointer type");

		return As<T>((As<const char*>(anyPointer) + howManyBytes));
	}


	template <typename T>
	T Displacement(const void* from, const void* to)
	{
		static_assert(std::is_pointer<T>::value == false, "Expected value type");

		return static_cast<T>(As<const char*>(to) - As<const char*>(from));
	}
}


#endif // LC_VERSION