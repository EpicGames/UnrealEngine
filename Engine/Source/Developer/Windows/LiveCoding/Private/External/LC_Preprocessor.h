// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// joins two tokens, even when the tokens are macros
#define LC_PP_JOIN_HELPER_HELPER(_0, _1)		_0##_1
#define LC_PP_JOIN_HELPER(_0, _1)				LC_PP_JOIN_HELPER_HELPER(_0, _1)
#define LC_PP_JOIN(_0, _1)						LC_PP_JOIN_HELPER(_0, _1)

// stringifies a token, even when the token is a macro
#define LC_PP_STRINGIFY_HELPER(_0)				#_0
#define LC_PP_STRINGIFY(_0)						LC_PP_STRINGIFY_HELPER(_0)

// generates a unique name
#define LC_PP_UNIQUE_NAME(_name)				LC_PP_JOIN(_name, __COUNTER__)


#endif // LC_VERSION