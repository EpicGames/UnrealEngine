// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_PrimitiveNames.h"


// BEGIN EPIC MOD
// main suffix for all live coding primitives except pipes
#define LPP						L"_UE_LC"
// END EPIC MOD
#define LPP_JOB					LPP L"_JOB"
#define LPP_MUTEX				LPP L"_IPM"
#define LPP_MEMORY				LPP L"_NSM"
#define LPP_EVENT				LPP L"_EVT"
#define LPP_SERVER_READY		LPP L"_SR"
#define LPP_COMPILE				LPP L"_CMP"
// BEGIN EPIC MOD
#define LPP_PIPE				L"\\\\.\\pipe\\UE_LC"
#define LPP_EXCEPTION_PIPE		L"\\\\.\\pipe\\UE_LC_EXC"
// END EPIC MOD
#define LPP_HEARTBEAT_MUTEX		LPP_MUTEX L"_HB"
#define LPP_HEARTBEAT_MEMORY	LPP_MEMORY L"_HB"
#define LPP_RESTART_REQUESTED	LPP_EVENT L"_RST_REQ"
#define LPP_RESTART_PREPARED	LPP_EVENT L"_RST_PREP"
#define LPP_RESTART				LPP_EVENT L"_RST"


std::wstring primitiveNames::JobGroup(const std::wstring& processGroupName)
{
	std::wstring name;
	name.reserve(128u);

	name += processGroupName;
	name += LPP_JOB;

	return name;
}


std::wstring primitiveNames::StartupMutex(const std::wstring& processGroupName)
{
	std::wstring name;
	name.reserve(128u);

	name += processGroupName;
	name += LPP_MUTEX;

	return name;
}


std::wstring primitiveNames::StartupNamedSharedMemory(const std::wstring& processGroupName)
{
	std::wstring name;
	name.reserve(128u);

	name += processGroupName;
	name += LPP_MEMORY;

	return name;
}


std::wstring primitiveNames::ServerReadyEvent(const std::wstring& processGroupName)
{
	std::wstring name;
	name.reserve(128u);

	name += processGroupName;
	name += LPP_SERVER_READY;

	return name;
}


std::wstring primitiveNames::CompilationEvent(const std::wstring& processGroupName)
{
	std::wstring name;
	name.reserve(128u);

	name += processGroupName;
	name += LPP_COMPILE;

	return name;
}


std::wstring primitiveNames::Pipe(const std::wstring& processGroupName)
{
	std::wstring name;
	name.reserve(128u);

	name += LPP_PIPE;
	name += processGroupName;

	return name;
}


std::wstring primitiveNames::ExceptionPipe(const std::wstring& processGroupName)
{
	std::wstring name;
	name.reserve(128u);

	name += LPP_EXCEPTION_PIPE;
	name += processGroupName;

	return name;
}


std::wstring primitiveNames::HeartBeatMutex(const std::wstring& processGroupName, Process::Id processId)
{
	std::wstring name;
	name.reserve(128u);

	name += processGroupName;
	name += LPP_HEARTBEAT_MUTEX;
	name += std::to_wstring(+processId);

	return name;
}


std::wstring primitiveNames::HeartBeatNamedSharedMemory(const std::wstring& processGroupName, Process::Id processId)
{
	std::wstring name;
	name.reserve(128u);

	name += processGroupName;
	name += LPP_HEARTBEAT_MEMORY;
	name += std::to_wstring(+processId);

	return name;
}


std::wstring primitiveNames::RequestRestart(Process::Id processId)
{
	std::wstring name;
	name.reserve(128u);

	name += LPP_RESTART_REQUESTED;
	name += std::to_wstring(+processId);

	return name;
}


std::wstring primitiveNames::PreparedRestart(Process::Id processId)
{
	std::wstring name;
	name.reserve(128u);

	name += LPP_RESTART_PREPARED;
	name += std::to_wstring(+processId);

	return name;
}


std::wstring primitiveNames::Restart(Process::Id processId)
{
	std::wstring name;
	name.reserve(128u);

	name += LPP_RESTART;
	name += std::to_wstring(+processId);

	return name;
}


#undef LPP
#undef LPP_JOB
#undef LPP_MUTEX
#undef LPP_MEMORY
#undef LPP_EVENT
#undef LPP_SERVER_READY
#undef LPP_COMPILE
#undef LPP_PIPE
#undef LPP_EXCEPTION_PIPE
#undef LPP_HEARTBEAT_MUTEX
#undef LPP_HEARTBEAT_MEMORY
#undef LPP_RESTART_REQUESTED
#undef LPP_RESTART_PREPARED
#undef LPP_RESTART


#endif // LC_VERSION