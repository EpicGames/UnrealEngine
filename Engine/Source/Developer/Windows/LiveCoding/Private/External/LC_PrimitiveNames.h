// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ProcessTypes.h"
// BEGIN EPIC MOD
#include <string>
// END EPIC MOD


namespace primitiveNames
{
	// client
	std::wstring JobGroup(const std::wstring& processGroupName);
	std::wstring StartupMutex(const std::wstring& processGroupName);
	std::wstring StartupNamedSharedMemory(const std::wstring& processGroupName);

	// server
	std::wstring ServerReadyEvent(const std::wstring& processGroupName);
	std::wstring CompilationEvent(const std::wstring& processGroupName);

	// pipes
	std::wstring Pipe(const std::wstring& processGroupName);
	std::wstring ExceptionPipe(const std::wstring& processGroupName);

	// heart beat
	std::wstring HeartBeatMutex(const std::wstring& processGroupName, Process::Id processId);
	std::wstring HeartBeatNamedSharedMemory(const std::wstring& processGroupName, Process::Id processId);

	// restart
	std::wstring RequestRestart(Process::Id processId);
	std::wstring PreparedRestart(Process::Id processId);
	std::wstring Restart(Process::Id processId);
}


#endif // LC_VERSION