// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Process.h"
#include "LC_Memory.h"
#include "LC_PointerUtil.h"
#include "LC_VirtualMemory.h"
#include "LC_WindowsInternalFunctions.h"
#include "LC_Thread.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include <Psapi.h>
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
// END EPIC MOD

// BEGIN EPIC MODS
#pragma warning(push)
#pragma warning(disable:6011) // warning C6011: Dereferencing NULL pointer 'processInfo'.
#pragma warning(disable:6335) // warning C6335: Leaking process information handle 'context->pi.hProcess'.
// END EPIC MODS

namespace Process
{
	struct Context
	{
		uint32_t flags;
		HANDLE pipeReadEnd;
		PROCESS_INFORMATION processInformation;
		Thread::Handle drainStdoutThread;
		std::wstring stdoutData;
	};
}


void* Process::Current::GetBase(void)
{
	return ::GetModuleHandle(NULL);
}


Process::Handle Process::Current::GetHandle(void)
{
	return Process::Handle(::GetCurrentProcess());
}


Process::Id Process::Current::GetId(void)
{
	return Process::Id(::GetCurrentProcessId());
}


Filesystem::Path Process::Current::GetImagePath(void)
{
	wchar_t filename[Filesystem::Path::CAPACITY] = { '\0' };
	::GetModuleFileNameW(NULL, filename, Filesystem::Path::CAPACITY);

	return Filesystem::Path(filename);
}


Filesystem::Path Process::Current::GetWorkingDirectory(void)
{
	wchar_t workingDirectory[Filesystem::Path::CAPACITY] = { '\0' };
	::GetCurrentDirectoryW(Filesystem::Path::CAPACITY, workingDirectory);

	return Filesystem::Path(workingDirectory);
}


std::wstring Process::Current::GetCommandLine(void)
{
	return std::wstring(::GetCommandLineW());
}


Process::Id Process::GetId(const Context* context)
{
	return Process::Id(context->processInformation.dwProcessId);
}


Process::Handle Process::GetHandle(const Context* context)
{
	return Process::Handle(context->processInformation.hProcess);
}


std::wstring Process::GetStdOutData(const Context* context)
{
	return context->stdoutData;
}


namespace
{
	static Thread::ReturnValue DrainPipe(Process::Context* context)
	{
		std::vector<char> stdoutData;
		stdoutData.reserve(1024u);

		char buffer[1024u];
		for (;;)
		{
			DWORD bytesRead = 0u;
			if (!::ReadFile(context->pipeReadEnd, buffer, sizeof(buffer) - 1u, &bytesRead, NULL))
			{
				// error while trying to read from the pipe, process has probably ended and closed its end of the pipe
				const DWORD error = ::GetLastError();
				if (error == ERROR_BROKEN_PIPE)
				{
					// this is expected
					break;
				}

				LC_ERROR_USER("Error 0x%X while reading from pipe", error);
				break;
			}

			stdoutData.insert(stdoutData.end(), buffer, buffer + bytesRead);
		}

		// convert stdout data to UTF16
		if (stdoutData.size() > 0u)
		{
			// cl.exe and link.exe write to stdout using the OEM codepage
			const int sizeNeeded = ::MultiByteToWideChar(CP_OEMCP, 0, stdoutData.data(), static_cast<int>(stdoutData.size()), NULL, 0);

			wchar_t* strTo = new wchar_t[static_cast<size_t>(sizeNeeded)];
			::MultiByteToWideChar(CP_OEMCP, 0, stdoutData.data(), static_cast<int>(stdoutData.size()), strTo, sizeNeeded);

			context->stdoutData.assign(strTo, static_cast<size_t>(sizeNeeded));
			delete[] strTo;
		}

		return Thread::ReturnValue(0u);
	}
}


Process::Context* Process::Spawn(const wchar_t* exePath, const wchar_t* workingDirectory, const wchar_t* commandLine, const void* environmentBlock, uint32_t flags)
{
	Context* context = new Context { flags, nullptr, ::PROCESS_INFORMATION {}, Thread::Handle(nullptr), {} };

	::SECURITY_ATTRIBUTES saAttr;
	saAttr.nLength = sizeof(SECURITY_ATTRIBUTES);
	// BEGIN EPIC MOD
	saAttr.bInheritHandle = Windows::TRUE;
	// END EPIC MOD
	saAttr.lpSecurityDescriptor = NULL;

	::STARTUPINFOW startupInfo = {};
	startupInfo.cb = sizeof(startupInfo);

	// BEGIN EPIC MOD
	Windows::HANDLE hProcessStdOutRead = NULL;
	Windows::HANDLE hProcessStdOutWrite = NULL;
	Windows::HANDLE hProcessStdErrWrite = NULL;
	// END EPIC MOD

	if (flags & SpawnFlags::REDIRECT_STDOUT)
	{
		// create a STD_OUT pipe for the child process
		if (!::CreatePipe(&hProcessStdOutRead, &hProcessStdOutWrite, &saAttr, 0))
		{
			LC_ERROR_USER("Cannot create stdout pipe. Error: 0x%X", ::GetLastError());
			delete context;
			return nullptr;
		}

		// create a duplicate of the STD_OUT write handle for the STD_ERR write handle. this is necessary in case the child
		// application closes one of its STD output handles.
		// BEGIN EPIC MOD
		if (!::DuplicateHandle(::GetCurrentProcess(), hProcessStdOutWrite, ::GetCurrentProcess(),
			&hProcessStdErrWrite, 0, Windows::TRUE, DUPLICATE_SAME_ACCESS))
		// END EPIC MOD
		{
			LC_ERROR_USER("Cannot duplicate stdout pipe. Error: 0x%X", ::GetLastError());
			::CloseHandle(hProcessStdOutRead);
			::CloseHandle(hProcessStdOutWrite);
			delete context;
			return nullptr;
		}

		// the spawned process will output data into the write-end of the pipe, and our process will read from the
		// read-end. because pipes can only do some buffering, we need to ensure that pipes never get clogged, otherwise
		// the spawned process could block due to the pipe being full.
		// therefore, we also create a new thread that continuously reads data from the pipe on our end.
		context->pipeReadEnd = hProcessStdOutRead;
		context->drainStdoutThread = Thread::CreateFromFunction("DrainStdoutPipe", 16u * 1024u, &DrainPipe, context);

		startupInfo.hStdOutput = hProcessStdOutWrite;
		startupInfo.hStdError = hProcessStdErrWrite;
		startupInfo.dwFlags = STARTF_USESTDHANDLES;
	}

	wchar_t* commandLineBuffer = nullptr;
	if (commandLine)
	{
		commandLineBuffer = new wchar_t[32768];
		::wcscpy_s(commandLineBuffer, 32768u, commandLine);
	}

	LC_LOG_DEV("%s", "Spawning process:");
	{
		LC_LOG_INDENT_DEV;
		LC_LOG_DEV("Executable: %S", exePath);
		LC_LOG_DEV("Command line: %S", commandLineBuffer ? commandLineBuffer : L"none");
		LC_LOG_DEV("Working directory: %S", workingDirectory ? workingDirectory : L"none");
		LC_LOG_DEV("Custom environment block: %S", environmentBlock ? L"yes" : L"no");
		LC_LOG_DEV("Flags: %u", flags);
	}

	DWORD creationFlags = CREATE_UNICODE_ENVIRONMENT;
	if (flags & SpawnFlags::NO_WINDOW)
	{
		creationFlags |= CREATE_NO_WINDOW;
	}
	if (flags & SpawnFlags::SUSPENDED)
	{
		creationFlags |= CREATE_SUSPENDED;
	}

	// the environment block is not written to by CreateProcess, so it is safe to const_cast (it's a Win32 API mistake)
	// BEGIN EPIC MOD
	const BOOL success = ::CreateProcessW(exePath, commandLineBuffer, NULL, NULL, Windows::TRUE, creationFlags, const_cast<void*>(environmentBlock), workingDirectory, &startupInfo, &context->processInformation);
	// END EPIC MOD
	if (success == 0)
	{
		LC_ERROR_USER("Could not spawn process %S. Error: %d", exePath, ::GetLastError());
	}

	delete[] commandLineBuffer;

	if (flags & SpawnFlags::REDIRECT_STDOUT)
	{
		// we don't need those ends of the pipe
		::CloseHandle(hProcessStdOutWrite);
		::CloseHandle(hProcessStdErrWrite);
	}

	return context;
}


void Process::Destroy(Context*& context)
{
	::CloseHandle(context->processInformation.hProcess);
	::CloseHandle(context->processInformation.hThread);

	memory::DeleteAndNull(context);
}


void Process::ResumeMainThread(Context* context)
{
	::ResumeThread(context->processInformation.hThread);
}


unsigned int Process::Wait(Context* context)
{
	// wait until process terminates
	::WaitForSingleObject(context->processInformation.hProcess, INFINITE);

	if (context->flags & SpawnFlags::REDIRECT_STDOUT)
	{
		// wait until all data is drained from the pipe
		Thread::Join(context->drainStdoutThread);
		Thread::Close(context->drainStdoutThread);

		// close remaining pipe handles
		::CloseHandle(context->pipeReadEnd);
	}

	DWORD exitCode = 0xFFFFFFFFu;
	::GetExitCodeProcess(context->processInformation.hProcess, &exitCode);

	return exitCode;
}


unsigned int Process::Wait(Handle handle)
{
	// wait until process terminates
	::WaitForSingleObject(+handle, INFINITE);

	DWORD exitCode = 0xFFFFFFFFu;
	::GetExitCodeProcess(+handle, &exitCode);

	return exitCode;
}


void Process::Terminate(Context* context)
{
	Terminate(Process::Handle(context->processInformation.hProcess));
}


void Process::Terminate(Handle handle)
{
	::TerminateProcess(+handle, 0u);

	// termination is asynchronous, wait until the process is really gone
	::WaitForSingleObject(+handle, INFINITE);
}


Process::Handle Process::Open(Id processId)
{
	// BEGIN EPIC MOD
	return Process::Handle(::OpenProcess(PROCESS_ALL_ACCESS, Windows::FALSE, +processId));
	// END EPIC MOD
}


void Process::Close(Handle& handle)
{
	::CloseHandle(+handle);
	handle = INVALID_HANDLE;
}


void Process::Suspend(Context* context)
{
	WindowsInternals::NtSuspendProcess(context->processInformation.hProcess);
}


void Process::Suspend(Handle handle)
{
	WindowsInternals::NtSuspendProcess(+handle);
}


void Process::Resume(Context* context)
{
	WindowsInternals::NtResumeProcess(context->processInformation.hProcess);
}


void Process::Resume(Handle handle)
{
	WindowsInternals::NtResumeProcess(+handle);
}


bool Process::IsActive(Handle handle)
{
	DWORD exitCode = 0u;
	const BOOL success = ::GetExitCodeProcess(+handle, &exitCode);
	if ((success != 0) && (exitCode == STILL_ACTIVE))
	{
		return true;
	}

	// either the function has failed (because the process terminated unexpectedly) or the exit code
	// signals that the process exited already.
	return false;
}


bool Process::IsWoW64(Handle handle)
{
	// a WoW64 process has a PEB32 instead of a real PEB.
	// if we get a meaningful pointer to this PEB32, the process is running under WoW64.
	ULONG_PTR peb32 = 0u;
	WindowsInternals::NtQueryInformationProcess(+handle, WindowsInternals::ProcessWow64Information, &peb32, sizeof(ULONG_PTR), NULL);

	return (peb32 != 0u);
}


void Process::ReadProcessMemory(Handle handle, const void* srcAddress, void* destBuffer, size_t size)
{
	WindowsInternals::NtReadVirtualMemory(+handle, const_cast<PVOID>(srcAddress), destBuffer, static_cast<ULONG>(size), NULL);
}


void Process::WriteProcessMemory(Handle handle, void* destAddress, const void* srcBuffer, size_t size)
{
	DWORD oldProtect = 0u;
	::VirtualProtectEx(+handle, destAddress, size, PAGE_READWRITE, &oldProtect);
	{
		// instead of the regular WriteProcessMemory function, we use an undocumented function directly.
		// this is because Windows 10 introduced a performance regression that causes WriteProcessMemory to be 100 times slower (!)
		// than in previous versions of Windows.
		// this bug was reported here:
		// https://developercommunity.visualstudio.com/content/problem/228061/writeprocessmemory-slowdown-on-windows-10.html
		WindowsInternals::NtWriteVirtualMemory(+handle, destAddress, const_cast<PVOID>(srcBuffer), static_cast<ULONG>(size), NULL);
	}
	::VirtualProtectEx(+handle, destAddress, size, oldProtect, &oldProtect);
}


void* Process::ScanMemoryRange(Handle handle, const void* lowerBound, const void* upperBound, size_t size, size_t alignment)
{
	for (const void* scan = lowerBound; /* nothing */; /* nothing */)
	{
		// align address to be scanned
		scan = pointer::AlignTop<const void*>(scan, alignment);
		if (pointer::Offset<const void*>(scan, size) >= upperBound)
		{
			// outside of range to scan
			LC_ERROR_DEV("Could not find memory range that fits 0x%X bytes with alignment 0x%X in range from 0x%p to 0x%p (scan: 0x%p)", size, alignment, lowerBound, upperBound, scan);
			return nullptr;
		}
		else if (scan < lowerBound)
		{
			// outside of range (possible wrap-around)
			LC_ERROR_DEV("Could not find memory range that fits 0x%X bytes with alignment 0x%X in range from 0x%p to 0x%p (scan: 0x%p)", size, alignment, lowerBound, upperBound, scan);
			return nullptr;
		}

		::MEMORY_BASIC_INFORMATION memoryInfo = {};
		::VirtualQueryEx(+handle, scan, &memoryInfo, sizeof(::MEMORY_BASIC_INFORMATION));

		if ((memoryInfo.RegionSize >= size) && (memoryInfo.State == MEM_FREE))
		{
			return memoryInfo.BaseAddress;
		}

		// keep on searching
		scan = pointer::Offset<const void*>(memoryInfo.BaseAddress, memoryInfo.RegionSize);
	}
}


uint32_t Process::ConvertPageProtectionToExecutableProtection(uint32_t protection)
{
	// cut off PAGE_GUARD, PAGE_NOCACHE, PAGE_WRITECOMBINE, and PAGE_REVERT_TO_FILE_MAP
	const uint32_t extraBits = protection & 0xFFFFFF00u;
	const uint32_t pageProtection = protection & 0x000000FFu;

	switch (pageProtection)
	{
		case PAGE_NOACCESS:
		case PAGE_READONLY:
		case PAGE_READWRITE:
		case PAGE_WRITECOPY:
			return (pageProtection << 4u) | extraBits;

		case PAGE_EXECUTE:
		case PAGE_EXECUTE_READ:
		case PAGE_EXECUTE_READWRITE:
		case PAGE_EXECUTE_WRITECOPY:
		default:
			return protection;
	}
}


void Process::MakePagesExecutable(Handle handle, void* address, size_t size)
{
	const uint32_t pageSize = VirtualMemory::GetPageSize();
	const void* endOfRegion = pointer::Offset<const void*>(address, size);

	for (const void* scan = address; /* nothing */; /* nothing */)
	{
		::MEMORY_BASIC_INFORMATION memoryInfo = {};
		const SIZE_T bytesInBuffer = ::VirtualQueryEx(+handle, scan, &memoryInfo, sizeof(::MEMORY_BASIC_INFORMATION));
		if (bytesInBuffer == 0u)
		{
			// could not query the protection, bail out
			break;
		}

		const uint32_t executableProtection = ConvertPageProtectionToExecutableProtection(memoryInfo.Protect);
		if (executableProtection != memoryInfo.Protect)
		{
			// change this page into an executable one
			DWORD oldProtection = 0u;
			::VirtualProtectEx(+handle, memoryInfo.BaseAddress, pageSize, executableProtection, &oldProtection);
		}

		const void* endOfThisRegion = pointer::Offset<const void*>(memoryInfo.BaseAddress, pageSize);
		if (endOfThisRegion >= endOfRegion)
		{
			// we are done
			break;
		}

		// keep on walking pages
		scan = endOfThisRegion;
	}
}


void Process::FlushInstructionCache(Handle handle, void* address, size_t size)
{
	::FlushInstructionCache(+handle, address, size);
}



Process::Environment Process::CreateEnvironment(Handle handle)
{
	const void* processEnvironment = nullptr;
	// EPIC BEGIN MOD
	SIZE_T processEnvironmentSize = 0;
	// EPIC END MOD

	const bool isWow64 = IsWoW64(handle);
	if (!isWow64)
	{
		// this is either a 32-bit process running on 32-bit Windows, or a 64-bit process running on 64-bit Windows.
		// the environment can be retrieved directly from the process' PEB and process parameters.
		WindowsInternals::NT_PROCESS_BASIC_INFORMATION pbi = {};
		WindowsInternals::NtQueryInformationProcess(+handle, WindowsInternals::ProcessBasicInformation, &pbi, sizeof(pbi), NULL);

		const WindowsInternals::NT_PEB peb = ReadProcessMemory<WindowsInternals::NT_PEB>(handle, pbi.PebBaseAddress);
		const WindowsInternals::RTL_USER_PROCESS_PARAMETERS parameters = ReadProcessMemory<WindowsInternals::RTL_USER_PROCESS_PARAMETERS>(handle, peb.ProcessParameters);

		processEnvironment = parameters.Environment;
		// EPIC BEGIN MOD
		processEnvironmentSize = parameters.EnvironmentSize;
		// EPIC END MOD
	}
	else
	{
		// this is a process running under WoW64.
		// we must get the environment from the PEB32 of the process, rather than the "real" PEB.
		ULONG_PTR peb32Wow64 = 0u;
		WindowsInternals::NtQueryInformationProcess(+handle, WindowsInternals::ProcessWow64Information, &peb32Wow64, sizeof(ULONG_PTR), NULL);

		const WindowsInternals::NT_PEB32 peb32 = ReadProcessMemory<WindowsInternals::NT_PEB32>(handle, pointer::FromInteger<const void*>(peb32Wow64));
		const WindowsInternals::RTL_USER_PROCESS_PARAMETERS32 parameters32 = ReadProcessMemory<WindowsInternals::RTL_USER_PROCESS_PARAMETERS32>(handle, pointer::FromInteger<const void*>(peb32.ProcessParameters32));

		processEnvironment = pointer::FromInteger<const void*>(parameters32.Environment);
		// EPIC BEGIN MOD
		processEnvironmentSize = parameters32.EnvironmentSize;
		// EPIC END MOD
	}

	if (!processEnvironment)
	{
		return Environment { 0u, nullptr };
	}

	// EPIC BEGIN MOD
	Environment environment = { processEnvironmentSize, ::malloc(processEnvironmentSize) };
	// EPIC END MOD
	ReadProcessMemory(handle, processEnvironment, environment.data, environment.size);

	return environment;
}


Process::Environment Process::CreateEnvironment(Context* context)
{
	return CreateEnvironment(Process::Handle(context->processInformation.hProcess));
}


void Process::DestroyEnvironment(Environment& environment)
{
	::free(environment.data);
	environment.data = nullptr;
	environment.size = 0u;
}

// BEGIN EPIC MOD
std::vector<Thread::Id> Process::EnumerateThreads(Id processId)
// END EPIC MOD
{
	// BEGIN EPIC MOD
	std::vector<Thread::Id> threadIds;
	// END EPIC MOD
	threadIds.reserve(256u);

	// 2MB should be enough for getting the process information, even on systems with high load
	ULONG bufferSize = 2048u * 1024u;
	void* processSnapshot = nullptr;
	WindowsInternals::NTSTATUS status = 0;

	do
	{
		processSnapshot = ::malloc(bufferSize);

		// try getting a process snapshot into the provided buffer
		// BEGIN EPIC MOD
		status = WindowsInternals::NtQuerySystemInformation.ExecNoResultCheck(WindowsInternals::SystemProcessInformation, processSnapshot, bufferSize, NULL);
		// END EPIC MOD

		if (status == STATUS_INFO_LENGTH_MISMATCH)
		{
			// buffer is too small, try again
			::free(processSnapshot);
			processSnapshot = nullptr;

			bufferSize *= 2u;
		}
		else if (status < 0)
		{
			// something went wrong
			// BEGIN EPIC MOD - PVS is having problems dealing with + here. 
			WindowsInternals::NtQuerySystemInformation.CheckResult(status); // write the error
			LC_ERROR_USER("Cannot enumerate threads in process (PID: %d)", static_cast<DWORD>(processId));
			// END EPIC MOD
			::free(processSnapshot);

			return threadIds;
		}
	}
	while (status == STATUS_INFO_LENGTH_MISMATCH);

	// find the process information for the given process ID
	{
		WindowsInternals::NT_SYSTEM_PROCESS_INFORMATION* processInfo = static_cast<WindowsInternals::NT_SYSTEM_PROCESS_INFORMATION*>(processSnapshot);

		while (processInfo != nullptr)
		{
			if (processInfo->UniqueProcessId == reinterpret_cast<HANDLE>(static_cast<DWORD_PTR>(+processId)))
			{
				// we found the process we're looking for
				break;
			}

			if (processInfo->NextEntryOffset == 0u)
			{
				// we couldn't find our process
				// BEGIN EPIC MOD - PVS is having problems dealing with + here. 
				LC_ERROR_USER("Cannot enumerate threads, process not found (PID: %d)", static_cast<DWORD>(processId));
				// END EPIC MOD
				::free(processSnapshot);

				return threadIds;
			}
			else
			{
				// walk to the next process info
				processInfo = pointer::Offset<WindowsInternals::NT_SYSTEM_PROCESS_INFORMATION*>(processInfo, processInfo->NextEntryOffset);
			}
		}

		// record all threads belonging to the given process
		if (processInfo)
		{
			for (ULONG i = 0u; i < processInfo->NumberOfThreads; ++i)
			{
				const DWORD threadId = static_cast<DWORD>(reinterpret_cast<DWORD_PTR>(processInfo->Threads[i].ClientId.UniqueThread));
				threadIds.push_back(Thread::Id(threadId));
			}
		}
	}

	::free(processSnapshot);

	return threadIds;
}

// BEGIN EPIC MOD
std::vector<Process::Module> Process::EnumerateModules(Handle handle)
// END EPIC MOD
{
	// 1024 modules should be enough for most processes
	// BEGIN EPIC MOD
	std::vector<Module> modules;
	// END EPIC MOD
	modules.reserve(1024u);

	WindowsInternals::NT_PROCESS_BASIC_INFORMATION pbi = {};
	WindowsInternals::NtQueryInformationProcess(+handle, WindowsInternals::ProcessBasicInformation, &pbi, sizeof(pbi), NULL);

	const WindowsInternals::NT_PEB processPEB = ReadProcessMemory<WindowsInternals::NT_PEB>(handle, pbi.PebBaseAddress);
	const WindowsInternals::NT_PEB_LDR_DATA loaderData = ReadProcessMemory<WindowsInternals::NT_PEB_LDR_DATA>(handle, processPEB.Ldr);

	::LIST_ENTRY* listHeader = loaderData.InLoadOrderModuleList.Flink;
	::LIST_ENTRY* currentNode = listHeader;
	do
	{
		const WindowsInternals::NT_LDR_DATA_TABLE_ENTRY entry = ReadProcessMemory<WindowsInternals::NT_LDR_DATA_TABLE_ENTRY>(handle, currentNode);

		wchar_t fullDllName[Filesystem::Path::CAPACITY] = { '\0' };

		// certain modules don't have a name and DLL base, skip those
		if ((entry.DllBase != nullptr) && (entry.FullDllName.Length > 0) && (entry.FullDllName.Buffer != nullptr))
		{
			ReadProcessMemory(handle, entry.FullDllName.Buffer, fullDllName, entry.FullDllName.Length);
			modules.emplace_back(Module { Filesystem::Path(fullDllName), entry.DllBase, entry.SizeOfImage });
		}

		currentNode = entry.InLoadOrderLinks.Flink;
		if (currentNode == nullptr)
		{
			break;
		}
	}
	while (listHeader != currentNode);

	return modules;
}


Filesystem::Path Process::GetImagePath(Handle handle)
{
	DWORD charCount = Filesystem::Path::CAPACITY;
	wchar_t processName[Filesystem::Path::CAPACITY] = { '\0' };
	::QueryFullProcessImageName(+handle, 0u, processName, &charCount);

	return Filesystem::Path(processName);
}


uint32_t Process::GetModuleSize(Handle handle, void* moduleBase)
{
	::MODULEINFO info = {};
	::GetModuleInformation(+handle, static_cast<HMODULE>(moduleBase), &info, sizeof(::MODULEINFO));
	
	return info.SizeOfImage;
}

// BEGIN EPIC MOD - Allow passing environment block for linker
Process::Environment* Process::CreateEnvironmentFromMap(const TMap<FString, FString>& Pairs)
{
	std::vector<wchar_t> environmentData;
	for (const TPair<FString, FString>& Pair : Pairs)
	{
		FString Variable = FString::Printf(TEXT("%s=%s"), *Pair.Key, *Pair.Value);
		environmentData.insert(environmentData.end(), *Variable, *Variable + (Variable.Len() + 1));
	}
	environmentData.push_back('\0');

	Environment* environment = new Environment;
	environment->size = environmentData.size();
	environment->data = ::malloc(environmentData.size() * sizeof(wchar_t));
	if (environment->data != nullptr)
	{
		memcpy(environment->data, environmentData.data(), environmentData.size() * sizeof(wchar_t));
	}
	return environment;
}
// END EPIC MOD


void Process::DumpMemory(Handle handle, const void* address, size_t size)
{
	uint8_t* memory = new uint8_t[size];
	ReadProcessMemory(handle, address, memory, size);

	LC_LOG_DEV("%s", "Raw data:");
	LC_LOG_INDENT_DEV;
	for (size_t i = 0u; i < size; ++i)
	{
		LC_LOG_DEV("0x%02X", memory[i]);
	}

	delete[] memory;
}

// BEGIN EPIC MODS
#pragma warning(pop)
// END EPIC MODS

#endif // LC_VERSION