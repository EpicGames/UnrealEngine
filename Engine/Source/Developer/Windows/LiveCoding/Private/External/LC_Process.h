// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
//#include "LC_Types.h"
// END EPIC MOD
#include "LC_ProcessTypes.h"
#include "LC_ThreadTypes.h"
// BEGIN EPIC MOD
#include <string>
#include <vector>
// END EPIC MOD

namespace Process
{
	// Opaque data structure identifying a spawned process.
	struct Context;

	// Current/calling process.
	namespace Current
	{
		// Returns the base address of the image of the calling process.
		void* GetBase(void);

		// Returns the process handle of the calling process.
		Handle GetHandle(void);

		// Returns the process Id of the calling process.
		Id GetId(void);

		// Returns the path to the executable image of the calling process.
		Filesystem::Path GetImagePath(void);

		// returns the working directory of the calling process.
		Filesystem::Path GetWorkingDirectory(void);

		// Returns the command line of the calling process.
		std::wstring GetCommandLine(void);
	}

	// Returns the underlying process Id for a process identified by the given context.
	Id GetId(const Context* context);

	// Returns the underlying process handle for a process identified by the given context.
	Handle GetHandle(const Context* context);

	// Returns the underlying stdout data for a process identified by the given context.
	std::wstring GetStdOutData(const Context* context);


	// Spawns a new process.
	Context* Spawn(const wchar_t* exePath, const wchar_t* workingDirectory, const wchar_t* commandLine, const void* environmentBlock, uint32_t flags);

	// Destroys a spawned process.
	void Destroy(Context*& context);

	// Resumes a process that was spawned in a suspended state.
	void ResumeMainThread(Context* context);

	// Waits until a spawned process has exited.
	unsigned int Wait(Context* context);

	// Waits until the given process has exited.
	unsigned int Wait(Handle handle);

	// Terminates a spawned process.
	void Terminate(Context* context);

	// Terminates the given process.
	void Terminate(Handle handle);


	// Opens a process with the given Id for full access.
	Handle Open(Id processId);

	// Closes an opened process.
	void Close(Handle& handle);


	// Suspends a process.
	void Suspend(Context* context);

	// Suspends a process.
	void Suspend(Handle handle);

	// Resumes a suspended process.
	void Resume(Context* context);

	// Resumes a suspended process.
	void Resume(Handle handle);

	// Returns whether the given process is still active.
	bool IsActive(Handle handle);

	// Returns whether the given process runs under Wow64 (32-bit emulation on 64-bit versions of Windows).
	bool IsWoW64(Handle handle);

	// Reads from process memory.
	void ReadProcessMemory(Handle handle, const void* srcAddress, void* destBuffer, size_t size);

	// Writes to process memory.
	void WriteProcessMemory(Handle handle, void* destAddress, const void* srcBuffer, size_t size);

	// Convenience function for reading a value of a certain type from process memory.
	template <typename T>
	T ReadProcessMemory(Handle handle, const void* srcAddress)
	{
		T value = {};
		ReadProcessMemory(handle, srcAddress, &value, sizeof(T));

		return value;
	}

	// Convenience function for writing a value of a certain type to process memory.
	template <typename T>
	void WriteProcessMemory(Handle handle, void* destAddress, const T& value)
	{
		WriteProcessMemory(handle, destAddress, &value, sizeof(T));
	}

	// Scans a region of memory in the given process until a free block of a given size is found. Will only consider blocks at addresses with the given alignment.
	void* ScanMemoryRange(Handle handle, const void* lowerBound, const void* upperBound, size_t size, size_t alignment);

	// Converts any combination of page protection flags (e.g. PAGE_NOACCESS, PAGE_GUARD, ...) to protection flags that specify an executable page (e.g. PAGE_EXECUTE).
	uint32_t ConvertPageProtectionToExecutableProtection(uint32_t protection);

	// Makes the memory pages in the given region executable (in case they aren't already), while keeping other protection flags intact.
	void MakePagesExecutable(Handle handle, void* address, size_t size);

	// Flushes a process's instruction cache.
	void FlushInstructionCache(Handle handle, void* address, size_t size);


	// Reads and reads the environment of a given process.
	Environment CreateEnvironment(Handle handle);

	// Reads the environment of a process identified by the given context.
	Environment CreateEnvironment(Context* context);

	// Destroys an environment.
	void DestroyEnvironment(Environment& environment);


	// Enumerates all threads of a process. Only call on suspended processes.
	// BEGIN EPIC MOD
	std::vector<Thread::Id> EnumerateThreads(Id processId);
	// END EPIC MOD

	// Enumerates all modules of a process. Only call on suspended processes.
	// BEGIN EPIC MOD
	std::vector<Module> EnumerateModules(Handle handle);
	// END EPIC MOD

	// Returns the full path of a process's image, e.g. "C:\Directory\Application.exe".
	Filesystem::Path GetImagePath(Handle handle);

	// Returns the size of a module loaded into the virtual address space of a given process.
	uint32_t GetModuleSize(Handle handle, void* moduleBase);


	// Dumps any process's memory.
	void DumpMemory(Handle handle, const void* address, size_t size);

	// BEGIN EPIC MOD - Allow passing environment block for linker
	Environment* CreateEnvironmentFromMap(const TMap<FString, FString>& Pairs);
	// END EPIC MOD
}


#endif // LC_VERSION