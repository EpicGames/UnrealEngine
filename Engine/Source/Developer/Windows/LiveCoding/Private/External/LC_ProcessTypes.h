// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

#include "LC_FilesystemTypes.h"
#include "LC_StrongTypedef.h"

namespace Process
{
	struct Context;

	struct Module
	{
		Filesystem::Path fullPath;
		void* baseAddress;
		uint32_t sizeOfImage;
	};

	struct Environment
	{
		size_t size;
		void* data;
	};

	struct SpawnFlags
	{
		enum Enum : uint32_t
		{
			NONE = 0u,
			REDIRECT_STDOUT = 1u << 0u,
			NO_WINDOW = 1u << 1u,
			SUSPENDED = 1u << 2u
		};
	};

	typedef StrongTypedef<HANDLE> Handle;
	typedef StrongTypedef<DWORD> Id;

	static const Handle INVALID_HANDLE = Handle(INVALID_HANDLE_VALUE);
}


#endif // LC_VERSION