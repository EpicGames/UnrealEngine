// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Restart.h"
#include "LC_Event.h"
#include "LC_PrimitiveNames.h"
#include "LC_Process.h"
// BEGIN EPIC MOD
#include "LC_Memory.h"
#include "HAL/PlatformMisc.h"
// END EPIC MOD

namespace
{
	static Event* g_requestRestart = nullptr;
	static Event* g_restartPrepared = nullptr;
	static Event* g_executeRestart = nullptr;
}


void restart::Startup(void)
{
	const Process::Id processId = Process::Current::GetId();

	g_requestRestart = new Event(primitiveNames::RequestRestart(processId).c_str(), Event::Type::AUTO_RESET);
	g_restartPrepared = new Event(primitiveNames::PreparedRestart(processId).c_str(), Event::Type::AUTO_RESET);
	g_executeRestart = new Event(primitiveNames::Restart(processId).c_str(), Event::Type::AUTO_RESET);
}


void restart::Shutdown(void)
{
	memory::DeleteAndNull(g_requestRestart);
	memory::DeleteAndNull(g_restartPrepared);
	memory::DeleteAndNull(g_executeRestart);
}


int restart::WasRequested(void)
{
	// check if the host requested a restart
	if (g_requestRestart->TryWait())
	{
		return 1;
	}

	return 0;
}


void restart::Execute(lpp::RestartBehaviour behaviour, unsigned int exitCode)
{
	// tell the host that we finished running any custom client code and prepared everything for a restart
	g_restartPrepared->Signal();

	// wait until the host tells us to restart
	g_executeRestart->Wait();

	switch (behaviour)
	{
		// BEGIN EPIC MODS - Use UE codepath for termination to ensure logs are flushed and session analytics are sent
		case lpp::LPP_RESTART_BEHAVIOR_REQUEST_EXIT:
			FPlatformMisc::RequestExit(true);
			break;
		// END EPIC MODS

		// https://docs.microsoft.com/en-us/windows/win32/api/processthreadsapi/nf-processthreadsapi-exitprocess
		case lpp::LPP_RESTART_BEHAVIOUR_DEFAULT_EXIT:
			ExitProcess(exitCode);

		// https://docs.microsoft.com/en-us/cpp/c-runtime-library/reference/exit-exit-exit?view=vs-2019
		case lpp::LPP_RESTART_BEHAVIOUR_EXIT_WITH_FLUSH:
			exit(static_cast<int>(exitCode));

		// https://docs.microsoft.com/en-us/cpp/c-runtime-library/reference/exit-exit-exit?view=vs-2019
		case lpp::LPP_RESTART_BEHAVIOUR_EXIT:
			_Exit(static_cast<int>(exitCode));

		// https://docs.microsoft.com/en-us/windows/win32/api/processthreadsapi/nf-processthreadsapi-terminateprocess
		case lpp::LPP_RESTART_BEHAVIOUR_INSTANT_TERMINATION:
			::TerminateProcess(::GetCurrentProcess(), exitCode);
			break;
	}
}


#endif // LC_VERSION