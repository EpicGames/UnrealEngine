// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

#include "LPP_API.h"


namespace restart
{
	void Startup(void);
	void Shutdown(void);

	int WasRequested(void);
	void Execute(lpp::RestartBehaviour behaviour, unsigned int exitCode);
}


#endif // LC_VERSION