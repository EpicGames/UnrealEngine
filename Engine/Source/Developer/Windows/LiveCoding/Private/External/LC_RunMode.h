// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

struct RunMode
{
	enum Enum
	{
		DEFAULT,
		EXTERNAL_BUILD_SYSTEM,

		INVALID
	};
};


#endif // LC_VERSION