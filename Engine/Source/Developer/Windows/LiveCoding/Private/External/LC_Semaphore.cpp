// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Semaphore.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

Semaphore::Semaphore(unsigned int initialValue, unsigned int maximumValue) :
	m_sema(::CreateSemaphore(nullptr, static_cast<LONG>(initialValue), static_cast<LONG>(maximumValue), nullptr))
{
}


Semaphore::~Semaphore(void)
{
	::CloseHandle(m_sema);
}


void Semaphore::Signal(void)
{
	::ReleaseSemaphore(m_sema, 1, nullptr);
}


bool Semaphore::Wait(void)
{
	return WaitTimeout(INFINITE);
}


bool Semaphore::WaitTimeout(unsigned int milliSeconds)
{
	const DWORD result = ::WaitForSingleObject(m_sema, milliSeconds);
	switch (result)
	{
		case WAIT_OBJECT_0:
			// semaphore was successfully signaled
			return true;

		case WAIT_TIMEOUT:
			// the operation timed out, which should never happen with a timeout of INFINITE
			if (milliSeconds == INFINITE)
			{
				LC_ERROR_DEV("%s", "Semaphore timed out.");
			}
			return false;

		case WAIT_ABANDONED:
			LC_ERROR_DEV("%s", "Wait() was called on a stale semaphore which was not released by the owning thread.");
			return false;

		case WAIT_FAILED:
			LC_ERROR_DEV("%s", "Failed to Wait() on a semaphore.");
			return false;

		default:
			return false;
	}
}


bool Semaphore::TryWait(void)
{
	return WaitTimeout(0u);
}


#endif // LC_VERSION