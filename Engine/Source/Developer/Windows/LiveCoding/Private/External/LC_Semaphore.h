// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "Windows/MinimalWindowsApi.h"
// END EPIC MOD

class Semaphore
{
public:
	Semaphore(unsigned int initialValue, unsigned int maximumValue);
	~Semaphore(void);

	// Signals the semaphore.
	void Signal(void);

	// Waits until the semaphore becomes signaled, blocking.
	bool Wait(void);

	// Waits until the semaphore becomes signaled, blocking until the timeout is reached.
	// Returns whether the semaphore was signaled.
	bool WaitTimeout(unsigned int milliSeconds);

	// Returns whether the semaphore was signaled, non-blocking.
	bool TryWait(void);

private:
	// BEGIN EPIC MOD
	Windows::HANDLE m_sema;
	// END EPIC MOD
};


#endif // LC_VERSION