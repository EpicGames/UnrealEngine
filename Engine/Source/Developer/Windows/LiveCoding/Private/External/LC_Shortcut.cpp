// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Shortcut.h"
// BEGIN EPIC MOD
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

int shortcut::ConvertKeysToShortcut(bool control, bool alt, bool shift, unsigned int virtualKey)
{
	int shortcutValue = static_cast<int>(virtualKey & 0xFFu);
	shortcutValue |= control ? (1 << 8) : 0;
	shortcutValue |= alt ? (1 << 9) : 0;
	shortcutValue |= shift ? (1 << 10) : 0;

	return shortcutValue;
}


bool shortcut::ContainsControl(int shortcutValue)
{
	return (shortcutValue & (1 << 8)) != 0;
}


bool shortcut::ContainsAlt(int shortcutValue)
{
	return (shortcutValue & (1 << 9)) != 0;
}


bool shortcut::ContainsShift(int shortcutValue)
{
	return (shortcutValue & (1 << 10)) != 0;
}


int shortcut::GetVirtualKeyCode(int shortcutValue)
{
	return shortcutValue & 0xFF;
}


std::wstring shortcut::ConvertShortcutToText(int shortcutValue)
{
	const bool control = ContainsControl(shortcutValue);
	const bool alt = ContainsAlt(shortcutValue);
	const bool shift = ContainsShift(shortcutValue);
	const int virtualKey = GetVirtualKeyCode(shortcutValue);

	// get name for all keys involved in the shortcut
	const size_t MANDATORY_KEY_COUNT = 3u;
	const unsigned int MANDATORY_KEYS[MANDATORY_KEY_COUNT] = { VK_CONTROL, VK_MENU, VK_SHIFT };
	const bool MANDATORY_KEYS_DOWN[MANDATORY_KEY_COUNT] = { control, alt, shift };

	std::wstring text;
	for (unsigned int i = 0u; i < MANDATORY_KEY_COUNT; ++i)
	{
		if (MANDATORY_KEYS_DOWN[i])
		{
			const UINT scanCode = ::MapVirtualKey(MANDATORY_KEYS[i], MAPVK_VK_TO_VSC);

			wchar_t buffer[1024] = {};
			::GetKeyNameText(static_cast<LONG>(scanCode << 16), buffer, 1024);

			// append '+' to other keys
			if (text.size() != 0u)
			{
				text += L"+";
			}
			text += buffer;
		}
	}

	{
		const UINT scanCode = ::MapVirtualKey(static_cast<UINT>(virtualKey), MAPVK_VK_TO_VSC);

		wchar_t buffer[1024] = {};
		::GetKeyNameText(static_cast<LONG>(scanCode << 16), buffer, 1024);

		text += L"+";
		text += buffer;
	}

	return text;
}


#endif // LC_VERSION