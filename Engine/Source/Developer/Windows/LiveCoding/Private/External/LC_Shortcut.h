// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include <string>
// END EPIC MOD

namespace shortcut
{
	int ConvertKeysToShortcut(bool control, bool alt, bool shift, unsigned int virtualKey);
	bool ContainsControl(int shortcutValue);
	bool ContainsAlt(int shortcutValue);
	bool ContainsShift(int shortcutValue);
	int GetVirtualKeyCode(int shortcutValue);
	std::wstring ConvertShortcutToText(int shortcutValue);
}


#endif // LC_VERSION