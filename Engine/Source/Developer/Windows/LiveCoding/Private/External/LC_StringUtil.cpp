// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD

#include "LC_StringUtil.h"
// BEGIN EPIC MOD
#if defined(__clang__) && defined(_MSC_VER) && _MSVC_LANG > 201402L && __clang_major__ < 13
// For reference: https://bugs.llvm.org/show_bug.cgi?id=41226#c16
#include <wchar.h>
#endif

#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

namespace detail
{
	static std::wstring ToWideString(const char* utf8Str, size_t length)
	{
		const int sizeNeeded = ::MultiByteToWideChar(CP_UTF8, 0, utf8Str, static_cast<int>(length), NULL, 0);

		wchar_t* wstrTo = static_cast<wchar_t*>(_alloca(static_cast<size_t>(sizeNeeded) * sizeof(wchar_t)));

		::MultiByteToWideChar(CP_UTF8, 0, utf8Str, static_cast<int>(length), wstrTo, sizeNeeded);
		return std::wstring(wstrTo, static_cast<size_t>(sizeNeeded));
	}


	template <typename T>
	static bool Matches(const T* str1, const T* str2)
	{
		unsigned int index = 0u;
		T c1 = str1[index];
		T c2 = str2[index];

		while ((c1 != '\0') && (c2 != '\0'))
		{
			if (c1 != c2)
			{
				// at least one character is different
				return false;
			}

			++index;
			c1 = str1[index];
			c2 = str2[index];
		}

		// reached the end of at least one string, but the string could be of different length
		return (c1 == c2);
	}


	template <typename T>
	static bool StartsWith(const T* str, const T* subString)
	{
		unsigned int index = 0u;
		T c1 = str[index];
		T c2 = subString[index];

		while ((c1 != '\0') && (c2 != '\0'))
		{
			if (c1 != c2)
			{
				// at least one character is different
				return false;
			}

			++index;
			c1 = str[index];
			c2 = subString[index];
		}

		// reached the end of at least one string.
		// if str has ended but subString has not, it cannot be fully contained in str.
		if ((c1 == '\0') && (c2 != '\0'))
		{
			return false;
		}

		return true;
	}

	// BEGIN EPIC MOD
	template <typename T>
	static const T* StartsWithEx(const T* str, const T* subString)
	{
		for (;;)
		{
			// Reached the end of the substring, return the remaining part of original string
			T c2 = *subString++;
			if (c2 == 0)
			{
				return str;
			}

			// If the characters don't match, we are done.  We don't have to do a check specifically
			// for 0 since we know that c2 must not be zero at this point.
			T c1 = *str++;
			if (c1 != c2)
			{
				return nullptr;
			}
		}
	}

	template <typename T>
	bool MatchesWildcardRecursive(const T* target, const T* wildcard)
	{
		for (;;)
		{
			T w = *wildcard++;

			// We have reached the end of the wildcard string, this is successful if have no more target string
			if (w == 0)
			{
				return *target == 0;
			}

			// If we have a wildcard character, then 
			else if (w == '*')
			{

				// Skip any multiple wildcards 
				for (; *wildcard == '*'; ++wildcard)
				{
				}

				w = *wildcard++;

				// If wildcard is at the end of the wildcards, then we have a match
				if (w == 0)
				{
					return true;
				}

				// Look through the target string for the given character.  
				// If we reach the end of the target string without finding a match
				// then this is a failed match.  If we find the character, recurse 
				// to match the remaining part of the string
				for (;;)
				{
					T t = *target++;
					if (t == 0)
					{
						return false;
					}
					else if (t == w && MatchesWildcardRecursive(target, wildcard))
					{
						return true;
					}
				}
			}
			
			// If we don't match the character, then we don't have a match
			else if (w != *target++)
			{
				return false;
			}
		}
	}

	// Temporary replacement for std::wstring::find
	static size_t WideFindOffset(const std::wstring& haystack, const std::wstring& needle)
	{
#if defined(__clang__) && defined(_MSC_VER) && _MSVC_LANG > 201402L && __clang_major__ < 13
		// For reference: https://bugs.llvm.org/show_bug.cgi?id=41226#c16
		const wchar_t* found = ::wcsstr(haystack.c_str(), needle.c_str());
		if (found)
		{
			return static_cast<size_t>(found - haystack.c_str());
		}
		else
		{
			return std::wstring::npos;
		}
#else
		return haystack.find(needle);
#endif
	}

	// END EPIC MOD
}


namespace string
{
	std::wstring ToWideString(const char* utf8Str)
	{
		return detail::ToWideString(utf8Str, strlen(utf8Str));
	}


	std::wstring ToWideString(const char* utf8Str, size_t count)
	{
		size_t length = 0u;
		// BEGIN EPIC MOD - PVS FIX
		while ((length < count) && (utf8Str[length] != '\0'))
		// END EPIC MOD
		{
			// find null-terminator
			++length;
		}

		return detail::ToWideString(utf8Str, length);
	}


	std::wstring ToWideString(const std::string& str)
	{
		return detail::ToWideString(str.c_str(), str.length());
	}


	std::wstring Replace(const std::wstring& str, const std::wstring& from, const std::wstring& to)
	{
		std::wstring result = str;

// BEGIN EPIC MOD
		size_t startPos = detail::WideFindOffset(str, from);
// END EPIC MOD
		if (startPos == std::wstring::npos)
			return result;

		result.replace(startPos, from.length(), to);
		return result;
	}


	std::string Replace(const std::string& str, const std::string& from, const std::string& to)
	{
		std::string result = str;

		size_t startPos = str.find(from);
		if (startPos == std::string::npos)
			return result;

		result.replace(startPos, from.length(), to);
		return result;
	}


	std::string ReplaceAll(const std::string& str, const std::string& from, const std::string& to)
	{
		std::string result(str);

		for (;;)
		{
			const size_t pos = result.find(from);
			if (pos == std::string::npos)
			{
				return result;
			}

			result.replace(pos, from.length(), to);
		}
	}


	std::wstring ReplaceAll(const std::wstring& str, const std::wstring& from, const std::wstring& to)
	{
		std::wstring result(str);

		for (;;)
		{
// BEGIN EPIC MOD
			const size_t pos = detail::WideFindOffset(result, from);
// END EPIC MOD
			if (pos == std::wstring::npos)
			{
				return result;
			}

			result.replace(pos, from.length(), to);
		}
	}


	std::string EraseAll(const std::string& str, const std::string& subString)
	{
		const size_t subStringLength = subString.length();
		std::string result(str);

		for (;;)
		{
			const size_t pos = result.find(subString);
			if (pos == std::string::npos)
			{
				return result;
			}

			result.erase(pos, subStringLength);
		}
	}


	std::wstring EraseAll(const std::wstring& str, const std::wstring& subString)
	{
		const size_t subStringLength = subString.length();
		std::wstring result(str);

		for (;;)
		{
// BEGIN EPIC MOD
			const size_t pos = detail::WideFindOffset(result, subString);
// END EPIC MOD
			if (pos == std::wstring::npos)
			{
				return result;
			}

			result.erase(pos, subStringLength);
		}
	}


	char* Find(char* str, const char* subString)
	{
		return strstr(str, subString);
	}


	wchar_t* Find(wchar_t* str, const wchar_t* subString)
	{
		return wcsstr(str, subString);
	}


	const char* Find(const char* str, const char* subString)
	{
		return strstr(str, subString);
	}


	const wchar_t* Find(const wchar_t* str, const wchar_t* subString)
	{
		return wcsstr(str, subString);
	}


	const wchar_t* Find(const wchar_t* str, size_t strLength, const wchar_t* subString, size_t subStringLength)
	{
		const wchar_t* end = str + strLength;
		while (str < end - subStringLength)
		{
			size_t i = 0;
			for (; i < subStringLength; ++i)
			{
				if (str[i] != subString[i])
					break;
			}

			if (i == subStringLength)
				return str;

			++str;
		}

		return nullptr;
	}


	bool Matches(const char* str1, const char* str2)
	{
		return detail::Matches(str1, str2);
	}


	bool Matches(const wchar_t* str1, const wchar_t* str2)
	{
		return detail::Matches(str1, str2);
	}


	bool Contains(const char* str, const char* subString)
	{
		return Find(str, subString) != nullptr;
	}


	bool Contains(const wchar_t* str, const wchar_t* subString)
	{
		return Find(str, subString) != nullptr;
	}


	bool StartsWith(const char* str, const char* subString)
	{
		return detail::StartsWith(str, subString);
	}


	bool StartsWith(const wchar_t* str, const wchar_t* subString)
	{
		return detail::StartsWith(str, subString);
	}


	// BEGIN EPIC MOD
	const char* StartsWithEx(const char* str, const char* subString)
	{
		return detail::StartsWithEx(str, subString);
	}

	const wchar_t* StartsWithEx(const wchar_t* str, const wchar_t* subString)
	{
		return detail::StartsWithEx(str, subString);
	}


	bool MatchWildcard(const char* target, const char* wildcard)
	{
		return detail::MatchesWildcardRecursive(target, wildcard);
	}

	bool MatchWildcard(const wchar_t* target, const wchar_t* wildcard)
	{
		return detail::MatchesWildcardRecursive(target, wildcard);
	}
	// END EPIC MOD


	std::string ToUpper(const char* str)
	{
		std::string upperStr(str);

		char* dest = &upperStr[0];
		for (size_t i = 0u; i < upperStr.size(); ++i)
		{
			dest[i] = static_cast<char>(::toupper(dest[i]));
		}

		return upperStr;
	}


	std::string ToUpper(const std::string& str)
	{
		return ToUpper(str.c_str());
	}


	std::wstring ToUpper(const wchar_t* str)
	{
		std::wstring upperStr(str);

		wchar_t* dest = &upperStr[0];
		for (size_t i = 0u; i < upperStr.size(); ++i)
		{
			dest[i] = static_cast<wchar_t>(::towupper(dest[i]));
		}

		return upperStr;
	}


	std::wstring ToUpper(const std::wstring& str)
	{
		return ToUpper(str.c_str());
	}


	std::wstring ToLower(const wchar_t* str)
	{
		std::wstring lowerStr(str);

		wchar_t* dest = &lowerStr[0];
		for (size_t i = 0u; i < lowerStr.size(); ++i)
		{
			dest[i] = static_cast<wchar_t>(::towlower(dest[i]));
		}

		return lowerStr;
	}


	std::wstring ToLower(const std::wstring& str)
	{
		return ToLower(str.c_str());
	}


	std::wstring MakeSafeName(const std::wstring& name)
	{
		std::wstring safeName(name);

		const size_t length = name.length();
		for (size_t i = 0u; i < length; ++i)
		{
			if ((name[i] == '\\') ||
				(name[i] == '/') ||
				(name[i] == '*') ||
				(name[i] == '?') ||
				(name[i] == '"') ||
				(name[i] == '<') ||
				(name[i] == '>') ||
				(name[i] == '|') ||
				(name[i] == ':') ||
				(name[i] == ';') ||
				(name[i] == ',') ||
				(name[i] == '.'))
			{
				safeName[i] = '_';
			}
		}

		return safeName;
	}
}


#endif // LC_VERSION