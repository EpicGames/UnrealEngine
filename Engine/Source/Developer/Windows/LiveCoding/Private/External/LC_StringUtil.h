// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include <string>
// END EPIC MOD

namespace string
{
	std::wstring ToWideString(const char* utf8Str);
	std::wstring ToWideString(const char* utf8Str, size_t count);
	std::wstring ToWideString(const std::string& str);

	std::string Replace(const std::string& str, const std::string& from, const std::string& to);
	std::wstring Replace(const std::wstring& str, const std::wstring& from, const std::wstring& to);
	std::string ReplaceAll(const std::string& str, const std::string& from, const std::string& to);
	std::wstring ReplaceAll(const std::wstring& str, const std::wstring& from, const std::wstring& to);

	std::string EraseAll(const std::string& str, const std::string& subString);
	std::wstring EraseAll(const std::wstring& str, const std::wstring& subString);

	char* Find(char* str, const char* subString);
	wchar_t* Find(wchar_t* str, const wchar_t* subString);

	const char* Find(const char* str, const char* subString);
	const wchar_t* Find(const wchar_t* str, const wchar_t* subString);
	const wchar_t* Find(const wchar_t* str, size_t strLength, const wchar_t* subString, size_t subStringLength);

	bool Matches(const char* str1, const char* str2);
	bool Matches(const wchar_t* str1, const wchar_t* str2);

	bool Contains(const char* str, const char* subString);
	bool Contains(const wchar_t* str, const wchar_t* subString);

	bool StartsWith(const char* str, const char* subString);
	bool StartsWith(const wchar_t* str, const wchar_t* subString);

	// BEGIN EPIC MOD
	const char* StartsWithEx(const char* str, const char* subString);
	const wchar_t* StartsWithEx(const wchar_t* str, const wchar_t* subString);

	bool MatchWildcard(const char* target, const char* wildcard);
	bool MatchWildcard(const wchar_t* target, const wchar_t* wildcard);
	// END EPIC MOD

	inline char ToLower(char c)
	{
		return static_cast<char>(::tolower(c));
	}

	inline wchar_t ToLower(wchar_t c)
	{
		return static_cast<wchar_t>(::towlower(c));
	}

	inline char ToUpper(char c)
	{
		return static_cast<char>(::toupper(c));
	}

	inline wchar_t ToUpper(wchar_t c)
	{
		return static_cast<wchar_t>(::towupper(c));
	}


	std::string ToUpper(const char* str);
	std::string ToUpper(const std::string& str);
	std::wstring ToUpper(const wchar_t* str);
	std::wstring ToUpper(const std::wstring& str);

	std::wstring ToLower(const wchar_t* str);
	std::wstring ToLower(const std::wstring& str);

	// Turns invalid characters (\ / : * ? " < > | : ; , .) in file names, names for OS objects, etc. into underscores
	std::wstring MakeSafeName(const std::wstring& name);

	// Returns the length of the given string without null terminator
	inline size_t GetLength(const char* str)
	{
		return strlen(str);
	}

	// Returns the length of the given string without null terminator
	inline size_t GetLength(const wchar_t* str)
	{
		return wcslen(str);
	}

	// TODO: temporary fix for Orbis
#if _WIN32
	template <typename T>
	inline T StringToInt(const wchar_t* str)
	{
		return static_cast<T>(::_wtoi(str));
	}

	template <typename T>
	inline std::wstring IntToString(T value)
	{
		// ensure that the largest 64-bit integers & sign & a null-terminator fit into the buffer
		wchar_t result[22u] = {};
		_itow_s(static_cast<int>(value), result, 10);

		return std::wstring(result);
	}
#endif
}


#endif // LC_VERSION