// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

//BEGIN EPIC MOD
#include <algorithm>
//END EPIC MOD

// the tag can be used to create distinct, non-convertible types, even though the underlying type might be compatible.
template <typename T, int TAG = 0>
class StrongTypedef
{
public:
	// Typedef to make the underlying type accessible.
	typedef T Type;

	// Default constructor, default initialization.
	inline StrongTypedef(void)
		: m_value()
	{
	}

	// Constructor.
	inline explicit StrongTypedef(T value)
		: m_value(value)
	{
	}

	// Allows static_cast to value of underlying type.
	inline explicit operator T(void) const
	{
		return m_value;
	}

	// Unary promotion operator, similar to built-in types.
	inline T operator+(void) const
	{
		return m_value;
	}

	// Comparison operator.
	inline bool operator==(StrongTypedef<T> other) const
	{
		return (m_value == other.m_value);
	}

	// Comparison operator.
	inline bool operator!=(StrongTypedef<T> other) const
	{
		return (m_value != other.m_value);
	}

	// Assignment operator to allow assignment from a value of underlying type.
	inline StrongTypedef<T>& operator=(T value)
	{
		m_value = value;
		return *this;
	}

	// Provides read-write access to the underlying value.
	T* Access(void)
	{
		return &m_value;
	}

	// Provides read access to the underlying value.
	const T* Access(void) const
	{
		return &m_value;
	}

private:
	T m_value;
};


#if _WIN32
// specializations to allow StrongTypedef<T> to be used as key in maps and sets
namespace std
{
	template <typename T>
	struct hash<StrongTypedef<T>>
	{
		inline std::size_t operator()(StrongTypedef<T> value) const
		{
			return +value;
		}
	};

	template <typename T>
	struct equal_to<StrongTypedef<T>>
	{
		inline bool operator()(StrongTypedef<T> lhs, StrongTypedef<T> rhs) const
		{
			return (lhs == rhs);
		}
	};
}
#endif


#endif // LC_VERSION