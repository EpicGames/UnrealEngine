// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_SyncPoint.h"
#include "LC_CriticalSection.h"
#include "LC_Semaphore.h"


namespace
{
	static CriticalSection g_syncPointCS;
	static volatile bool g_isSyncPointUsed = false;

	static Semaphore g_enterUserSyncPoint(0u, 1u);
	static Semaphore g_leaveUserSyncPoint(0u, 1u);
	static Semaphore g_dllSyncPoint(0u, 1u);
}


void syncPoint::Enter(void)
{
	// if the sync point is not used by the user, do nothing
	if (!g_isSyncPointUsed)
	{
		return;
	}

	// tell user code that we're inside the sync point now, and in turn wait until it has reached its sync point
	g_dllSyncPoint.Signal();
	g_enterUserSyncPoint.Wait();
}


void syncPoint::Leave(void)
{
	// if the sync point is not used by the user, do nothing
	if (!g_isSyncPointUsed)
	{
		return;
	}

	// tell user code that we're finished
	g_leaveUserSyncPoint.Signal();
}


void syncPoint::EnterTarget(void)
{
	// mark the sync point as being used as soon as we enter this function the first time
	{
		CriticalSection::ScopedLock lock(&g_syncPointCS);
		g_isSyncPointUsed = true;
	}

	if (g_dllSyncPoint.TryWait())
	{
		// DLL code is currently inside sync point. tell DLL code that we are here, and wait for it to leave.
		g_enterUserSyncPoint.Signal();
		g_leaveUserSyncPoint.Wait();
	}
}


#endif // LC_VERSION