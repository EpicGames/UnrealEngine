// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD

namespace syncPoint
{
	// Called when a host process wants to enter the sync point
	void Enter(void);

	// Called when a host process wants to leave the sync point
	void Leave(void);

	// Called by the LPP API when user code in a target process wants to enter the sync point
	void EnterTarget(void);
}


#endif // LC_VERSION