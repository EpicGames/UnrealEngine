// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Thread.h"
// BEGIN EPIC MOD
#include "LC_Platform.h"
#include "LC_Logging.h"
#include <process.h>
// END EPIC MOD

// BEGIN EPIC MODS
#pragma warning(push)
#pragma warning(disable:6322) // warning C6322: Empty _except block.
#pragma warning(disable:6258) // warning C6258: Using TerminateThread does not allow proper thread clean up.
// END EPIC MODS

void Thread::Current::SetName(const char* name)
{
	const DWORD MS_VC_EXCEPTION = 0x406D1388;

	struct THREADNAME_INFO
	{
		ULONG_PTR dwType;		// Must be 0x1000.
		ULONG_PTR szName;		// Pointer to name (in user addr space).
		ULONG_PTR dwThreadID;	// Thread ID (-1 = caller thread).
		ULONG_PTR dwFlags;		// Reserved for future use, must be zero.
	};

	// code for setting a thread's name taken from http://msdn.microsoft.com/en-us/library/xcb2z8hs.aspx
	// note that the pragma directives in the code sample are wrong, and will not work for 32-bit builds.
	THREADNAME_INFO info;
	info.dwType = 0x1000;
	info.szName = reinterpret_cast<ULONG_PTR>(name);
	info.dwThreadID = static_cast<ULONG_PTR>(-1);
	info.dwFlags = 0;

	__try
	{
		RaiseException(MS_VC_EXCEPTION, 0, 4u, reinterpret_cast<ULONG_PTR*>(&info));
	}
	__except (EXCEPTION_EXECUTE_HANDLER)
	{
	}
}


void Thread::Current::SleepSeconds(unsigned int seconds)
{
	::Sleep(seconds * 1000u);
}


void Thread::Current::SleepMilliSeconds(unsigned int milliSeconds)
{
	::Sleep(milliSeconds);
}


void Thread::Current::Yield(void)
{
	::_mm_pause();
}


Thread::Id Thread::Current::GetId(void)
{
	return Thread::Id(::GetCurrentThreadId());
}


Thread::Handle Thread::Create(unsigned int stackSize, Function function, void* context)
{
	const uintptr_t result = _beginthreadex(nullptr, stackSize, function, context, 0u, nullptr);
	if (result == 0u)
	{
		const DWORD error = ::GetLastError();
		LC_ERROR_USER("Error 0x%X while trying to create thread", error);
	}

	// BEGIN EPIC MOD
	return Thread::Handle(reinterpret_cast<Windows::HANDLE>(result));
	// END EPIC MOD
}


void Thread::Join(Handle handle)
{
	::WaitForSingleObject(+handle, INFINITE);
}


void Thread::Terminate(Handle handle)
{
	::TerminateThread(+handle, 0u);
}


Thread::Handle Thread::Open(Id threadId)
{
	// BEGIN EPIC MOD
	return Thread::Handle(::OpenThread(THREAD_ALL_ACCESS, Windows::FALSE, +threadId));
	// END EPIC MOD
}


void Thread::Close(Handle& handle)
{
	::CloseHandle(+handle);
	handle = INVALID_HANDLE;
}


void Thread::Suspend(Handle handle)
{
	::SuspendThread(+handle);
}


void Thread::Resume(Handle handle)
{
	::ResumeThread(+handle);
}


void Thread::SetPriority(Handle handle, int priority)
{
	::SetThreadPriority(+handle, priority);
}


int Thread::GetPriority(Handle handle)
{
	return ::GetThreadPriority(+handle);
}


void Thread::SetContext(Handle handle, const Context* context)
{
	::SetThreadContext(+handle, context->Access());
}


Thread::Context Thread::GetContext(Handle handle)
{
	CONTEXT threadContext = {};
	threadContext.ContextFlags = CONTEXT_ALL;
	::GetThreadContext(+handle, &threadContext);

	return Thread::Context(threadContext);
}


const void* Thread::ReadInstructionPointer(const Context* context)
{
#if LC_64_BIT
	return reinterpret_cast<const void*>(context->Access()->Rip);
#else
	return reinterpret_cast<const void*>(context->Access()->Eip);
#endif
}


void Thread::WriteInstructionPointer(Context* context, const void* ip)
{
#if LC_64_BIT
	context->Access()->Rip = reinterpret_cast<DWORD64>(ip);
#else
	context->Access()->Eip = reinterpret_cast<DWORD>(ip);
#endif
}


Thread::Id Thread::GetId(Handle handle)
{
	return Thread::Id(::GetThreadId(+handle));
}

// BEGIN EPIC MODS
#pragma warning(pop)
// END EPIC MODS


#endif // LC_VERSION