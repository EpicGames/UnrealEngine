// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ThreadTypes.h"
// BEGIN EPIC MOD
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD


namespace Thread
{
	// Current/calling thread.
	namespace Current
	{
		// Sets the name of the calling thread.
		void SetName(const char* name);

		// Sleeps the calling thread for the given number of seconds.
		void SleepSeconds(unsigned int seconds);

		// Sleeps the calling thread for the given number of milliseconds.
		void SleepMilliSeconds(unsigned int milliSeconds);

		// Yields the calling thread.
		void Yield(void);

		// Returns the thread Id of the calling thread.
		Id GetId(void);
	}

	// Creates a raw thread.
	Handle Create(unsigned int stackSize, Function function, void* context);

	// Creates a thread from a given function and provided arguments.
	template <typename F, typename... Args>
	inline Handle CreateFromFunction(const char* threadName, unsigned int stackSize, F ptrToFunction, Args&&... args);

	// Creates a thread from a given member function, instance and provided arguments.
	template <class C, typename F, typename... Args>
	inline Handle CreateFromMemberFunction(const char* threadName, unsigned int stackSize, C* instance, F ptrToMemberFunction, Args&&... args);

	// Joins the given thread.
	void Join(Handle handle);

	// Terminates the given thread.
	void Terminate(Handle handle);


	// Opens a thread with the given Id for full access.
	Handle Open(Id threadId);

	// Closes a created or opened thread.
	void Close(Handle& handle);


	// Suspends a thread.
	void Suspend(Handle handle);

	// Resumes a suspended thread.
	void Resume(Handle handle);

	// Sets a thread's priority.
	void SetPriority(Handle handle, int priority);

	// Returns a thread's priority.
	int GetPriority(Handle handle);

	// Sets a thread's context. Only call on suspended threads.
	void SetContext(Handle handle, const Context* context);

	// Returns a thread's context. Only call on suspended threads.
	Context GetContext(Handle handle);

	// Reads a thread context's instruction pointer.
	const void* ReadInstructionPointer(const Context* context);

	// Writes a thread context's instruction pointer.
	void WriteInstructionPointer(Context* context, const void* ip);


	// Returns the thread Id of the given thread.
	Id GetId(Handle handle);
}


template <typename F, typename... Args>
inline Thread::Handle Thread::CreateFromFunction(const char* threadName, unsigned int stackSize, F ptrToFunction, Args&&... args)
{
	// a helper lambda that calls the given functions with the provided arguments.
	// because this lambda needs to capture, it cannot be converted to a function pointer implicitly,
	// and therefore cannot be used as a thread function directly.
	// however, we can solve this by another indirection.
	auto captureLambda = [threadName, ptrToFunction, args...]() -> Thread::ReturnValue
	{
		Thread::Current::SetName(threadName);

		return (*ptrToFunction)(args...);
	};

	// helper to make the following easier to read
	typedef decltype(captureLambda) CaptureLambdaType;

	// the lambda with captures will be called in the thread about to be created, hence it
	// must be allocated on the heap.
	CaptureLambdaType* lambdaOnHeap = new CaptureLambdaType(captureLambda);

	// here's the trick: we generate another capture-less lambda that has the same signature as a thread function,
	// and internally cast the given object to its original type, calling the lambda with captures from within
	// this capture-less lambda.
	auto capturelessLambda = [](void* lambdaContext) -> unsigned int
	{
		CaptureLambdaType* lambdaOnHeap = static_cast<CaptureLambdaType*>(lambdaContext);
		const Thread::ReturnValue result = (*lambdaOnHeap)();
		delete lambdaOnHeap;

		return +result;
	};

	Thread::Function threadFunction = capturelessLambda;
	Thread::Handle handle = Thread::Create(stackSize, threadFunction, lambdaOnHeap);

	return handle;
}


template <class C, typename F, typename... Args>
inline Thread::Handle Thread::CreateFromMemberFunction(const char* threadName, unsigned int stackSize, C* instance, F ptrToMemberFunction, Args&&... args)
{
	// a helper lambda that calls the given functions with the provided arguments.
	// because this lambda needs to capture, it cannot be converted to a function pointer implicitly,
	// and therefore cannot be used as a thread function directly.
	// however, we can solve this by another indirection.
	auto captureLambda = [threadName, ptrToMemberFunction, instance, args...]() -> Thread::ReturnValue
	{
		Thread::Current::SetName(threadName);

		return (instance->*ptrToMemberFunction)(args...);
	};

	// helper to make the following easier to read
	typedef decltype(captureLambda) CaptureLambdaType;

	// the lambda with captures will be called in the thread about to be created, hence it
	// must be allocated on the heap.
	CaptureLambdaType* lambdaOnHeap = new CaptureLambdaType(captureLambda);

	// here's the trick: we generate another capture-less lambda that has the same signature as a thread function,
	// and internally cast the given object to its original type, calling the lambda with captures from within
	// this capture-less lambda.
	auto capturelessLambda = [](void* lambdaContext) -> unsigned int
	{
		CaptureLambdaType* lambdaOnHeap = static_cast<CaptureLambdaType*>(lambdaContext);
		const Thread::ReturnValue result = (*lambdaOnHeap)();
		delete lambdaOnHeap;

		return +result;
	};

	Thread::Function threadFunction = capturelessLambda;
	Thread::Handle handle = Thread::Create(stackSize, threadFunction, lambdaOnHeap);

	return handle;
}


#endif // LC_VERSION