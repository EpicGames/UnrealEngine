// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_StrongTypedef.h"
// BEGIN EPIC MOD
#include "Windows/WindowsHWrapper.h"
#include "LC_Platform.h"
// END EPIC MOD

namespace Thread
{
	// BEGIN EPIC MOD
	typedef StrongTypedef<Windows::HANDLE> Handle;
	// END EPIC MOD
	typedef StrongTypedef<DWORD> Id;
	typedef StrongTypedef<CONTEXT> Context;
	typedef StrongTypedef<unsigned int> ReturnValue;

	// used internally
	typedef unsigned int (__stdcall *Function)(void*);

	static const Handle INVALID_HANDLE = Handle(INVALID_HANDLE_VALUE);
}


#endif // LC_VERSION