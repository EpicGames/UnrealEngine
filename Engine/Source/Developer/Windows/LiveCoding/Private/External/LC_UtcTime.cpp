// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_UtcTime.h"
// BEGIN EPIC MOD
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

uint64_t utcTime::GetCurrent(void)
{
	FILETIME fileTime = {};
	::GetSystemTimeAsFileTime(&fileTime);

	ULARGE_INTEGER asLargeInteger = {};
	asLargeInteger.LowPart = fileTime.dwLowDateTime;
	asLargeInteger.HighPart = fileTime.dwHighDateTime;

	return asLargeInteger.QuadPart;
}


#endif // LC_VERSION