// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include <inttypes.h>
// END EPIC MOD

namespace utcTime
{
	uint64_t GetCurrent(void);
}


#endif // LC_VERSION