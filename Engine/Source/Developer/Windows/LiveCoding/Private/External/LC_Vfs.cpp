// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

// BEGIN EPIC MOD

#if LC_VERSION == 1

#include "LC_Vfs.h"

namespace Filesystem
{

types::vector<VfsEntry> g_vfsEntries;

void AddVfsEntry(const wchar_t* virtualPath, const wchar_t* localPath)
{
	g_vfsEntries.emplace_back(virtualPath, localPath);
}

const types::vector<VfsEntry>& GetVfsEntries()
{
	return g_vfsEntries;
}

std::wstring Devirtualize(const std::wstring& path)
{
	for (auto& entry : g_vfsEntries)
		if (entry.virtualPath.compare(0, entry.virtualPath.size(), path, 0, entry.virtualPath.size()) == 0)
			return entry.localPath + (path.data() + entry.virtualPath.size());
	return path;
}

const wchar_t* Devirtualize(const wchar_t* path, Path& temp)
{
	for (auto& entry : g_vfsEntries)
	{
		if (entry.virtualPath.compare(0, entry.virtualPath.size(), path, 0, entry.virtualPath.size()) != 0)
			continue;
		temp = entry.localPath.c_str();
		temp += (path + entry.virtualPath.size());
		return temp.GetString();
	}
	return path;
}

}

#endif

// END EPIC MOD
