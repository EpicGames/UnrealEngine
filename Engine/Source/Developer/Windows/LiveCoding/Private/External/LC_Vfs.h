// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

// BEGIN EPIC MOD

#if LC_VERSION == 1

#include "LC_FilesystemTypes.h"
#include "LC_Types.h"

namespace Filesystem
{
	struct VfsEntry
	{
		std::wstring virtualPath;
		std::wstring localPath;
	};
	void AddVfsEntry(const wchar_t* virtualPath, const wchar_t* localPath);

	const types::vector<VfsEntry>& GetVfsEntries();

	std::wstring Devirtualize(const std::wstring& path);

	const wchar_t* Devirtualize(const wchar_t* path, Path& temp);
}


#endif

// END EPIC MOD
