// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_VirtualMemory.h"


namespace
{
	static uint32_t GetAllocationGranularity(void)
	{
		::SYSTEM_INFO info = {};
		::GetSystemInfo(&info);
		return info.dwAllocationGranularity;
	}


	static uint32_t GetPageSize(void)
	{
		::SYSTEM_INFO info = {};
		::GetSystemInfo(&info);
		return info.dwPageSize;
	}
}


void* VirtualMemory::Allocate(size_t size)
{
	return ::VirtualAlloc(nullptr, size, MEM_COMMIT | MEM_RESERVE, PAGE_READWRITE);
}


void VirtualMemory::Free(void* ptr)
{
	::VirtualFree(ptr, 0u, MEM_RELEASE);
}


void* VirtualMemory::Allocate(Process::Handle handle, size_t size, PageType::Enum pageType)
{
	return ::VirtualAllocEx(+handle, nullptr, size, MEM_COMMIT | MEM_RESERVE, pageType);
}


void VirtualMemory::Free(Process::Handle handle, void* ptr)
{
	::VirtualFreeEx(+handle, ptr, 0u, MEM_RELEASE);
}


uint32_t VirtualMemory::GetAllocationGranularity(void)
{
	static const uint32_t allocationGranularity = ::GetAllocationGranularity();
	return allocationGranularity;
}


uint32_t VirtualMemory::GetPageSize(void)
{
	static const uint32_t pageSize = ::GetPageSize();
	return pageSize;
}


#endif // LC_VERSION