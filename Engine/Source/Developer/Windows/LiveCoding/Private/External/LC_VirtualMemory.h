// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ProcessTypes.h"
#include "LC_VirtualMemoryTypes.h"


namespace VirtualMemory
{
	// Allocates virtual memory.
	void* Allocate(size_t size);

	// Frees virtual memory.
	void Free(void* ptr);

	// Allocates virtual memory in the given process.
	void* Allocate(Process::Handle handle, size_t size, PageType::Enum pageType);

	// Frees virtual memory in the given process.
	void Free(Process::Handle handle, void* ptr);


	// Returns the allocation granularity of the virtual memory system.
	uint32_t GetAllocationGranularity(void);

	// Returns the page size used by the virtual memory system.
	uint32_t GetPageSize(void);
}


#endif // LC_VERSION