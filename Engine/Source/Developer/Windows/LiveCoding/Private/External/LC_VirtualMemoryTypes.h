// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

namespace VirtualMemory
{
	struct PageType
	{
		enum Enum
		{
			READ_WRITE = PAGE_READWRITE,
			EXECUTE_READ_WRITE = PAGE_EXECUTE_READWRITE
		};
	};
}


#endif // LC_VERSION