// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

// BEGIN EPIC MOD

#if LC_VERSION == 1

//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_WindowsInternalFunctions.h"


WindowsInternals::Function<decltype(NtSuspendProcess)> WindowsInternals::NtSuspendProcess("ntdll.dll", "NtSuspendProcess");
WindowsInternals::Function<decltype(NtResumeProcess)> WindowsInternals::NtResumeProcess("ntdll.dll", "NtResumeProcess");
WindowsInternals::Function<decltype(NtReadVirtualMemory)> WindowsInternals::NtReadVirtualMemory("ntdll.dll", "NtReadVirtualMemory");
WindowsInternals::Function<decltype(NtWriteVirtualMemory)> WindowsInternals::NtWriteVirtualMemory("ntdll.dll", "NtWriteVirtualMemory");
WindowsInternals::Function<decltype(NtQuerySystemInformation)> WindowsInternals::NtQuerySystemInformation("ntdll.dll", "NtQuerySystemInformation");
WindowsInternals::Function<decltype(NtQueryInformationProcess)> WindowsInternals::NtQueryInformationProcess("ntdll.dll", "NtQueryInformationProcess");
WindowsInternals::Function<decltype(NtContinue)> WindowsInternals::NtContinue("ntdll.dll", "NtContinue");


#endif // LC_VERSION