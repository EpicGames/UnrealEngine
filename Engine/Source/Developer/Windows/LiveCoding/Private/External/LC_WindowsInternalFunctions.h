// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

#include "LC_WindowsInternals.h"
// BEGIN EPIC MODS
#include "LC_Logging.h"
// END EPIC MODS

namespace WindowsInternals
{
	// Helper class that allows us to call any function in any Windows DLL, as long as it is exported and we know its signature.
	// Base template.
	template <typename T>
	class Function {};

	// Partial specialization for matching any function signature.
	template <typename R, typename... Args>
	class Function<R (Args...)>
	{
		typedef R (NTAPI *PtrToFunction)(Args...);

	public:
		inline Function(const char* moduleName, const char* functionName);

		inline R operator()(Args... args) const;

		// BEGIN EPIC MODS
		inline R ExecNoResultCheck(Args... args) const;
		// END EPIC MODS

		// Helper for letting us check the result for arbitrary return types.
		// Base template.
		template <typename T>
		inline void CheckResult(T) const {}

		// Explicit specialization for NTSTATUS return values.
		template <>
		inline void CheckResult(NTSTATUS result) const
		{
			if (!NT_SUCCESS(result))
			{
				LC_ERROR_USER("Call to function %s in module %s failed. Error: 0x%X", m_functionName, m_moduleName, result);
			}
		}

		// BEGIN EPIC MODS
	private:
		// END EPIC MODS

		const char* m_moduleName;
		const char* m_functionName;
		PtrToFunction m_function;
	};
}


template <typename R, typename... Args>
inline WindowsInternals::Function<R (Args...)>::Function(const char* moduleName, const char* functionName)
	: m_moduleName(moduleName)
	, m_functionName(functionName)
	, m_function(nullptr)
{
	HMODULE module = ::GetModuleHandleA(moduleName);
	if (!module)
	{
		LC_ERROR_USER("Cannot get handle for module %s", moduleName);
		return;
	}

	m_function = reinterpret_cast<PtrToFunction>(reinterpret_cast<uintptr_t>(::GetProcAddress(module, functionName)));
	if (!m_function)
	{
		LC_ERROR_USER("Cannot get address of function %s in module %s", functionName, moduleName);
	}
}


template <typename R, typename... Args>
inline R WindowsInternals::Function<R (Args...)>::operator()(Args... args) const
{
	const R result = m_function(args...);
	CheckResult(result);

	return result;
}

// BEGIN EPIC MODS
template <typename R, typename... Args>
inline R WindowsInternals::Function<R(Args...)>::ExecNoResultCheck(Args... args) const
{
	const R result = m_function(args...);

	return result;
}
// BEGIN EPIC MODS

// These are undocumented functions found in ntdll.dll.
// We don't call them directly, but use them for "extracting" their signature using decltype.
extern "C" WindowsInternals::NTSTATUS NtSuspendProcess(HANDLE ProcessHandle);
extern "C" WindowsInternals::NTSTATUS NtResumeProcess(HANDLE ProcessHandle);
extern "C" WindowsInternals::NTSTATUS NtReadVirtualMemory(HANDLE ProcessHandle, PVOID BaseAddress, PVOID Buffer, ULONG NumberOfBytesToRead, PULONG NumberOfBytesRead);
extern "C" WindowsInternals::NTSTATUS NtWriteVirtualMemory(HANDLE ProcessHandle, PVOID BaseAddress, PVOID Buffer, ULONG NumberOfBytesToWrite, PULONG NumberOfBytesWritten);
extern "C" WindowsInternals::NTSTATUS NtQuerySystemInformation(WindowsInternals::NT_SYSTEM_INFORMATION_CLASS SystemInformationClass, PVOID SystemInformation, ULONG SystemInformationLength, PULONG ReturnLength);
extern "C" WindowsInternals::NTSTATUS NtQueryInformationProcess(HANDLE ProcessHandle, WindowsInternals::NT_PROCESS_INFORMATION_CLASS ProcessInformationClass, PVOID ProcessInformation, ULONG ProcessInformationLength, PULONG ReturnLength);
extern "C" WindowsInternals::NTSTATUS NtContinue(CONTEXT* ThreadContext, BOOLEAN RaiseAlert);


// Cache important undocumented functions.
namespace WindowsInternals
{
	extern Function<decltype(NtSuspendProcess)> NtSuspendProcess;
	extern Function<decltype(NtResumeProcess)> NtResumeProcess;
	extern Function<decltype(NtReadVirtualMemory)> NtReadVirtualMemory;
	extern Function<decltype(NtWriteVirtualMemory)> NtWriteVirtualMemory;
	extern Function<decltype(NtQuerySystemInformation)> NtQuerySystemInformation;
	extern Function<decltype(NtQueryInformationProcess)> NtQueryInformationProcess;
	extern Function<decltype(NtContinue)> NtContinue;
}


#endif // LC_VERSION