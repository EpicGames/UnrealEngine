// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

// https://docs.microsoft.com/en-us/openspecs/windows_protocols/ms-erref/87fba13e-bf06-450e-83b1-9241dc81e781
// found in <ntstatus.h>
#define STATUS_INFO_LENGTH_MISMATCH		((WindowsInternals::NTSTATUS)0xC0000004L)

// found in <winternl.h>
#define NT_SUCCESS(Status)				((WindowsInternals::NTSTATUS)(Status) >= 0)


namespace WindowsInternals
{
	// most of these types are defined in <winternl.h>, but we cannot include that header because it lacks
	// a few undocumented values/members that we need for our purposes.
	// note that our definitions aren't complete either, we only define what we need.

	// https://docs.microsoft.com/en-us/openspecs/windows_protocols/ms-erref/87fba13e-bf06-450e-83b1-9241dc81e781
	typedef LONG NTSTATUS;

	// https://www.geoffchappell.com/studies/windows/km/ntoskrnl/api/ex/sysinfo/class.htm
	// found in <winternl.h>
	enum NT_SYSTEM_INFORMATION_CLASS
	{
		SystemProcessInformation = 5
	};

	// https://docs.microsoft.com/en-us/openspecs/windows_protocols/ms-tsts/a11e7129-685b-4535-8d37-21d4596ac057
	// found in <wdm.h> in WDK
	struct NT_CLIENT_ID
	{
		HANDLE UniqueProcess;
		HANDLE UniqueThread;
	};

	// https://docs.microsoft.com/en-us/openspecs/windows_protocols/ms-tsts/e82d73e4-cedb-4077-9099-d58f3459722f
	struct NT_SYSTEM_THREAD_INFORMATION
	{
		LARGE_INTEGER KernelTime;
		LARGE_INTEGER UserTime;
		LARGE_INTEGER CreateTime;
		ULONG WaitTime;
		PVOID StartAddress;
		NT_CLIENT_ID ClientId;
		LONG Priority;
		LONG BasePriority;
		ULONG ContextSwitches;
		ULONG ThreadState;
		ULONG WaitReason;
	};

	// https://docs.microsoft.com/en-us/openspecs/windows_protocols/ms-tsts/c90753f2-f9f9-490d-846d-6bdd41eae7f8
	// found in <winternl.h>
	struct NT_UNICODE_STRING
	{
		USHORT Length;
		USHORT MaximumLength;
		PWSTR Buffer;
	};

	// BEGIN EPIC MOD
	struct NT_UNICODE_STRING32
	{
		USHORT Length;
		USHORT MaximumLength;
		ULONG Buffer;
	};

	struct NT_ANSI_STRING
	{
		USHORT Length;
		USHORT MaximumLength;
		PSTR Buffer;
	};

	struct NT_ANSI_STRING32
	{
		USHORT Length;
		USHORT MaximumLength;
		ULONG Buffer;
	};
	// END EPIC MOD

	// https://www.geoffchappell.com/studies/windows/km/ntoskrnl/api/ex/sysinfo/process.htm
	// found in <winternl.h>
	struct NT_SYSTEM_PROCESS_INFORMATION
	{
		ULONG NextEntryOffset;
		ULONG NumberOfThreads;
		LARGE_INTEGER WorkingSetPrivateSize;
		ULONG HardFaultCount;
		ULONG NumberOfThreadsHighWatermark;
		ULONGLONG CycleTime;
		LARGE_INTEGER CreateTime;
		LARGE_INTEGER UserTime;
		LARGE_INTEGER KernelTime;
		NT_UNICODE_STRING ImageName;
		LONG BasePriority;
		PVOID UniqueProcessId;
		PVOID InheritedFromUniqueProcessId;
		ULONG HandleCount;
		ULONG SessionId;
		ULONG_PTR UniqueProcessKey;
		ULONG_PTR PeakVirtualSize;
		ULONG_PTR VirtualSize;
		ULONG PageFaultCount;
		ULONG_PTR PeakWorkingSetSize;
		ULONG_PTR WorkingSetSize;
		ULONG_PTR QuotaPeakPagedPoolUsage;
		ULONG_PTR QuotaPagedPoolUsage;
		ULONG_PTR QuotaPeakNonPagedPoolUsage;
		ULONG_PTR QuotaNonPagedPoolUsage;
		ULONG_PTR PagefileUsage;
		ULONG_PTR PeakPagefileUsage;
		ULONG_PTR PrivatePageCount;
		LARGE_INTEGER ReadOperationCount;
		LARGE_INTEGER WriteOperationCount;
		LARGE_INTEGER OtherOperationCount;
		LARGE_INTEGER ReadTransferCount;
		LARGE_INTEGER WriteTransferCount;
		LARGE_INTEGER OtherTransferCount;
		NT_SYSTEM_THREAD_INFORMATION Threads[1];	// variable size data
	};

	// https://docs.microsoft.com/en-us/windows/desktop/api/winternl/nf-winternl-ntqueryinformationprocess
	enum NT_PROCESS_INFORMATION_CLASS
	{
		ProcessBasicInformation = 0,
		ProcessWow64Information = 26
	};

	// BEGIN EPIC MOD
	struct RLT_DRIVE_LETTER_CURDIR
	{
		WORD Flags;
		WORD Length;
		ULONG TimeStemp;
		NT_ANSI_STRING DosPath;
	};

	struct RLT_DRIVE_LETTER_CURDIR32
	{
		WORD Flags;
		WORD Length;
		ULONG TimeStemp;
		NT_ANSI_STRING32 DosPath;
	};
	// END EPIC MOD

	// https://docs.microsoft.com/en-us/windows/desktop/api/winternl/ns-winternl-_rtl_user_process_parameters
	// https://www.nirsoft.net/kernel_struct/vista/RTL_USER_PROCESS_PARAMETERS.html
	// found in <winternl.h>
	struct RTL_USER_PROCESS_PARAMETERS
	{
		BYTE Reserved1[16];
		PVOID Reserved2[10];
		NT_UNICODE_STRING ImagePathName;
		NT_UNICODE_STRING CommandLine;
		PWSTR Environment;
		// EPIC BEGIN MOD
		ULONG Reserved3[9];
		NT_UNICODE_STRING Reserved4[4];
		RLT_DRIVE_LETTER_CURDIR CurrentDirectories[32];
		ULONG EnvironmentSize;
		// EPIC END MOD
	};

	// similar to RTL_USER_PROCESS_PARAMETERS, altered to behave as a struct containing 32-bit pointers in a 64-bit environment
	struct RTL_USER_PROCESS_PARAMETERS32
	{
		char Reserved[72];
		ULONG Environment;
		// EPIC BEGIN MOD
		ULONG Reserved3[9];
		NT_UNICODE_STRING32 Reserved4[4];
		RLT_DRIVE_LETTER_CURDIR32 CurrentDirectories[32];
		ULONG EnvironmentSize;
		// EPIC END MOD
	};

	// https://www.geoffchappell.com/studies/windows/win32/ntdll/structs/ldr_data_table_entry.htm
	// found in <winternl.h>
	struct NT_LDR_DATA_TABLE_ENTRY
	{
		LIST_ENTRY InLoadOrderLinks;
		LIST_ENTRY InMemoryOrderLinks;
		LIST_ENTRY InInitializationOrderLinks;
		PVOID DllBase;
		PVOID EntryPoint;
		ULONG SizeOfImage;
		NT_UNICODE_STRING FullDllName;
		NT_UNICODE_STRING BaseDllName;
		ULONG Flags;
		USHORT LoadCount;
		USHORT ObsoleteLoadCount;
		USHORT TlsIndex;
		LIST_ENTRY HashLinks;
	};

	// https://docs.microsoft.com/en-us/windows/desktop/api/winternl/ns-winternl-_peb_ldr_data
	// https://www.geoffchappell.com/studies/windows/win32/ntdll/structs/peb_ldr_data.htm
	struct NT_PEB_LDR_DATA
	{
		ULONG Length;
		BOOLEAN Initialized;
		PVOID SsHandle;
		LIST_ENTRY InLoadOrderModuleList;
		LIST_ENTRY InMemoryOrderModuleList;
		LIST_ENTRY InInitializationOrderModuleList;
		PVOID EntryInProgress;
		BOOLEAN ShutdownInProgress;
		HANDLE ShutdownThreadId;
	};

	// found in <winternl.h>
	typedef VOID (NTAPI *NT_PS_POST_PROCESS_INIT_ROUTINE)(VOID);

	// https://docs.microsoft.com/en-us/windows/desktop/api/winternl/ns-winternl-_peb
	struct NT_PEB
	{
		BYTE Reserved1[2];
		BYTE BeingDebugged;
		BYTE Reserved2[1];
		PVOID Reserved3[2];
		NT_PEB_LDR_DATA* Ldr;
		RTL_USER_PROCESS_PARAMETERS* ProcessParameters;
		PVOID Reserved4[3];
		PVOID AtlThunkSListPtr;
		PVOID Reserved5;
		ULONG Reserved6;
		PVOID Reserved7;
		ULONG Reserved8;
		ULONG AtlThunkSListPtr32;
		PVOID Reserved9[45];
		BYTE Reserved10[96];
		NT_PS_POST_PROCESS_INIT_ROUTINE* PostProcessInitRoutine;
		BYTE Reserved11[128];
		PVOID Reserved12[1];
		ULONG SessionId;
	};

	// similar to NT_PEB, altered to behave as a struct containing 32-bit pointers in a 64-bit environment
	struct NT_PEB32
	{
		char Reserved[16];
		ULONG ProcessParameters32;
	};

	// https://docs.microsoft.com/en-us/windows/desktop/api/winternl/nf-winternl-ntqueryinformationprocess
	struct NT_PROCESS_BASIC_INFORMATION
	{
		PVOID Reserved1;
		NT_PEB* PebBaseAddress;
		PVOID Reserved2[2];
		ULONG_PTR UniqueProcessId;
		PVOID Reserved3;
	};
}

#endif // LC_VERSION