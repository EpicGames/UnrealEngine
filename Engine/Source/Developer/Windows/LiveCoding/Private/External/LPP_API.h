// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

/******************************************************************************/
/* HOOKS                                                                      */
/******************************************************************************/

// concatenates two preprocessor tokens, even when the tokens themselves are macros
#define LPP_CONCATENATE_HELPER_HELPER(_a, _b)		_a##_b
#define LPP_CONCATENATE_HELPER(_a, _b)				LPP_CONCATENATE_HELPER_HELPER(_a, _b)
#define LPP_CONCATENATE(_a, _b)						LPP_CONCATENATE_HELPER(_a, _b)

// generates a unique identifier inside a translation unit
#define LPP_IDENTIFIER(_identifier)					LPP_CONCATENATE(_identifier, __LINE__)

// custom section names for hooks
#define LPP_PREPATCH_SECTION						".lpp_prepatch_hooks"
#define LPP_POSTPATCH_SECTION						".lpp_postpatch_hooks"
#define LPP_COMPILE_START_SECTION					".lpp_compile_start_hooks"
#define LPP_COMPILE_SUCCESS_SECTION					".lpp_compile_success_hooks"
#define LPP_COMPILE_ERROR_SECTION					".lpp_compile_error_hooks"
#define LPP_COMPILE_ERROR_MESSAGE_SECTION			".lpp_compile_error_message_hooks"
// BEGIN EPIC MOD - Add the ability for pre and post compile notifications
#define LPP_PRECOMPILE_SECTION						".lpp_precompile_hooks"
#define LPP_POSTCOMPILE_SECTION						".lpp_postcompile_hooks"
// END EPIC MOD

// register a pre-patch hook in a custom section
#define LPP_PREPATCH_HOOK(_function)																																													\
	__pragma(section(LPP_PREPATCH_SECTION, read)) __declspec(allocate(LPP_PREPATCH_SECTION)) extern void (*LPP_IDENTIFIER(lpp_prepatch_hook_function))(void);															\
	__pragma(section(LPP_PREPATCH_SECTION, read)) __declspec(allocate(LPP_PREPATCH_SECTION)) void (*LPP_IDENTIFIER(lpp_prepatch_hook_function))(void) = &_function

// register a post-patch hook in a custom section
#define LPP_POSTPATCH_HOOK(_function)																																													\
	__pragma(section(LPP_POSTPATCH_SECTION, read)) __declspec(allocate(LPP_POSTPATCH_SECTION)) extern void (*LPP_IDENTIFIER(lpp_postpatch_hook_function))(void);														\
	__pragma(section(LPP_POSTPATCH_SECTION, read)) __declspec(allocate(LPP_POSTPATCH_SECTION)) void (*LPP_IDENTIFIER(lpp_postpatch_hook_function))(void) = &_function

// register a compile start hook in a custom section
#define LPP_COMPILE_START_HOOK(_function)																																												\
	__pragma(section(LPP_COMPILE_START_SECTION, read)) __declspec(allocate(LPP_COMPILE_START_SECTION)) extern void (*LPP_IDENTIFIER(lpp_compile_start_hook_function))(void);											\
	__pragma(section(LPP_COMPILE_START_SECTION, read)) __declspec(allocate(LPP_COMPILE_START_SECTION)) void (*LPP_IDENTIFIER(lpp_compile_start_hook_function))(void) = &_function

// register a compile success hook in a custom section
#define LPP_COMPILE_SUCCESS_HOOK(_function)																																												\
	__pragma(section(LPP_COMPILE_SUCCESS_SECTION, read)) __declspec(allocate(LPP_COMPILE_SUCCESS_SECTION)) extern void (*LPP_IDENTIFIER(lpp_compile_success_hook_function))(void);										\
	__pragma(section(LPP_COMPILE_SUCCESS_SECTION, read)) __declspec(allocate(LPP_COMPILE_SUCCESS_SECTION)) void (*LPP_IDENTIFIER(lpp_compile_success_hook_function))(void) = &_function

// register a compile error hook in a custom section
#define LPP_COMPILE_ERROR_HOOK(_function)																																												\
	__pragma(section(LPP_COMPILE_ERROR_SECTION, read)) __declspec(allocate(LPP_COMPILE_ERROR_SECTION)) extern void (*LPP_IDENTIFIER(lpp_compile_error_hook_function))(void);											\
	__pragma(section(LPP_COMPILE_ERROR_SECTION, read)) __declspec(allocate(LPP_COMPILE_ERROR_SECTION)) void (*LPP_IDENTIFIER(lpp_compile_error_hook_function))(void) = &_function

// register a compile error message hook in a custom section
#define LPP_COMPILE_ERROR_MESSAGE_HOOK(_function)																																										\
	__pragma(section(LPP_COMPILE_ERROR_MESSAGE_SECTION, read)) __declspec(allocate(LPP_COMPILE_ERROR_MESSAGE_SECTION)) extern void (*LPP_IDENTIFIER(lpp_compile_error_message_hook_function))(const wchar_t*);			\
	__pragma(section(LPP_COMPILE_ERROR_MESSAGE_SECTION, read)) __declspec(allocate(LPP_COMPILE_ERROR_MESSAGE_SECTION)) void (*LPP_IDENTIFIER(lpp_compile_error_message_hook_function))(const wchar_t*) = &_function


// BEGIN EPIC MOD - Add the ability for pre and post compile notifications
// register a pre-compile hook in a custom section
#define LPP_PRECOMPILE_HOOK(_function)																												\
	__pragma(section(LPP_PRECOMPILE_SECTION, read))																									\
	__declspec(allocate(LPP_PRECOMPILE_SECTION)) extern void (*LPP_IDENTIFIER(lpp_precompile_hook_function))(void) = &_function

// register a post-compile hook in a custom section
#define LPP_POSTCOMPILE_HOOK(_function)																												\
	__pragma(section(LPP_POSTCOMPILE_SECTION, read))																								\
	__declspec(allocate(LPP_POSTCOMPILE_SECTION)) extern void (*LPP_IDENTIFIER(lpp_postcompile_hook_function))(void) = &_function
// END EPIC MOD

/******************************************************************************/
/* HELPERS                                                                    */
/******************************************************************************/

#ifdef __cplusplus
#	define LPP_NS_BEGIN			namespace lpp {
#	define LPP_NS_END			}
#	define LPP_API				inline
#	define LPP_CAST(_type)		reinterpret_cast<_type>
#	define LPP_NULL				nullptr
#else
#	define LPP_NS_BEGIN
#	define LPP_NS_END
#	define LPP_API				static inline
#	define LPP_CAST(_type)		(_type)
#	define LPP_NULL				NULL
#endif

// helper macros to call a function in a DLL with an arbitrary signature without a compiler warning
#define LPP_CALL1(_module, _functionName, _returnType, _args1)								(LPP_CAST(_returnType (__cdecl*)(_args1))(LPP_CAST(uintptr_t)(GetProcAddress(_module, _functionName))))
#define LPP_CALL2(_module, _functionName, _returnType, _args1, _args2)						(LPP_CAST(_returnType (__cdecl*)(_args1, _args2))(LPP_CAST(uintptr_t)(GetProcAddress(_module, _functionName))))
#define LPP_CALL3(_module, _functionName, _returnType, _args1, _args2, _args3)				(LPP_CAST(_returnType (__cdecl*)(_args1, _args2, _args3))(LPP_CAST(uintptr_t)(GetProcAddress(_module, _functionName))))
#define LPP_CALL4(_module, _functionName, _returnType, _args1, _args2, _args3, _args4)		(LPP_CAST(_returnType (__cdecl*)(_args1, _args2, _args3, _args4))(LPP_CAST(uintptr_t)(GetProcAddress(_module, _functionName))))

// linker pseudo-variable representing the DOS header of the module we're being compiled into:
// Raymond Chen: https://blogs.msdn.microsoft.com/oldnewthing/20041025-00/?p=37483
// BEGIN EPIC MOD
//EXTERN_C IMAGE_DOS_HEADER __ImageBase;
// END EPIC MOD

/******************************************************************************/
/* API                                                                        */
/******************************************************************************/

// version string
#define LPP_VERSION "1.6.10"

// macros to temporarily enable/disable optimizations
#ifdef __clang__
#	define LPP_ENABLE_OPTIMIZATIONS		_Pragma("clang optimize on")
#	define LPP_DISABLE_OPTIMIZATIONS	_Pragma("clang optimize off")
#else
#	define LPP_ENABLE_OPTIMIZATIONS		__pragma(optimize("", on))
#	define LPP_DISABLE_OPTIMIZATIONS	__pragma(optimize("", off))
#endif


LPP_NS_BEGIN

// BEGIN EPIC MOD
#if 0
// starts up Live++. must be called after loading the Live++ DLL, before registering a process group.
LPP_API void lppStartup(HMODULE livePP)
{
	return LPP_CALL1(livePP, "LppStartup", void, void)();
}


// shuts down Live++. must be called before unloading the Live++ DLL.
LPP_API void lppShutdown(HMODULE livePP)
{
	return LPP_CALL1(livePP, "LppShutdown", void, void)();
}


// retrieves the version number of the DLL.
LPP_API const char* lppGetVersion(HMODULE livePP)
{
	return LPP_CALL1(livePP, "LppGetVersion", const char*, void)();
}


// checks whether the API and DLL versions match.
// returns 1 on success, 0 on failure.
LPP_API int lppCheckVersion(HMODULE livePP)
{
	return LPP_CALL1(livePP, "LppCheckVersion", int, const char*)(LPP_VERSION);
}


// registers a Live++ process group.
LPP_API void lppRegisterProcessGroup(HMODULE livePP, const char* groupName)
{
	LPP_CALL1(livePP, "LppRegisterProcessGroup", void, const char*)(groupName);
}


// calls the synchronization point.
LPP_API void lppSyncPoint(HMODULE livePP)
{
	LPP_CALL1(livePP, "LppSyncPoint", void, void)();
}


// waits until the operation identified by the given token is finished.
LPP_API void lppWaitForToken(HMODULE livePP, void* token)
{
	LPP_CALL1(livePP, "LppWaitForToken", void, void*)(token);
}


// triggers a recompile.
LPP_API void lppTriggerRecompile(HMODULE livePP)
{
	LPP_CALL1(livePP, "LppTriggerRecompile", void, void)();
}


// logs a message in the Live+ UI.
LPP_API void lppLogMessage(HMODULE livePP, const wchar_t* message)
{
	LPP_CALL1(livePP, "LppLogMessage", void, const wchar_t*)(message);
}


// builds a patch using the given object files.
LPP_API void lppBuildPatch(HMODULE livePP, const wchar_t* moduleNames[], const wchar_t* objPaths[], const wchar_t* amalgamatedObjPaths[], unsigned int count)
{
	LPP_CALL4(livePP, "LppBuildPatch", void, const wchar_t*[], const wchar_t*[], const wchar_t*[], unsigned int)(moduleNames, objPaths, amalgamatedObjPaths, count);
}


// installs Live++'s exception handler.
LPP_API void lppInstallExceptionHandler(HMODULE livePP)
{
	LPP_CALL1(livePP, "LppInstallExceptionHandler", void, void)();
}


// makes Live++ listen to changed .obj files compiled by an external build system.
LPP_API void lppUseExternalBuildSystem(HMODULE livePP)
{
	LPP_CALL1(livePP, "LppUseExternalBuildSystem", void, void)();
}


// triggers a restart.
LPP_API void lppTriggerRestart(HMODULE livePP)
{
	LPP_CALL1(livePP, "LppTriggerRestart", int, void)();
}


// returns 1 if the calling process wants to be restarted.
LPP_API int lppWantsRestart(HMODULE livePP)
{
	return LPP_CALL1(livePP, "LppWantsRestart", int, void)();
}
#endif
// END EPIC MOD


enum RestartBehaviour
{
	// BEGIN EPIC MODS - Use UE codepath for termination to ensure logs are flushed and session analytics are sent
	LPP_RESTART_BEHAVIOR_REQUEST_EXIT,				// FPlatforMisc::RequestExit(true)
	// END EPIC MODS
	LPP_RESTART_BEHAVIOUR_DEFAULT_EXIT,				// ExitProcess()
	LPP_RESTART_BEHAVIOUR_EXIT_WITH_FLUSH,			// exit()
	LPP_RESTART_BEHAVIOUR_EXIT,						// _Exit()
	LPP_RESTART_BEHAVIOUR_INSTANT_TERMINATION		// TerminateProcess
};

// BEGIN EPIC MOD
#if 0
// restarts the calling process, does not return.
LPP_API void lppRestart(HMODULE livePP, enum RestartBehaviour behaviour, unsigned int exitCode)
{
	LPP_CALL2(livePP, "LppRestart", void, enum RestartBehaviour, unsigned int)(behaviour, exitCode);
}


// asynchronously enables Live++ for just the given .exe or .dll, but not its import modules.
// returns a token that can be waited upon using lppWaitForToken().
LPP_API void* lppEnableModuleAsync(HMODULE livePP, const wchar_t* nameOfExeOrDll)
{
	return LPP_CALL1(livePP, "LppEnableModule", void*, const wchar_t*)(nameOfExeOrDll);
}


// asynchronously enables Live++ for all given .exes and .dlls, but not its import modules.
// returns a token that can be waited upon using lppWaitForToken().
LPP_API void* lppEnableModulesAsync(HMODULE livePP, const wchar_t* namesOfExeOrDll[], unsigned int count)
{
	return LPP_CALL2(livePP, "LppEnableModules", void*, const wchar_t*[], unsigned int)(namesOfExeOrDll, count);
}


// asynchronously enables Live++ for the given .exe or .dll and all its import modules.
// returns a token that can be waited upon using lppWaitForToken().
LPP_API void* lppEnableAllModulesAsync(HMODULE livePP, const wchar_t* nameOfExeOrDll)
{
	return LPP_CALL1(livePP, "LppEnableAllModules", void*, const wchar_t*)(nameOfExeOrDll);
}


// asynchronously enables Live++ for just the .exe or .dll this function is being called from, but not its import modules.
// returns a token that can be waited upon using lppWaitForToken().
LPP_API void* lppEnableCallingModuleAsync(HMODULE livePP)
{
	wchar_t path[MAX_PATH] = { 0 };
	GetModuleFileNameW(LPP_CAST(HMODULE)(&__ImageBase), path, MAX_PATH);
	return lppEnableModuleAsync(livePP, path);
}


// asynchronously enables Live++ for the .exe or .dll this function is being called from and all its import modules.
// returns a token that can be waited upon using lppWaitForToken().
LPP_API void* lppEnableAllCallingModulesAsync(HMODULE livePP)
{
	wchar_t path[MAX_PATH] = { 0 };
	GetModuleFileNameW(LPP_CAST(HMODULE)(&__ImageBase), path, MAX_PATH);
	return lppEnableAllModulesAsync(livePP, path);
}


// synchronously enables Live++ for just the given .exe or .dll, but not its import modules.
LPP_API void lppEnableModuleSync(HMODULE livePP, const wchar_t* nameOfExeOrDll)
{
	void* token = lppEnableModuleAsync(livePP, nameOfExeOrDll);
	lppWaitForToken(livePP, token);
}


// synchronously enables Live++ for all given .exes and .dlls, but not its import modules.
LPP_API void lppEnableModulesSync(HMODULE livePP, const wchar_t* namesOfExeOrDll[], unsigned int count)
{
	void* token = lppEnableModulesAsync(livePP, namesOfExeOrDll, count);
	lppWaitForToken(livePP, token);
}


// synchronously enables Live++ for the given .exe or .dll and all its import modules.
LPP_API void lppEnableAllModulesSync(HMODULE livePP, const wchar_t* nameOfExeOrDll)
{
	void* token = lppEnableAllModulesAsync(livePP, nameOfExeOrDll);
	lppWaitForToken(livePP, token);
}


// synchronously enables Live++ for just the .exe or .dll this function is being called from, but not its import modules.
LPP_API void lppEnableCallingModuleSync(HMODULE livePP)
{
	void* token = lppEnableCallingModuleAsync(livePP);
	lppWaitForToken(livePP, token);
}


// synchronously enables Live++ for the .exe or .dll this function is being called from and all its import modules.
LPP_API void lppEnableAllCallingModulesSync(HMODULE livePP)
{
	void* token = lppEnableAllCallingModulesAsync(livePP);
	lppWaitForToken(livePP, token);
}


// asynchronously disables Live++ for just the given .exe or .dll, but not its import modules.
// returns a token that can be waited upon using lppWaitForToken().
LPP_API void* lppDisableModuleAsync(HMODULE livePP, const wchar_t* nameOfExeOrDll)
{
	return LPP_CALL1(livePP, "LppDisableModule", void*, const wchar_t*)(nameOfExeOrDll);
}


// asynchronously disables Live++ for all given .exes and .dlls, but not its import modules.
// returns a token that can be waited upon using lppWaitForToken().
LPP_API void* lppDisableModulesAsync(HMODULE livePP, const wchar_t* namesOfExeOrDll[], unsigned int count)
{
	return LPP_CALL2(livePP, "LppDisableModules", void*, const wchar_t*[], unsigned int)(namesOfExeOrDll, count);
}


// asynchronously disables Live++ for the given .exe or .dll and all its import modules.
// returns a token that can be waited upon using lppWaitForToken().
LPP_API void* lppDisableAllModulesAsync(HMODULE livePP, const wchar_t* nameOfExeOrDll)
{
	return LPP_CALL1(livePP, "LppDisableAllModules", void*, const wchar_t*)(nameOfExeOrDll);
}


// asynchronously disables Live++ for just the .exe or .dll this function is being called from, but not its import modules.
// returns a token that can be waited upon using lppWaitForToken().
LPP_API void* lppDisableCallingModuleAsync(HMODULE livePP)
{
	wchar_t path[MAX_PATH] = { 0 };
	GetModuleFileNameW(LPP_CAST(HMODULE)(&__ImageBase), path, MAX_PATH);
	return lppDisableModuleAsync(livePP, path);
}


// asynchronously disables Live++ for the .exe or .dll this function is being called from and all its import modules.
// returns a token that can be waited upon using lppWaitForToken().
LPP_API void* lppDisableAllCallingModulesAsync(HMODULE livePP)
{
	wchar_t path[MAX_PATH] = { 0 };
	GetModuleFileNameW(LPP_CAST(HMODULE)(&__ImageBase), path, MAX_PATH);
	return lppDisableAllModulesAsync(livePP, path);
}


// synchronously disables Live++ for just the given .exe or .dll, but not its import modules.
LPP_API void lppDisableModuleSync(HMODULE livePP, const wchar_t* nameOfExeOrDll)
{
	void* token = lppDisableModuleAsync(livePP, nameOfExeOrDll);
	lppWaitForToken(livePP, token);
}


// synchronously disables Live++ for all given .exes and .dlls, but not its import modules.
LPP_API void lppDisableModulesSync(HMODULE livePP, const wchar_t* namesOfExeOrDll[], unsigned int count)
{
	void* token = lppDisableModulesAsync(livePP, namesOfExeOrDll, count);
	lppWaitForToken(livePP, token);
}


// synchronously disables Live++ for the given .exe or .dll and all its import modules.
LPP_API void lppDisableAllModulesSync(HMODULE livePP, const wchar_t* nameOfExeOrDll)
{
	void* token = lppDisableAllModulesAsync(livePP, nameOfExeOrDll);
	lppWaitForToken(livePP, token);
}


// synchronously disables Live++ for just the .exe or .dll this function is being called from, but not its import modules.
LPP_API void lppDisableCallingModuleSync(HMODULE livePP)
{
	void* token = lppDisableCallingModuleAsync(livePP);
	lppWaitForToken(livePP, token);
}


// synchronously disables Live++ for the .exe or .dll this function is being called from and all its import modules.
LPP_API void lppDisableAllCallingModulesSync(HMODULE livePP)
{
	void* token = lppDisableAllCallingModulesAsync(livePP);
	lppWaitForToken(livePP, token);
}


// sets any boolean API setting. check the documentation for a full list of available settings.
LPP_API void lppApplySettingBool(HMODULE livePP, const char* settingName, int value)
{
	LPP_CALL2(livePP, "LppApplySettingBool", void, const char*, int)(settingName, value);
}


// sets any integer API setting. check the documentation for a full list of available settings.
LPP_API void lppApplySettingInt(HMODULE livePP, const char* settingName, int value)
{
	LPP_CALL2(livePP, "LppApplySettingInt", void, const char*, int)(settingName, value);
}


// sets any string API setting. check the documentation for a full list of available settings.
LPP_API void lppApplySettingString(HMODULE livePP, const char* settingName, const wchar_t* value)
{
	LPP_CALL2(livePP, "LppApplySettingString", void, const char*, const wchar_t*)(settingName, value);
}


// loads the correct 32-bit/64-bit DLL (based on architecture), checks if the versions match, and registers a Live++ process group.
LPP_API HMODULE lppLoadAndRegister(const wchar_t* pathWithoutTrailingSlash, const char* groupName)
{
	wchar_t path[1024] = { 0 };

	// we deliberately do not use memcpy or strcpy here, as that could force more #includes in the user's code
	unsigned int index = 0u;
	while (*pathWithoutTrailingSlash != L'\0')
	{
		if (index >= 1023u)
		{
			// no space left in buffer for this character and a null terminator
			return LPP_NULL;
		}

		path[index++] = *pathWithoutTrailingSlash++;
	}

#ifdef _WIN64
	const wchar_t* dllName = L"/x64/LPP_x64.dll";
#else
	const wchar_t* dllName = L"/x86/LPP_x86.dll";
#endif

	while (*dllName != L'\0')
	{
		if (index >= 1023u)
		{
			// no space left in buffer for this character and a null terminator
			return LPP_NULL;
		}

		path[index++] = *dllName++;
	}

	path[index++] = L'\0';

	HMODULE livePP = LoadLibraryW(path);
	if (livePP)
	{
		if (!lppCheckVersion(livePP))
		{
			// version mismatch detected
			FreeLibrary(livePP);
			return LPP_NULL;
		}

		lppStartup(livePP);
		lppRegisterProcessGroup(livePP, groupName);
	}

	return livePP;
}
#endif
// END EPIC MOD

LPP_NS_END


#undef LPP_CALL1
#undef LPP_CALL2
#undef LPP_CALL3
#undef LPP_CALL4

#endif // LC_VERSION