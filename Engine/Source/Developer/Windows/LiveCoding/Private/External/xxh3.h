/*
 * xxHash - Extremely Fast Hash algorithm
 * Development source file for `xxh3`
 * Copyright (C) 2019-present, Yann Collet
 *
 * BSD 2-Clause License (https://www.opensource.org/licenses/bsd-license.php)
 *
 * Redistribution and use in source and binary forms, with or without
 * modification, are permitted provided that the following conditions are
 * met:
 *
 *    * Redistributions of source code must retain the above copyright
 *      notice, this list of conditions and the following disclaimer.
 *    * Redistributions in binary form must reproduce the above
 *      copyright notice, this list of conditions and the following disclaimer
 *      in the documentation and/or other materials provided with the
 *      distribution.
 *
 * THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS
 * "AS IS" AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT
 * LIMITED TO, THE IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR
 * A PARTICULAR PURPOSE ARE DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT
 * OWNER OR CONTRIBUTORS BE LIABLE FOR ANY DIRECT, INDIRECT, INCIDENTAL,
 * SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES (INCLUDING, BUT NOT
 * LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES; LOSS OF USE,
 * DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON ANY
 * THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
 * (INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE
 * OF THIS SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
 *
 * You can contact the author at:
 *   - xxHash homepage: https://www.xxhash.com
 *   - xxHash source repository: https://github.com/Cyan4973/xxHash
 */

/*
 * Note: This file is separated for development purposes.
 * It will be integrated into `xxhash.h` when development stage is completed.
 *
 * Credit: most of the work on vectorial and asm variants comes from @easyaspi314
 */

#ifndef XXH3_H_1397135465
#define XXH3_H_1397135465

/* ===   Dependencies   === */
#ifndef XXHASH_H_5627135585666179
/* special: when including `xxh3.h` directly, turn on XXH_INLINE_ALL */
#  undef XXH_INLINE_ALL   /* avoid redefinition */
#  define XXH_INLINE_ALL
#endif
#include "xxhash.h"


/* ===   Compiler specifics   === */

#if defined (__STDC_VERSION__) && __STDC_VERSION__ >= 199901L   /* >= C99 */
#  define XXH_RESTRICT   restrict
#else
/* Note: it might be useful to define __restrict or __restrict__ for some C++ compilers */
#  define XXH_RESTRICT   /* disable */
#endif

#if (defined(__GNUC__) && (__GNUC__ >= 3))  \
  || (defined(__INTEL_COMPILER) && (__INTEL_COMPILER >= 800)) \
  || defined(__clang__)
#    define XXH_likely(x) __builtin_expect(x, 1)
#    define XXH_unlikely(x) __builtin_expect(x, 0)
#else
#    define XXH_likely(x) (x)
#    define XXH_unlikely(x) (x)
#endif

#if defined(__GNUC__)
#  if defined(__AVX2__)
#    include <immintrin.h>
#  elif defined(__SSE2__)
#    include <emmintrin.h>
#  elif defined(__ARM_NEON__) || defined(__ARM_NEON)
#    define inline __inline__  /* clang bug */
#    include <arm_neon.h>
#    undef inline
#  endif
#elif defined(_MSC_VER)
#  include <intrin.h>
#endif

/*
 * One goal of XXH3 is to make it fast on both 32-bit and 64-bit, while
 * remaining a true 64-bit/128-bit hash function.
 *
 * This is done by prioritizing a subset of 64-bit operations that can be
 * emulated without too many steps on the average 32-bit machine.
 *
 * For example, these two lines seem similar, and run equally fast on 64-bit:
 *
 *   xxh_u64 x;
 *   x ^= (x >> 47); // good
 *   x ^= (x >> 13); // bad
 *
 * However, to a 32-bit machine, there is a major difference.
 *
 * x ^= (x >> 47) looks like this:
 *
 *   x.lo ^= (x.hi >> (47 - 32));
 *
 * while x ^= (x >> 13) looks like this:
 *
 *   // note: funnel shifts are not usually cheap.
 *   x.lo ^= (x.lo >> 13) | (x.hi << (32 - 13));
 *   x.hi ^= (x.hi >> 13);
 *
 * The first one is significantly faster than the second, simply because the
 * shift is larger than 32. This means:
 *  - All the bits we need are in the upper 32 bits, so we can ignore the lower
 *    32 bits in the shift.
 *  - The shift result will always fit in the lower 32 bits, and therefore,
 *    we can ignore the upper 32 bits in the xor.
 *
 * Thanks to this optimization, XXH3 only requires these features to be efficient:
 *
 *  - Usable unaligned access
 *  - A 32-bit or 64-bit ALU
 *      - If 32-bit, a decent ADC instruction
 *  - A 32 or 64-bit multiply with a 64-bit result
 *  - For the 128-bit variant, a decent byteswap helps short inputs.
 *
 * The first two are already required by XXH32, and almost all 32-bit and 64-bit
 * platforms which can run XXH32 can run XXH3 efficiently.
 *
 * Thumb-1, the classic 16-bit only subset of ARM's instruction set, is one
 * notable exception.
 *
 * First of all, Thumb-1 lacks support for the UMULL instruction which
 * performs the important long multiply. This means numerous __aeabi_lmul
 * calls.
 *
 * Second of all, the 8 functional registers are just not enough.
 * Setup for __aeabi_lmul, byteshift loads, pointers, and all arithmetic need
 * Lo registers, and this shuffling results in thousands more MOVs than A32.
 *
 * A32 and T32 don't have this limitation. They can access all 14 registers,
 * do a 32->64 multiply with UMULL, and the flexible operand allowing free
 * shifts is helpful, too.
 *
 * Therefore, we do a quick sanity check.
 *
 * If compiling Thumb-1 for a target which supports ARM instructions, we will
 * emit a warning, as it is not a "sane" platform to compile for.
 *
 * Usually, if this happens, it is because of an accident and you probably need
 * to specify -march, as you likely meant to compile for a newer architecture.
 */
#if defined(__thumb__) && !defined(__thumb2__) && defined(__ARM_ARCH_ISA_ARM)
#   warning "XXH3 is highly inefficient without ARM or Thumb-2."
#endif

/* ==========================================
 * Vectorization detection
 * ========================================== */
#define XXH_SCALAR 0 /* Portable scalar version */
#define XXH_SSE2   1 /* SSE2 for Pentium 4 and all x86_64 */
#define XXH_AVX2   2 /* AVX2 for Haswell and Bulldozer */
#define XXH_NEON   3 /* NEON for most ARMv7-A and all AArch64 */
#define XXH_VSX    4 /* VSX and ZVector for POWER8/z13 */

#ifndef XXH_VECTOR    /* can be defined on command line */
#  if defined(__AVX2__)
#    define XXH_VECTOR XXH_AVX2
#  elif defined(__SSE2__) || defined(_M_AMD64) || defined(_M_X64) || (defined(_M_IX86_FP) && (_M_IX86_FP == 2))
#    define XXH_VECTOR XXH_SSE2
#  elif defined(__GNUC__) /* msvc support maybe later */ \
  && (defined(__ARM_NEON__) || defined(__ARM_NEON)) \
  && (defined(__LITTLE_ENDIAN__) /* We only support little endian NEON */ \
    || (defined(__BYTE_ORDER__) && __BYTE_ORDER__ == __ORDER_LITTLE_ENDIAN__))
#    define XXH_VECTOR XXH_NEON
#  elif (defined(__PPC64__) && defined(__POWER8_VECTOR__)) \
     || (defined(__s390x__) && defined(__VEC__)) \
     && defined(__GNUC__) /* TODO: IBM XL */
#    define XXH_VECTOR XXH_VSX
#  else
#    define XXH_VECTOR XXH_SCALAR
#  endif
#endif

/*
 * Controls the alignment of the accumulator.
 * This is for compatibility with aligned vector loads, which are usually faster.
 */
#ifndef XXH_ACC_ALIGN
#  if XXH_VECTOR == XXH_SCALAR  /* scalar */
#     define XXH_ACC_ALIGN 8
#  elif XXH_VECTOR == XXH_SSE2  /* sse2 */
#     define XXH_ACC_ALIGN 16
#  elif XXH_VECTOR == XXH_AVX2  /* avx2 */
#     define XXH_ACC_ALIGN 32
#  elif XXH_VECTOR == XXH_NEON  /* neon */
#     define XXH_ACC_ALIGN 16
#  elif XXH_VECTOR == XXH_VSX   /* vsx */
#     define XXH_ACC_ALIGN 16
#  endif
#endif

/*
 * UGLY HACK:
 * GCC usually generates the best code with -O3 for xxHash.
 *
 * However, when targeting AVX2, it is overzealous in its unrolling resulting
 * in code roughly 3/4 the speed of Clang.
 *
 * There are other issues, such as GCC splitting _mm256_loadu_si256 into
 * _mm_loadu_si128 + _mm256_inserti128_si256. This is an optimization which
 * only applies to Sandy and Ivy Bridge... which don't even support AVX2.
 *
 * That is why when compiling the AVX2 version, it is recommended to use either
 *   -O2 -mavx2 -march=haswell
 * or
 *   -O2 -mavx2 -mno-avx256-split-unaligned-load
 * for decent performance, or to use Clang instead.
 *
 * Fortunately, we can control the first one with a pragma that forces GCC into
 * -O2, but the other one we can't control without "failed to inline always
 * inline function due to target mismatch" warnings.
 */
#if XXH_VECTOR == XXH_AVX2 /* AVX2 */ \
  && defined(__GNUC__) && !defined(__clang__) /* GCC, not Clang */ \
  && defined(__OPTIMIZE__) && !defined(__OPTIMIZE_SIZE__) /* respect -O0 and -Os */
#  pragma GCC push_options
#  pragma GCC optimize("-O2")
#endif


#if XXH_VECTOR == XXH_NEON
/*
 * NEON's setup for vmlal_u32 is a little more complicated than it is on
 * SSE2, AVX2, and VSX.
 *
 * While PMULUDQ and VMULEUW both perform a mask, VMLAL.U32 performs an upcast.
 *
 * To do the same operation, the 128-bit 'Q' register needs to be split into
 * two 64-bit 'D' registers, performing this operation::
 *
 *   [                a                 |                 b                ]
 *            |              '---------. .--------'                |
 *            |                         x                          |
 *            |              .---------' '--------.                |
 *   [ a & 0xFFFFFFFF | b & 0xFFFFFFFF ],[    a >> 32     |     b >> 32    ]
 *
 * Due to significant changes in aarch64, the fastest method for aarch64 is
 * completely different than the fastest method for ARMv7-A.
 *
 * ARMv7-A treats D registers as unions overlaying Q registers, so modifying
 * D11 will modify the high half of Q5. This is similar to how modifying AH
 * will only affect bits 8-15 of AX on x86.
 *
 * VZIP takes two registers, and puts even lanes in one register and odd lanes
 * in the other.
 *
 * On ARMv7-A, this strangely modifies both parameters in place instead of
 * taking the usual 3-operand form.
 *
 * Therefore, if we want to do this, we can simply use a D-form VZIP.32 on the
 * lower and upper halves of the Q register to end up with the high and low
 * halves where we want - all in one instruction.
 *
 *   vzip.32   d10, d11       @ d10 = { d10[0], d11[0] }; d11 = { d10[1], d11[1] }
 *
 * Unfortunately we need inline assembly for this: Instructions modifying two
 * registers at once is not possible in GCC or Clang's IR, and they have to
 * create a copy.
 *
 * aarch64 requires a different approach.
 *
 * In order to make it easier to write a decent compiler for aarch64, many
 * quirks were removed, such as conditional execution.
 *
 * NEON was also affected by this.
 *
 * aarch64 cannot access the high bits of a Q-form register, and writes to a
 * D-form register zero the high bits, similar to how writes to W-form scalar
 * registers (or DWORD registers on x86_64) work.
 *
 * The formerly free vget_high intrinsics now require a vext (with a few
 * exceptions)
 *
 * Additionally, VZIP was replaced by ZIP1 and ZIP2, which are the equivalent
 * of PUNPCKL* and PUNPCKH* in SSE, respectively, in order to only modify one
 * operand.
 *
 * The equivalent of the VZIP.32 on the lower and upper halves would be this
 * mess:
 *
 *   ext     v2.4s, v0.4s, v0.4s, #2 // v2 = { v0[2], v0[3], v0[0], v0[1] }
 *   zip1    v1.2s, v0.2s, v2.2s     // v1 = { v0[0], v2[0] }
 *   zip2    v0.2s, v0.2s, v1.2s     // v0 = { v0[1], v2[1] }
 *
 * Instead, we use a literal downcast, vmovn_u64 (XTN), and vshrn_n_u64 (SHRN):
 *
 *   shrn    v1.2s, v0.2d, #32  // v1 = (uint32x2_t)(v0 >> 32);
 *   xtn     v0.2s, v0.2d       // v0 = (uint32x2_t)(v0 & 0xFFFFFFFF);
 *
 * This is available on ARMv7-A, but is less efficient than a single VZIP.32.
 */

/*
 * Function-like macro:
 * void XXH_SPLIT_IN_PLACE(uint64x2_t &in, uint32x2_t &outLo, uint32x2_t &outHi)
 * {
 *     outLo = (uint32x2_t)(in & 0xFFFFFFFF);
 *     outHi = (uint32x2_t)(in >> 32);
 *     in = UNDEFINED;
 * }
 */
# if !defined(XXH_NO_VZIP_HACK) /* define to disable */ \
   && defined(__GNUC__) \
   && !defined(__aarch64__) && !defined(__arm64__)
#  define XXH_SPLIT_IN_PLACE(in, outLo, outHi)                                              \
    do {                                                                                    \
      /* Undocumented GCC/Clang operand modifier: %e0 = lower D half, %f0 = upper D half */ \
      /* https://github.com/gcc-mirror/gcc/blob/38cf91e5/gcc/config/arm/arm.c#L22486 */     \
      /* https://github.com/llvm-mirror/llvm/blob/2c4ca683/lib/Target/ARM/ARMAsmPrinter.cpp#L399 */ \
      __asm__("vzip.32  %e0, %f0" : "+w" (in));                                             \
      (outLo) = vget_low_u32 (vreinterpretq_u32_u64(in));                                   \
      (outHi) = vget_high_u32(vreinterpretq_u32_u64(in));                                   \
   } while (0)
# else
#  define XXH_SPLIT_IN_PLACE(in, outLo, outHi)                                            \
    do {                                                                                  \
      (outLo) = vmovn_u64    (in);                                                        \
      (outHi) = vshrn_n_u64  ((in), 32);                                                  \
    } while (0)
# endif
#endif  /* XXH_VECTOR == XXH_NEON */

/*
 * VSX and Z Vector helpers.
 *
 * This is very messy, and any pull requests to clean this up are welcome.
 *
 * There are a lot of problems with supporting VSX and s390x, due to
 * inconsistent intrinsics, spotty coverage, and multiple endiannesses.
 */
#if XXH_VECTOR == XXH_VSX
#  if defined(__s390x__)
#    include <s390intrin.h>
#  else
#    include <altivec.h>
#  endif

#  undef vector /* Undo the pollution */

typedef __vector unsigned long long xxh_u64x2;
typedef __vector unsigned char xxh_u8x16;
typedef __vector unsigned xxh_u32x4;

# ifndef XXH_VSX_BE
#  if defined(__BIG_ENDIAN__) \
  || (defined(__BYTE_ORDER__) && __BYTE_ORDER__ == __ORDER_BIG_ENDIAN__)
#    define XXH_VSX_BE 1
#  elif defined(__VEC_ELEMENT_REG_ORDER__) && __VEC_ELEMENT_REG_ORDER__ == __ORDER_BIG_ENDIAN__
#    warning "-maltivec=be is not recommended. Please use native endianness."
#    define XXH_VSX_BE 1
#  else
#    define XXH_VSX_BE 0
#  endif
# endif /* !defined(XXH_VSX_BE) */

# if XXH_VSX_BE
/* A wrapper for POWER9's vec_revb. */
#  if defined(__POWER9_VECTOR__) || (defined(__clang__) && defined(__s390x__))
#    define XXH_vec_revb vec_revb
#  else
XXH_FORCE_INLINE xxh_u64x2 XXH_vec_revb(xxh_u64x2 val)
{
    xxh_u8x16 const vByteSwap = { 0x07, 0x06, 0x05, 0x04, 0x03, 0x02, 0x01, 0x00,
                                  0x0F, 0x0E, 0x0D, 0x0C, 0x0B, 0x0A, 0x09, 0x08 };
    return vec_perm(val, val, vByteSwap);
}
#  endif
# endif /* XXH_VSX_BE */

/*
 * Performs an unaligned load and byte swaps it on big endian.
 */
XXH_FORCE_INLINE xxh_u64x2 XXH_vec_loadu(const void *ptr)
{
    xxh_u64x2 ret;
    memcpy(&ret, ptr, sizeof(xxh_u64x2));
# if XXH_VSX_BE
    ret = XXH_vec_revb(ret);
# endif
    return ret;
}

/*
 * vec_mulo and vec_mule are very problematic intrinsics on PowerPC
 *
 * These intrinsics weren't added until GCC 8, despite existing for a while,
 * and they are endian dependent. Also, their meaning swap depending on version.
 * */
# if defined(__s390x__)
 /* s390x is always big endian, no issue on this platform */
#  define XXH_vec_mulo vec_mulo
#  define XXH_vec_mule vec_mule
# elif defined(__clang__) && __has_builtin(__builtin_altivec_vmuleuw)
/* Clang has a better way to control this, we can just use the builtin which doesn't swap. */
#  define XXH_vec_mulo __builtin_altivec_vmulouw
#  define XXH_vec_mule __builtin_altivec_vmuleuw
# else
/* gcc needs inline assembly */
/* Adapted from https://github.com/google/highwayhash/blob/master/highwayhash/hh_vsx.h. */
XXH_FORCE_INLINE xxh_u64x2 XXH_vec_mulo(xxh_u32x4 a, xxh_u32x4 b)
{
    xxh_u64x2 result;
    __asm__("vmulouw %0, %1, %2" : "=v" (result) : "v" (a), "v" (b));
    return result;
}
XXH_FORCE_INLINE xxh_u64x2 XXH_vec_mule(xxh_u32x4 a, xxh_u32x4 b)
{
    xxh_u64x2 result;
    __asm__("vmuleuw %0, %1, %2" : "=v" (result) : "v" (a), "v" (b));
    return result;
}
# endif /* XXH_vec_mulo, XXH_vec_mule */
#endif /* XXH_VECTOR == XXH_VSX */


/* prefetch
 * can be disabled, by declaring XXH_NO_PREFETCH build macro */
#if defined(XXH_NO_PREFETCH)
#  define XXH_PREFETCH(ptr)  (void)(ptr)  /* disabled */
#else
#  if defined(_MSC_VER) && (defined(_M_X64) || defined(_M_I86))  /* _mm_prefetch() is not defined outside of x86/x64 */
#    include <mmintrin.h>   /* https://msdn.microsoft.com/fr-fr/library/84szxsww(v=vs.90).aspx */
#    define XXH_PREFETCH(ptr)  _mm_prefetch((const char*)(ptr), _MM_HINT_T0)
#  elif defined(__GNUC__) && ( (__GNUC__ >= 4) || ( (__GNUC__ == 3) && (__GNUC_MINOR__ >= 1) ) )
#    define XXH_PREFETCH(ptr)  __builtin_prefetch((ptr), 0 /* rw==read */, 3 /* locality */)
#  else
#    define XXH_PREFETCH(ptr) (void)(ptr)  /* disabled */
#  endif
#endif  /* XXH_NO_PREFETCH */


/* ==========================================
 * XXH3 default settings
 * ========================================== */

#define XXH_SECRET_DEFAULT_SIZE 192   /* minimum XXH3_SECRET_SIZE_MIN */

#if (XXH_SECRET_DEFAULT_SIZE < XXH3_SECRET_SIZE_MIN)
#  error "default keyset is not large enough"
#endif

/* Pseudorandom secret taken directly from FARSH */
XXH_ALIGN(64) static const xxh_u8 kSecret[XXH_SECRET_DEFAULT_SIZE] = {
    0xb8, 0xfe, 0x6c, 0x39, 0x23, 0xa4, 0x4b, 0xbe, 0x7c, 0x01, 0x81, 0x2c, 0xf7, 0x21, 0xad, 0x1c,
    0xde, 0xd4, 0x6d, 0xe9, 0x83, 0x90, 0x97, 0xdb, 0x72, 0x40, 0xa4, 0xa4, 0xb7, 0xb3, 0x67, 0x1f,
    0xcb, 0x79, 0xe6, 0x4e, 0xcc, 0xc0, 0xe5, 0x78, 0x82, 0x5a, 0xd0, 0x7d, 0xcc, 0xff, 0x72, 0x21,
    0xb8, 0x08, 0x46, 0x74, 0xf7, 0x43, 0x24, 0x8e, 0xe0, 0x35, 0x90, 0xe6, 0x81, 0x3a, 0x26, 0x4c,
    0x3c, 0x28, 0x52, 0xbb, 0x91, 0xc3, 0x00, 0xcb, 0x88, 0xd0, 0x65, 0x8b, 0x1b, 0x53, 0x2e, 0xa3,
    0x71, 0x64, 0x48, 0x97, 0xa2, 0x0d, 0xf9, 0x4e, 0x38, 0x19, 0xef, 0x46, 0xa9, 0xde, 0xac, 0xd8,
    0xa8, 0xfa, 0x76, 0x3f, 0xe3, 0x9c, 0x34, 0x3f, 0xf9, 0xdc, 0xbb, 0xc7, 0xc7, 0x0b, 0x4f, 0x1d,
    0x8a, 0x51, 0xe0, 0x4b, 0xcd, 0xb4, 0x59, 0x31, 0xc8, 0x9f, 0x7e, 0xc9, 0xd9, 0x78, 0x73, 0x64,

    0xea, 0xc5, 0xac, 0x83, 0x34, 0xd3, 0xeb, 0xc3, 0xc5, 0x81, 0xa0, 0xff, 0xfa, 0x13, 0x63, 0xeb,
    0x17, 0x0d, 0xdd, 0x51, 0xb7, 0xf0, 0xda, 0x49, 0xd3, 0x16, 0x55, 0x26, 0x29, 0xd4, 0x68, 0x9e,
    0x2b, 0x16, 0xbe, 0x58, 0x7d, 0x47, 0xa1, 0xfc, 0x8f, 0xf8, 0xb8, 0xd1, 0x7a, 0xd0, 0x31, 0xce,
    0x45, 0xcb, 0x3a, 0x8f, 0x95, 0x16, 0x04, 0x28, 0xaf, 0xd7, 0xfb, 0xca, 0xbb, 0x4b, 0x40, 0x7e,
};

/*
 * Does a 32-bit to 64-bit long multiply.
 *
 * Wraps __emulu on MSVC x86 because it tends to call __allmul when it doesn't
 * need to (but it shouldn't need to anyways, it is about 7 instructions to do
 * a 64x64 multiply...). Since we know that this will _always_ emit MULL, we
 * use that instead of the normal method.
 *
 * If you are compiling for platforms like Thumb-1 and don't have a better option,
 * you may also want to write your own long multiply routine here.
 *
 * XXH_FORCE_INLINE xxh_u64 XXH_mult32to64(xxh_u64 x, xxh_u64 y)
 * {
 *    return (x & 0xFFFFFFFF) * (y & 0xFFFFFFFF);
 * }
 */
#if defined(_MSC_VER) && defined(_M_IX86)
#    include <intrin.h>
#    define XXH_mult32to64(x, y) __emulu((unsigned)(x), (unsigned)(y))
#else
/*
 * Downcast + upcast is usually better than masking on older compilers like
 * GCC 4.2 (especially 32-bit ones), all without affecting newer compilers.
 *
 * The other method, (x & 0xFFFFFFFF) * (y & 0xFFFFFFFF), will AND both operands
 * and perform a full 64x64 multiply -- entirely redundant on 32-bit.
 */
#    define XXH_mult32to64(x, y) ((xxh_u64)(xxh_u32)(x) * (xxh_u64)(xxh_u32)(y))
#endif

/*
 * Calculates a 64->128-bit long multiply.
 *
 * Uses __uint128_t and _umul128 if available, otherwise uses a scalar version.
 */
static XXH128_hash_t
XXH_mult64to128(xxh_u64 lhs, xxh_u64 rhs)
{
    /*
     * GCC/Clang __uint128_t method.
     *
     * On most 64-bit targets, GCC and Clang define a __uint128_t type.
     * This is usually the best way as it usually uses a native long 64-bit
     * multiply, such as MULQ on x86_64 or MUL + UMULH on aarch64.
     *
     * Usually.
     *
     * Despite being a 32-bit platform, Clang (and emscripten) define this type
     * despite not having the arithmetic for it. This results in a laggy
     * compiler builtin call which calculates a full 128-bit multiply.
     * In that case it is best to use the portable one.
     * https://github.com/Cyan4973/xxHash/issues/211#issuecomment-515575677
     */
#if defined(__GNUC__) && !defined(__wasm__) \
    && defined(__SIZEOF_INT128__) \
    || (defined(_INTEGRAL_MAX_BITS) && _INTEGRAL_MAX_BITS >= 128)

    __uint128_t product = (__uint128_t)lhs * (__uint128_t)rhs;
    XXH128_hash_t const r128 = { (xxh_u64)(product), (xxh_u64)(product >> 64) };
    return r128;

    /*
     * MSVC for x64's _umul128 method.
     *
     * xxh_u64 _umul128(xxh_u64 Multiplier, xxh_u64 Multiplicand, xxh_u64 *HighProduct);
     *
     * This compiles to single operand MUL on x64.
     */
#elif defined(_M_X64) || defined(_M_IA64)

#ifndef _MSC_VER
#   pragma intrinsic(_umul128)
#endif
    xxh_u64 product_high;
    xxh_u64 const product_low = _umul128(lhs, rhs, &product_high);
    XXH128_hash_t const r128 = { product_low, product_high };
    return r128;

#else
    /*
     * Portable scalar method. Optimized for 32-bit and 64-bit ALUs.
     *
     * This is a fast and simple grade school multiply, which is shown below
     * with base 10 arithmetic instead of base 0x100000000.
     *
     *           9 3 // D2 lhs = 93
     *         x 7 5 // D2 rhs = 75
     *     ----------
     *           1 5 // D2 lo_lo = (93 % 10) * (75 % 10) = 15
     *         4 5 | // D2 hi_lo = (93 / 10) * (75 % 10) = 45
     *         2 1 | // D2 lo_hi = (93 % 10) * (75 / 10) = 21
     *     + 6 3 | | // D2 hi_hi = (93 / 10) * (75 / 10) = 63
     *     ---------
     *         2 7 | // D2 cross = (15 / 10) + (45 % 10) + 21 = 27
     *     + 6 7 | | // D2 upper = (27 / 10) + (45 / 10) + 63 = 67
     *     ---------
     *       6 9 7 5 // D4 res = (27 * 10) + (15 % 10) + (67 * 100) = 6975
     *
     * The reasons for adding the products like this are:
     *  1. It avoids manual carry tracking. Just like how
     *     (9 * 9) + 9 + 9 = 99, the same applies with this for UINT64_MAX.
     *     This avoids a lot of complexity.
     *
     *  2. It hints for, and on Clang, compiles to, the powerful UMAAL
     *     instruction available in ARM's Digital Signal Processing extension
     *     in 32-bit ARMv6 and later, which is shown below:
     *
     *         void UMAAL(xxh_u32 *RdLo, xxh_u32 *RdHi, xxh_u32 Rn, xxh_u32 Rm)
     *         {
     *             xxh_u64 product = (xxh_u64)*RdLo * (xxh_u64)*RdHi + Rn + Rm;
     *             *RdLo = (xxh_u32)(product & 0xFFFFFFFF);
     *             *RdHi = (xxh_u32)(product >> 32);
     *         }
     *
     *     This instruction was designed for efficient long multiplication, and
     *     allows this to be calculated in only 4 instructions at speeds
     *     comparable to some 64-bit ALUs.
     *
     *  3. It isn't terrible on other platforms. Usually this will be a couple
     *     of 32-bit ADD/ADCs.
     */

    /* First calculate all of the cross products. */
    xxh_u64 const lo_lo = XXH_mult32to64(lhs & 0xFFFFFFFF, rhs & 0xFFFFFFFF);
    xxh_u64 const hi_lo = XXH_mult32to64(lhs >> 32,        rhs & 0xFFFFFFFF);
    xxh_u64 const lo_hi = XXH_mult32to64(lhs & 0xFFFFFFFF, rhs >> 32);
    xxh_u64 const hi_hi = XXH_mult32to64(lhs >> 32,        rhs >> 32);

    /* Now add the products together. These will never overflow. */
    xxh_u64 const cross = (lo_lo >> 32) + (hi_lo & 0xFFFFFFFF) + lo_hi;
    xxh_u64 const upper = (hi_lo >> 32) + (cross >> 32)        + hi_hi;
    xxh_u64 const lower = (cross << 32) | (lo_lo & 0xFFFFFFFF);

    XXH128_hash_t r128 = { lower, upper };
    return r128;
#endif
}

/*
 * Does a 64-bit to 128-bit multiply, then XOR folds it.
 *
 * The reason for the separate function is to prevent passing too many structs
 * around by value. This will hopefully inline the multiply, but we don't force it.
 */
static xxh_u64
XXH3_mul128_fold64(xxh_u64 lhs, xxh_u64 rhs)
{
    XXH128_hash_t product = XXH_mult64to128(lhs, rhs);
    return product.low64 ^ product.high64;
}

/* Seems to produce slightly better code on GCC for some reason. */
XXH_FORCE_INLINE xxh_u64 XXH_xorshift64(xxh_u64 v64, int shift)
{
    XXH_ASSERT(0 <= shift && shift < 64);
    return v64 ^ (v64 >> shift);
}

/*
 * We don't need to (or want to) mix as much as XXH64.
 *
 * Short hashes are more evenly distributed, so it isn't necessary.
 */
static XXH64_hash_t XXH3_avalanche(xxh_u64 h64)
{
    h64 = XXH_xorshift64(h64, 37);
    h64 *= 0x165667919E3779F9ULL;
    h64 = XXH_xorshift64(h64, 32);
    return h64;
}


/* ==========================================
 * Short keys
 * ==========================================
 * One of the shortcomings of XXH32 and XXH64 was that their performance was
 * sub-optimal on short lengths. It used an iterative algorithm which strongly
 * favored lengths that were a multiple of 4 or 8.
 *
 * Instead of iterating over individual inputs, we use a set of single shot
 * functions which piece together a range of lengths and operate in constant time.
 *
 * Additionally, the number of multiplies has been significantly reduced. This
 * reduces latency, especially when emulating 64-bit multiplies on 32-bit.
 *
 * Depending on the platform, this may or may not be faster than XXH32, but it
 * is almost guaranteed to be faster than XXH64.
 */

/*
 * At very short lengths, there isn't enough input to fully hide secrets, or use
 * the entire secret.
 *
 * There is also only a limited amount of mixing we can do before significantly
 * impacting performance.
 *
 * Therefore, we use different sections of the secret and always mix two secret
 * samples with an XOR. This should have no effect on performance on the
 * seedless or withSeed variants because everything _should_ be constant folded
 * by modern compilers.
 *
 * The XOR mixing hides individual parts of the secret and increases entropy.
 *
 * This adds an extra layer of strength for custom secrets.
 */
XXH_FORCE_INLINE XXH64_hash_t
XXH3_len_1to3_64b(const xxh_u8* input, size_t len, const xxh_u8* secret, XXH64_hash_t seed)
{
    XXH_ASSERT(input != NULL);
    XXH_ASSERT(1 <= len && len <= 3);
    XXH_ASSERT(secret != NULL);
    /*
     * len = 1: combined = { input[0], 0x01, input[0], input[0] }
     * len = 2: combined = { input[1], 0x02, input[0], input[1] }
     * len = 3: combined = { input[2], 0x03, input[0], input[1] }
     */
    {   xxh_u8 const c1 = input[0];
        xxh_u8 const c2 = input[len >> 1];
        xxh_u8 const c3 = input[len - 1];
        xxh_u32 const combined = ((xxh_u32)c1<<16) | (((xxh_u32)c2) << 24) | (((xxh_u32)c3) << 0) | (((xxh_u32)len) << 8);
        xxh_u64 const bitflip = (XXH_readLE32(secret) ^ XXH_readLE32(secret+4)) + seed;
        xxh_u64 const keyed = (xxh_u64)combined ^ bitflip;
        xxh_u64 const mixed = keyed * PRIME64_1;
        return XXH3_avalanche(mixed);
    }
}

XXH_FORCE_INLINE XXH64_hash_t
XXH3_len_4to8_64b(const xxh_u8* input, size_t len, const xxh_u8* secret, XXH64_hash_t seed)
{
    XXH_ASSERT(input != NULL);
    XXH_ASSERT(secret != NULL);
    XXH_ASSERT(4 <= len && len < 8);
    seed ^= (xxh_u64)XXH_swap32((xxh_u32)seed) << 32;
    {   xxh_u32 const input1 = XXH_readLE32(input);
        xxh_u32 const input2 = XXH_readLE32(input + len - 4);
        xxh_u64 const bitflip = (XXH_readLE64(secret+8) ^ XXH_readLE64(secret+16)) - seed;
        xxh_u64 const input64 = input2 + (((xxh_u64)input1) << 32);
        xxh_u64 x = input64 ^ bitflip;
        /* this mix is inspired by Pelle Evensen's rrmxmx */
        x ^= XXH_rotl64(x, 49) ^ XXH_rotl64(x, 24);
        x *= 0x9FB21C651E98DF25ULL;
        x ^= (x >> 35) + len ;
        x *= 0x9FB21C651E98DF25ULL;
        return XXH_xorshift64(x, 28);
    }
}

XXH_FORCE_INLINE XXH64_hash_t
XXH3_len_9to16_64b(const xxh_u8* input, size_t len, const xxh_u8* secret, XXH64_hash_t seed)
{
    XXH_ASSERT(input != NULL);
    XXH_ASSERT(secret != NULL);
    XXH_ASSERT(8 <= len && len <= 16);
    {   xxh_u64 const bitflip1 = (XXH_readLE64(secret+24) ^ XXH_readLE64(secret+32)) + seed;
        xxh_u64 const bitflip2 = (XXH_readLE64(secret+40) ^ XXH_readLE64(secret+48)) - seed;
        xxh_u64 const input_lo = XXH_readLE64(input)           ^ bitflip1;
        xxh_u64 const input_hi = XXH_readLE64(input + len - 8) ^ bitflip2;
        xxh_u64 const acc = len
                          + XXH_swap64(input_lo) + input_hi
                          + XXH3_mul128_fold64(input_lo, input_hi);
        return XXH3_avalanche(acc);
    }
}

XXH_FORCE_INLINE XXH64_hash_t
XXH3_len_0to16_64b(const xxh_u8* input, size_t len, const xxh_u8* secret, XXH64_hash_t seed)
{
    XXH_ASSERT(len <= 16);
    {   if (XXH_likely(len >  8)) return XXH3_len_9to16_64b(input, len, secret, seed);
        if (XXH_likely(len >= 4)) return XXH3_len_4to8_64b(input, len, secret, seed);
        if (len) return XXH3_len_1to3_64b(input, len, secret, seed);
        return XXH3_avalanche((PRIME64_1 + seed) ^ (XXH_readLE64(secret+56) ^ XXH_readLE64(secret+64)));
    }
}

/*
 * DISCLAIMER: There are known *seed-dependent* multicollisions here due to
 * multiplication by zero, affecting hashes of lengths 17 to 240.
 *
 * However, they are very unlikely.
 *
 * Keep this in mind when using the unseeded XXH3_64bits() variant: As with all
 * unseeded non-cryptographic hashes, it does not attempt to defend itself
 * against specially crafted inputs, only random inputs.
 *
 * Compared to classic UMAC where a 1 in 2^31 chance of 4 consecutive bytes
 * cancelling out the secret is taken an arbitrary number of times (addressed
 * in XXH3_accumulate_512), this collision is very unlikely with random inputs
 * and/or proper seeding:
 *
 * This only has a 1 in 2^63 chance of 8 consecutive bytes cancelling out, in a
 * function that is only called up to 16 times per hash with up to 240 bytes of
 * input.
 *
 * This is not too bad for a non-cryptographic hash function, especially with
 * only 64 bit outputs.
 *
 * The 128-bit variant (which trades some speed for strength) is NOT affected
 * by this, although it is always a good idea to use a proper seed if you care
 * about strength.
 */
XXH_FORCE_INLINE xxh_u64 XXH3_mix16B(const xxh_u8* XXH_RESTRICT input,
                                     const xxh_u8* XXH_RESTRICT secret, xxh_u64 seed64)
{
#if defined(__GNUC__) && !defined(__clang__) /* GCC, not Clang */ \
  && defined(__i386__) && defined(__SSE2__)  /* x86 + SSE2 */ \
  && !defined(XXH_ENABLE_AUTOVECTORIZE)      /* Define to disable like XXH32 hack */
    /*
     * UGLY HACK:
     * GCC for x86 tends to autovectorize the 128-bit multiply, resulting in
     * slower code.
     *
     * By forcing seed64 into a register, we disrupt the cost model and
     * cause it to scalarize. See `XXH32_round()`
     *
     * FIXME: Clang's output is still _much_ faster -- On an AMD Ryzen 3600,
     * XXH3_64bits @ len=240 runs at 4.6 GB/s with Clang 9, but 3.3 GB/s on
     * GCC 9.2, despite both emitting scalar code.
     *
     * GCC generates much better scalar code than Clang for the rest of XXH3,
     * which is why finding a more optimal codepath is an interest.
     */
    __asm__ ("" : "+r" (seed64));
#endif
    {   xxh_u64 const input_lo = XXH_readLE64(input);
        xxh_u64 const input_hi = XXH_readLE64(input+8);
        return XXH3_mul128_fold64(
            input_lo ^ (XXH_readLE64(secret)   + seed64),
            input_hi ^ (XXH_readLE64(secret+8) - seed64)
        );
    }
}

/* For mid range keys, XXH3 uses a Mum-hash variant. */
XXH_FORCE_INLINE XXH64_hash_t
XXH3_len_17to128_64b(const xxh_u8* XXH_RESTRICT input, size_t len,
                     const xxh_u8* XXH_RESTRICT secret, size_t secretSize,
                     XXH64_hash_t seed)
{
    XXH_ASSERT(secretSize >= XXH3_SECRET_SIZE_MIN); (void)secretSize;
    XXH_ASSERT(16 < len && len <= 128);

    {   xxh_u64 acc = len * PRIME64_1;
        if (len > 32) {
            if (len > 64) {
                if (len > 96) {
                    acc += XXH3_mix16B(input+48, secret+96, seed);
                    acc += XXH3_mix16B(input+len-64, secret+112, seed);
                }
                acc += XXH3_mix16B(input+32, secret+64, seed);
                acc += XXH3_mix16B(input+len-48, secret+80, seed);
            }
            acc += XXH3_mix16B(input+16, secret+32, seed);
            acc += XXH3_mix16B(input+len-32, secret+48, seed);
        }
        acc += XXH3_mix16B(input+0, secret+0, seed);
        acc += XXH3_mix16B(input+len-16, secret+16, seed);

        return XXH3_avalanche(acc);
    }
}

#define XXH3_MIDSIZE_MAX 240

XXH_NO_INLINE XXH64_hash_t
XXH3_len_129to240_64b(const xxh_u8* XXH_RESTRICT input, size_t len,
                      const xxh_u8* XXH_RESTRICT secret, size_t secretSize,
                      XXH64_hash_t seed)
{
    XXH_ASSERT(secretSize >= XXH3_SECRET_SIZE_MIN); (void)secretSize;
    XXH_ASSERT(128 < len && len <= XXH3_MIDSIZE_MAX);

    #define XXH3_MIDSIZE_STARTOFFSET 3
    #define XXH3_MIDSIZE_LASTOFFSET  17

    {   xxh_u64 acc = len * PRIME64_1;
        int const nbRounds = (int)len / 16;
        int i;
        for (i=0; i<8; i++) {
            acc += XXH3_mix16B(input+(16*i), secret+(16*i), seed);
        }
        acc = XXH3_avalanche(acc);
        XXH_ASSERT(nbRounds >= 8);
#if defined(__clang__)                                /* Clang */ \
    && (defined(__ARM_NEON) || defined(__ARM_NEON__)) /* NEON */ \
    && !defined(XXH_ENABLE_AUTOVECTORIZE)             /* Define to disable */
        /*
         * UGLY HACK:
         * Clang for ARMv7-A tries to vectorize this loop, similar to GCC x86.
         * In everywhere else, it uses scalar code.
         *
         * For 64->128-bit multiplies, even if the NEON was 100% optimal, it
         * would still be slower than UMAAL (see XXH_mult64to128).
         *
         * Unfortunately, Clang doesn't handle the long multiplies properly and
         * converts them to the nonexistent "vmulq_u64" intrinsic, which is then
         * scalarized into an ugly mess of VMOV.32 instructions.
         *
         * This mess is difficult to avoid without turning autovectorization
         * off completely, but they are usually relatively minor and/or not
         * worth it to fix.
         *
         * This loop is the easiest to fix, as unlike XXH32, this pragma
         * _actually works_ because it is a loop vectorization instead of an
         * SLP vectorization.
         */
        #pragma clang loop vectorize(disable)
#endif
        for (i=8 ; i < nbRounds; i++) {
            acc += XXH3_mix16B(input+(16*i), secret+(16*(i-8)) + XXH3_MIDSIZE_STARTOFFSET, seed);
        }
        /* last bytes */
        acc += XXH3_mix16B(input + len - 16, secret + XXH3_SECRET_SIZE_MIN - XXH3_MIDSIZE_LASTOFFSET, seed);
        return XXH3_avalanche(acc);
    }
}


/* ===    Long Keys    === */

#define STRIPE_LEN 64
#define XXH_SECRET_CONSUME_RATE 8   /* nb of secret bytes consumed at each accumulation */
#define ACC_NB (STRIPE_LEN / sizeof(xxh_u64))

typedef enum { XXH3_acc_64bits, XXH3_acc_128bits } XXH3_accWidth_e;

/*
 * XXH3_accumulate_512 is the tightest loop for long inputs, and it is the most optimized.
 *
 * It is a hardened version of UMAC, based off of FARSH's implementation.
 *
 * This was chosen because it adapts quite well to 32-bit, 64-bit, and SIMD
 * implementations, and it is ridiculously fast.
 *
 * We harden it by mixing the original input to the accumulators as well as the product.
 *
 * This means that in the (relatively likely) case of a multiply by zero, the
 * original input is preserved.
 *
 * On 128-bit inputs, we swap 64-bit pairs when we add the input to improve
 * cross-pollination, as otherwise the upper and lower halves would be
 * essentially independent.
 *
 * This doesn't matter on 64-bit hashes since they all get merged together in
 * the end, so we skip the extra step.
 *
 * Both XXH3_64bits and XXH3_128bits use this subroutine.
 */
XXH_FORCE_INLINE void
XXH3_accumulate_512(      void* XXH_RESTRICT acc,
                    const void* XXH_RESTRICT input,
                    const void* XXH_RESTRICT secret,
                    XXH3_accWidth_e accWidth)
{
#if (XXH_VECTOR == XXH_AVX2)

    XXH_ASSERT((((size_t)acc) & 31) == 0);
    {   XXH_ALIGN(32) __m256i* const xacc    =       (__m256i *) acc;
        /* Unaligned. This is mainly for pointer arithmetic, and because
         * _mm256_loadu_si256 requires  a const __m256i * pointer for some reason. */
        const         __m256i* const xinput  = (const __m256i *) input;
        /* Unaligned. This is mainly for pointer arithmetic, and because
         * _mm256_loadu_si256 requires a const __m256i * pointer for some reason. */
        const         __m256i* const xsecret = (const __m256i *) secret;

        size_t i;
        for (i=0; i < STRIPE_LEN/sizeof(__m256i); i++) {
            /* data_vec    = xinput[i]; */
            __m256i const data_vec    = _mm256_loadu_si256    (xinput+i);
            /* key_vec     = xsecret[i]; */
            __m256i const key_vec     = _mm256_loadu_si256   (xsecret+i);
            /* data_key    = data_vec ^ key_vec; */
            __m256i const data_key    = _mm256_xor_si256     (data_vec, key_vec);
            /* data_key_lo = data_key >> 32; */
            __m256i const data_key_lo = _mm256_shuffle_epi32 (data_key, _MM_SHUFFLE(0, 3, 0, 1));
            /* product     = (data_key & 0xffffffff) * (data_key_lo & 0xffffffff); */
            __m256i const product     = _mm256_mul_epu32     (data_key, data_key_lo);
            if (accWidth == XXH3_acc_128bits) {
                /* xacc[i] += swap(data_vec); */
                __m256i const data_swap = _mm256_shuffle_epi32(data_vec, _MM_SHUFFLE(1, 0, 3, 2));
                __m256i const sum       = _mm256_add_epi64(xacc[i], data_swap);
                /* xacc[i] += product; */
                xacc[i] = _mm256_add_epi64(product, sum);
            } else {  /* XXH3_acc_64bits */
                /* xacc[i] += data_vec; */
                __m256i const sum = _mm256_add_epi64(xacc[i], data_vec);
                /* xacc[i] += product; */
                xacc[i] = _mm256_add_epi64(product, sum);
            }
    }   }

#elif (XXH_VECTOR == XXH_SSE2)

    /* SSE2 is just a half-scale version of the AVX2 version. */
    XXH_ASSERT((((size_t)acc) & 15) == 0);
    {   XXH_ALIGN(16) __m128i* const xacc    =       (__m128i *) acc;
        /* Unaligned. This is mainly for pointer arithmetic, and because
         * _mm_loadu_si128 requires a const __m128i * pointer for some reason. */
        const         __m128i* const xinput  = (const __m128i *) input;
        /* Unaligned. This is mainly for pointer arithmetic, and because
         * _mm_loadu_si128 requires a const __m128i * pointer for some reason. */
        const         __m128i* const xsecret = (const __m128i *) secret;

        size_t i;
        for (i=0; i < STRIPE_LEN/sizeof(__m128i); i++) {
            /* data_vec    = xinput[i]; */
            __m128i const data_vec    = _mm_loadu_si128   (xinput+i);
            /* key_vec     = xsecret[i]; */
            __m128i const key_vec     = _mm_loadu_si128   (xsecret+i);
            /* data_key    = data_vec ^ key_vec; */
            __m128i const data_key    = _mm_xor_si128     (data_vec, key_vec);
            /* data_key_lo = data_key >> 32; */
            __m128i const data_key_lo = _mm_shuffle_epi32 (data_key, _MM_SHUFFLE(0, 3, 0, 1));
            /* product     = (data_key & 0xffffffff) * (data_key_lo & 0xffffffff); */
            __m128i const product     = _mm_mul_epu32     (data_key, data_key_lo);
            if (accWidth == XXH3_acc_128bits) {
                /* xacc[i] += swap(data_vec); */
                __m128i const data_swap = _mm_shuffle_epi32(data_vec, _MM_SHUFFLE(1,0,3,2));
                __m128i const sum       = _mm_add_epi64(xacc[i], data_swap);
                /* xacc[i] += product; */
                xacc[i] = _mm_add_epi64(product, sum);
            } else {  /* XXH3_acc_64bits */
                /* xacc[i] += data_vec; */
                __m128i const sum = _mm_add_epi64(xacc[i], data_vec);
                /* xacc[i] += product; */
                xacc[i] = _mm_add_epi64(product, sum);
            }
    }   }

#elif (XXH_VECTOR == XXH_NEON)

    XXH_ASSERT((((size_t)acc) & 15) == 0);
    {
        XXH_ALIGN(16) uint64x2_t* const xacc = (uint64x2_t *) acc;
        /* We don't use a uint32x4_t pointer because it causes bus errors on ARMv7. */
        uint8_t const* const xinput = (const uint8_t *) input;
        uint8_t const* const xsecret  = (const uint8_t *) secret;

        size_t i;
        for (i=0; i < STRIPE_LEN / sizeof(uint64x2_t); i++) {
            /* data_vec = xinput[i]; */
            uint8x16_t data_vec    = vld1q_u8(xinput  + (i * 16));
            /* key_vec  = xsecret[i];  */
            uint8x16_t key_vec     = vld1q_u8(xsecret + (i * 16));
            /* data_key = data_vec ^ key_vec; */
            uint64x2_t data_key    = vreinterpretq_u64_u8(veorq_u8(data_vec, key_vec));
            uint32x2_t data_key_lo, data_key_hi;
            if (accWidth == XXH3_acc_64bits) {
                /* xacc[i] += data_vec; */
                xacc[i] = vaddq_u64 (xacc[i], vreinterpretq_u64_u8(data_vec));
            } else {  /* XXH3_acc_128bits */
                /* xacc[i] += swap(data_vec); */
                uint64x2_t const data64  = vreinterpretq_u64_u8(data_vec);
                uint64x2_t const swapped = vextq_u64(data64, data64, 1);
                xacc[i] = vaddq_u64 (xacc[i], swapped);
            }
            /* data_key_lo = (uint32x2_t) (data_key & 0xFFFFFFFF);
             * data_key_hi = (uint32x2_t) (data_key >> 32);
             * data_key = UNDEFINED; */
            XXH_SPLIT_IN_PLACE(data_key, data_key_lo, data_key_hi);
            /* xacc[i] += (uint64x2_t) data_key_lo * (uint64x2_t) data_key_hi; */
            xacc[i] = vmlal_u32 (xacc[i], data_key_lo, data_key_hi);

        }
    }

#elif (XXH_VECTOR == XXH_VSX)
          xxh_u64x2* const xacc     =       (xxh_u64x2*) acc;    /* presumed aligned */
    xxh_u64x2 const* const xinput   = (xxh_u64x2 const*) input;   /* no alignment restriction */
    xxh_u64x2 const* const xsecret  = (xxh_u64x2 const*) secret;    /* no alignment restriction */
    xxh_u64x2 const v32 = { 32, 32 };
    size_t i;
    for (i = 0; i < STRIPE_LEN / sizeof(xxh_u64x2); i++) {
        /* data_vec = xinput[i]; */
        xxh_u64x2 const data_vec = XXH_vec_loadu(xinput + i);
        /* key_vec = xsecret[i]; */
        xxh_u64x2 const key_vec  = XXH_vec_loadu(xsecret + i);
        xxh_u64x2 const data_key = data_vec ^ key_vec;
        /* shuffled = (data_key << 32) | (data_key >> 32); */
        xxh_u32x4 const shuffled = (xxh_u32x4)vec_rl(data_key, v32);
        /* product = ((xxh_u64x2)data_key & 0xFFFFFFFF) * ((xxh_u64x2)shuffled & 0xFFFFFFFF); */
        xxh_u64x2 const product  = XXH_vec_mulo((xxh_u32x4)data_key, shuffled);
        xacc[i] += product;

        if (accWidth == XXH3_acc_64bits) {
            xacc[i] += data_vec;
        } else {  /* XXH3_acc_128bits */
            /* swap high and low halves */
#ifdef __s390x__
            xxh_u64x2 const data_swapped = vec_permi(data_vec, data_vec, 2);
#else
            xxh_u64x2 const data_swapped = vec_xxpermdi(data_vec, data_vec, 2);
#endif
            xacc[i] += data_swapped;
        }
    }

#else   /* scalar variant of Accumulator - universal */

    XXH_ALIGN(XXH_ACC_ALIGN) xxh_u64* const xacc = (xxh_u64*) acc; /* presumed aligned */
    const xxh_u8* const xinput = (const xxh_u8*) input;  /* no alignment restriction */
    const xxh_u8* const xsecret  = (const xxh_u8*) secret;   /* no alignment restriction */
    size_t i;
    XXH_ASSERT(((size_t)acc & (XXH_ACC_ALIGN-1)) == 0);
    for (i=0; i < ACC_NB; i++) {
        xxh_u64 const data_val = XXH_readLE64(xinput + 8*i);
        xxh_u64 const data_key = data_val ^ XXH_readLE64(xsecret + i*8);

        if (accWidth == XXH3_acc_64bits) {
            xacc[i] += data_val;
        } else {
            xacc[i ^ 1] += data_val; /* swap adjacent lanes */
        }
        xacc[i] += XXH_mult32to64(data_key & 0xFFFFFFFF, data_key >> 32);
    }
#endif
}

/*
 * XXH3_scrambleAcc: Scrambles the accumulators to improve mixing.
 *
 * Multiplication isn't perfect, as explained by Google in HighwayHash:
 *
 *  // Multiplication mixes/scrambles bytes 0-7 of the 64-bit result to
 *  // varying degrees. In descending order of goodness, bytes
 *  // 3 4 2 5 1 6 0 7 have quality 228 224 164 160 100 96 36 32.
 *  // As expected, the upper and lower bytes are much worse.
 *
 * Source: https://github.com/google/highwayhash/blob/0aaf66b/highwayhash/hh_avx2.h#L291
 *
 * Since our algorithm uses a pseudorandom secret to add some variance into the
 * mix, we don't need to (or want to) mix as often or as much as HighwayHash does.
 *
 * This isn't as tight as XXH3_accumulate, but still written in SIMD to avoid
 * extraction.
 *
 * Both XXH3_64bits and XXH3_128bits use this subroutine.
 */
XXH_FORCE_INLINE void
XXH3_scrambleAcc(void* XXH_RESTRICT acc, const void* XXH_RESTRICT secret)
{
#if (XXH_VECTOR == XXH_AVX2)

    XXH_ASSERT((((size_t)acc) & 31) == 0);
    {   XXH_ALIGN(32) __m256i* const xacc = (__m256i*) acc;
        /* Unaligned. This is mainly for pointer arithmetic, and because
         * _mm256_loadu_si256 requires a const __m256i * pointer for some reason. */
        const         __m256i* const xsecret = (const __m256i *) secret;
        const __m256i prime32 = _mm256_set1_epi32((int)PRIME32_1);

        size_t i;
        for (i=0; i < STRIPE_LEN/sizeof(__m256i); i++) {
            /* xacc[i] ^= (xacc[i] >> 47) */
            __m256i const acc_vec     = xacc[i];
            __m256i const shifted     = _mm256_srli_epi64    (acc_vec, 47);
            __m256i const data_vec    = _mm256_xor_si256     (acc_vec, shifted);
            /* xacc[i] ^= xsecret; */
            __m256i const key_vec     = _mm256_loadu_si256   (xsecret+i);
            __m256i const data_key    = _mm256_xor_si256     (data_vec, key_vec);

            /* xacc[i] *= PRIME32_1; */
            __m256i const data_key_hi = _mm256_shuffle_epi32 (data_key, _MM_SHUFFLE(0, 3, 0, 1));
            __m256i const prod_lo     = _mm256_mul_epu32     (data_key, prime32);
            __m256i const prod_hi     = _mm256_mul_epu32     (data_key_hi, prime32);
            xacc[i] = _mm256_add_epi64(prod_lo, _mm256_slli_epi64(prod_hi, 32));
        }
    }

#elif (XXH_VECTOR == XXH_SSE2)

    XXH_ASSERT((((size_t)acc) & 15) == 0);
    {   XXH_ALIGN(16) __m128i* const xacc = (__m128i*) acc;
        /* Unaligned. This is mainly for pointer arithmetic, and because
         * _mm_loadu_si128 requires a const __m128i * pointer for some reason. */
        const         __m128i* const xsecret = (const __m128i *) secret;
        const __m128i prime32 = _mm_set1_epi32((int)PRIME32_1);

        size_t i;
        for (i=0; i < STRIPE_LEN/sizeof(__m128i); i++) {
            /* xacc[i] ^= (xacc[i] >> 47) */
            __m128i const acc_vec     = xacc[i];
            __m128i const shifted     = _mm_srli_epi64    (acc_vec, 47);
            __m128i const data_vec    = _mm_xor_si128     (acc_vec, shifted);
            /* xacc[i] ^= xsecret[i]; */
            __m128i const key_vec     = _mm_loadu_si128   (xsecret+i);
            __m128i const data_key    = _mm_xor_si128     (data_vec, key_vec);

            /* xacc[i] *= PRIME32_1; */
            __m128i const data_key_hi = _mm_shuffle_epi32 (data_key, _MM_SHUFFLE(0, 3, 0, 1));
            __m128i const prod_lo     = _mm_mul_epu32     (data_key, prime32);
            __m128i const prod_hi     = _mm_mul_epu32     (data_key_hi, prime32);
            xacc[i] = _mm_add_epi64(prod_lo, _mm_slli_epi64(prod_hi, 32));
        }
    }

#elif (XXH_VECTOR == XXH_NEON)

    XXH_ASSERT((((size_t)acc) & 15) == 0);

    {   uint64x2_t* xacc       = (uint64x2_t*) acc;
        uint8_t const* xsecret = (uint8_t const*) secret;
        uint32x2_t prime       = vdup_n_u32 (PRIME32_1);

        size_t i;
        for (i=0; i < STRIPE_LEN/sizeof(uint64x2_t); i++) {
            /* xacc[i] ^= (xacc[i] >> 47); */
            uint64x2_t acc_vec  = xacc[i];
            uint64x2_t shifted  = vshrq_n_u64 (acc_vec, 47);
            uint64x2_t data_vec = veorq_u64   (acc_vec, shifted);

            /* xacc[i] ^= xsecret[i]; */
            uint8x16_t key_vec  = vld1q_u8(xsecret + (i * 16));
            uint64x2_t data_key = veorq_u64(data_vec, vreinterpretq_u64_u8(key_vec));

            /* xacc[i] *= PRIME32_1 */
            uint32x2_t data_key_lo, data_key_hi;
            /* data_key_lo = (uint32x2_t) (xacc[i] & 0xFFFFFFFF);
             * data_key_hi = (uint32x2_t) (xacc[i] >> 32);
             * xacc[i] = UNDEFINED; */
            XXH_SPLIT_IN_PLACE(data_key, data_key_lo, data_key_hi);
            {   /*
                 * prod_hi = (data_key >> 32) * PRIME32_1;
                 *
                 * Avoid vmul_u32 + vshll_n_u32 since Clang 6 and 7 will
                 * incorrectly "optimize" this:
                 *   tmp     = vmul_u32(vmovn_u64(a), vmovn_u64(b));
                 *   shifted = vshll_n_u32(tmp, 32);
                 * to this:
                 *   tmp     = "vmulq_u64"(a, b); // no such thing!
                 *   shifted = vshlq_n_u64(tmp, 32);
                 *
                 * However, unlike SSE, Clang lacks a 64-bit multiply routine
                 * for NEON, and it scalarizes two 64-bit multiplies instead.
                 *
                 * vmull_u32 has the same timing as vmul_u32, and it avoids
                 * this bug completely.
                 * See https://bugs.llvm.org/show_bug.cgi?id=39967
                 */
                uint64x2_t prod_hi = vmull_u32 (data_key_hi, prime);
                /* xacc[i] = prod_hi << 32; */
                xacc[i] = vshlq_n_u64(prod_hi, 32);
                /* xacc[i] += (prod_hi & 0xFFFFFFFF) * PRIME32_1; */
                xacc[i] = vmlal_u32(xacc[i], data_key_lo, prime);
            }
    }   }

#elif (XXH_VECTOR == XXH_VSX)

    XXH_ASSERT((((size_t)acc) & 15) == 0);

    {         xxh_u64x2* const xacc    =       (xxh_u64x2*) acc;
        const xxh_u64x2* const xsecret = (const xxh_u64x2*) secret;
        /* constants */
        xxh_u64x2 const v32  = { 32, 32 };
        xxh_u64x2 const v47 = { 47, 47 };
        xxh_u32x4 const prime = { PRIME32_1, PRIME32_1, PRIME32_1, PRIME32_1 };
        size_t i;
        for (i = 0; i < STRIPE_LEN / sizeof(xxh_u64x2); i++) {
            /* xacc[i] ^= (xacc[i] >> 47); */
            xxh_u64x2 const acc_vec  = xacc[i];
            xxh_u64x2 const data_vec = acc_vec ^ (acc_vec >> v47);

            /* xacc[i] ^= xsecret[i]; */
            xxh_u64x2 const key_vec  = XXH_vec_loadu(xsecret + i);
            xxh_u64x2 const data_key = data_vec ^ key_vec;

            /* xacc[i] *= PRIME32_1 */
            /* prod_lo = ((xxh_u64x2)data_key & 0xFFFFFFFF) * ((xxh_u64x2)prime & 0xFFFFFFFF);  */
            xxh_u64x2 const prod_even  = XXH_vec_mule((xxh_u32x4)data_key, prime);
            /* prod_hi = ((xxh_u64x2)data_key >> 32) * ((xxh_u64x2)prime >> 32);  */
            xxh_u64x2 const prod_odd  = XXH_vec_mulo((xxh_u32x4)data_key, prime);
            xacc[i] = prod_odd + (prod_even << v32);
    }   }

#else   /* scalar variant of Scrambler - universal */

    XXH_ALIGN(XXH_ACC_ALIGN) xxh_u64* const xacc = (xxh_u64*) acc;   /* presumed aligned */
    const xxh_u8* const xsecret = (const xxh_u8*) secret;   /* no alignment restriction */
    size_t i;
    XXH_ASSERT((((size_t)acc) & (XXH_ACC_ALIGN-1)) == 0);
    for (i=0; i < ACC_NB; i++) {
        xxh_u64 const key64 = XXH_readLE64(xsecret + 8*i);
        xxh_u64 acc64 = xacc[i];
        acc64 = XXH_xorshift64(acc64, 47);
        acc64 ^= key64;
        acc64 *= PRIME32_1;
        xacc[i] = acc64;
    }

#endif
}

#define XXH_PREFETCH_DIST 384

/*
 * XXH3_accumulate()
 * Loops over XXH3_accumulate_512().
 * Assumption: nbStripes will not overflow the secret size
 */
XXH_FORCE_INLINE void
XXH3_accumulate(     xxh_u64* XXH_RESTRICT acc,
                const xxh_u8* XXH_RESTRICT input,
                const xxh_u8* XXH_RESTRICT secret,
                      size_t nbStripes,
                      XXH3_accWidth_e accWidth)
{
    size_t n;
    for (n = 0; n < nbStripes; n++ ) {
        const xxh_u8* const in = input + n*STRIPE_LEN;
        XXH_PREFETCH(in + XXH_PREFETCH_DIST);
        XXH3_accumulate_512(acc,
                            in,
                            secret + n*XXH_SECRET_CONSUME_RATE,
                            accWidth);
    }
}

XXH_FORCE_INLINE void
XXH3_hashLong_internal_loop( xxh_u64* XXH_RESTRICT acc,
                      const xxh_u8* XXH_RESTRICT input, size_t len,
                      const xxh_u8* XXH_RESTRICT secret, size_t secretSize,
                            XXH3_accWidth_e accWidth)
{
    size_t const nb_rounds = (secretSize - STRIPE_LEN) / XXH_SECRET_CONSUME_RATE;
    size_t const block_len = STRIPE_LEN * nb_rounds;
    size_t const nb_blocks = len / block_len;

    size_t n;

    XXH_ASSERT(secretSize >= XXH3_SECRET_SIZE_MIN);

    for (n = 0; n < nb_blocks; n++) {
        XXH3_accumulate(acc, input + n*block_len, secret, nb_rounds, accWidth);
        XXH3_scrambleAcc(acc, secret + secretSize - STRIPE_LEN);
    }

    /* last partial block */
    XXH_ASSERT(len > STRIPE_LEN);
    {   size_t const nbStripes = (len - (block_len * nb_blocks)) / STRIPE_LEN;
        XXH_ASSERT(nbStripes <= (secretSize / XXH_SECRET_CONSUME_RATE));
        XXH3_accumulate(acc, input + nb_blocks*block_len, secret, nbStripes, accWidth);

        /* last stripe */
        if (len & (STRIPE_LEN - 1)) {
            const xxh_u8* const p = input + len - STRIPE_LEN;
            /* Do not align on 8, so that the secret is different from the scrambler */
#define XXH_SECRET_LASTACC_START 7
            XXH3_accumulate_512(acc, p, secret + secretSize - STRIPE_LEN - XXH_SECRET_LASTACC_START, accWidth);
    }   }
}

XXH_FORCE_INLINE xxh_u64
XXH3_mix2Accs(const xxh_u64* XXH_RESTRICT acc, const xxh_u8* XXH_RESTRICT secret)
{
    return XXH3_mul128_fold64(
               acc[0] ^ XXH_readLE64(secret),
               acc[1] ^ XXH_readLE64(secret+8) );
}

static XXH64_hash_t
XXH3_mergeAccs(const xxh_u64* XXH_RESTRICT acc, const xxh_u8* XXH_RESTRICT secret, xxh_u64 start)
{
    xxh_u64 result64 = start;

    result64 += XXH3_mix2Accs(acc+0, secret +  0);
    result64 += XXH3_mix2Accs(acc+2, secret + 16);
    result64 += XXH3_mix2Accs(acc+4, secret + 32);
    result64 += XXH3_mix2Accs(acc+6, secret + 48);

    return XXH3_avalanche(result64);
}

#define XXH3_INIT_ACC { PRIME32_3, PRIME64_1, PRIME64_2, PRIME64_3, \
                        PRIME64_4, PRIME32_2, PRIME64_5, PRIME32_1 };

XXH_FORCE_INLINE XXH64_hash_t
XXH3_hashLong_internal(const xxh_u8* XXH_RESTRICT input, size_t len,
                       const xxh_u8* XXH_RESTRICT secret, size_t secretSize)
{
    XXH_ALIGN(XXH_ACC_ALIGN) xxh_u64 acc[ACC_NB] = XXH3_INIT_ACC;

    XXH3_hashLong_internal_loop(acc, input, len, secret, secretSize, XXH3_acc_64bits);

    /* converge into final hash */
    XXH_STATIC_ASSERT(sizeof(acc) == 64);
    /* do not align on 8, so that the secret is different from the accumulator */
#define XXH_SECRET_MERGEACCS_START 11
    XXH_ASSERT(secretSize >= sizeof(acc) + XXH_SECRET_MERGEACCS_START);
    return XXH3_mergeAccs(acc, secret + XXH_SECRET_MERGEACCS_START, (xxh_u64)len * PRIME64_1);
}

/*
 * It's important for performance that XXH3_hashLong is not inlined. Not sure
 * why (uop cache maybe?), but the difference is large and easily measurable.
 */
XXH_NO_INLINE XXH64_hash_t
XXH3_hashLong_64b_defaultSecret(const xxh_u8* XXH_RESTRICT input, size_t len)
{
    return XXH3_hashLong_internal(input, len, kSecret, sizeof(kSecret));
}

/*
 * It's important for performance that XXH3_hashLong is not inlined. Not sure
 * why (uop cache maybe?), but the difference is large and easily measurable.
 */
XXH_NO_INLINE XXH64_hash_t
XXH3_hashLong_64b_withSecret(const xxh_u8* XXH_RESTRICT input, size_t len,
                             const xxh_u8* XXH_RESTRICT secret, size_t secretSize)
{
    return XXH3_hashLong_internal(input, len, secret, secretSize);
}


XXH_FORCE_INLINE void XXH_writeLE64(void* dst, xxh_u64 v64)
{
    if (!XXH_CPU_LITTLE_ENDIAN) v64 = XXH_swap64(v64);
    memcpy(dst, &v64, sizeof(v64));
}

/* XXH3_initCustomSecret() :
 * destination `customSecret` is presumed allocated and same size as `kSecret`.
 */
XXH_FORCE_INLINE void XXH3_initCustomSecret(xxh_u8* customSecret, xxh_u64 seed64)
{
    int const nbRounds = XXH_SECRET_DEFAULT_SIZE / 16;
    int i;

    XXH_STATIC_ASSERT((XXH_SECRET_DEFAULT_SIZE & 15) == 0);

    for (i=0; i < nbRounds; i++) {
        XXH_writeLE64(customSecret + 16*i,     XXH_readLE64(kSecret + 16*i)     + seed64);
        XXH_writeLE64(customSecret + 16*i + 8, XXH_readLE64(kSecret + 16*i + 8) - seed64);
    }
}


/*
 * XXH3_hashLong_64b_withSeed():
 * Generate a custom key based on alteration of default kSecret with the seed,
 * and then use this key for long mode hashing.
 *
 * This operation is decently fast but nonetheless costs a little bit of time.
 * Try to avoid it whenever possible (typically when seed==0).
 *
 * It's important for performance that XXH3_hashLong is not inlined. Not sure
 * why (uop cache maybe?), but the difference is large and easily measurable.
 */
XXH_NO_INLINE XXH64_hash_t
XXH3_hashLong_64b_withSeed(const xxh_u8* input, size_t len, XXH64_hash_t seed)
{
    XXH_ALIGN(8) xxh_u8 secret[XXH_SECRET_DEFAULT_SIZE];
    if (seed==0) return XXH3_hashLong_64b_defaultSecret(input, len);
    XXH3_initCustomSecret(secret, seed);
    return XXH3_hashLong_internal(input, len, secret, sizeof(secret));
}

/* ===   Public entry point   === */

XXH_PUBLIC_API XXH64_hash_t XXH3_64bits(const void* input, size_t len)
{
    if (len <= 16) return XXH3_len_0to16_64b((const xxh_u8*)input, len, kSecret, 0);
    if (len <= 128) return XXH3_len_17to128_64b((const xxh_u8*)input, len, kSecret, sizeof(kSecret), 0);
    if (len <= XXH3_MIDSIZE_MAX) return XXH3_len_129to240_64b((const xxh_u8*)input, len, kSecret, sizeof(kSecret), 0);
    return XXH3_hashLong_64b_defaultSecret((const xxh_u8*)input, len);
}

XXH_PUBLIC_API XXH64_hash_t
XXH3_64bits_withSecret(const void* input, size_t len, const void* secret, size_t secretSize)
{
    XXH_ASSERT(secretSize >= XXH3_SECRET_SIZE_MIN);
    /*
     * If an action is to be taken if `secret` conditions are not respected,
     * it should be done here.
     * For now, it's a contract pre-condition.
     * Adding a check and a branch here would cost performance at every hash.
     */
    if (len <= 16) return XXH3_len_0to16_64b((const xxh_u8*)input, len, (const xxh_u8*)secret, 0);
    if (len <= 128) return XXH3_len_17to128_64b((const xxh_u8*)input, len, (const xxh_u8*)secret, secretSize, 0);
    if (len <= XXH3_MIDSIZE_MAX) return XXH3_len_129to240_64b((const xxh_u8*)input, len, (const xxh_u8*)secret, secretSize, 0);
    return XXH3_hashLong_64b_withSecret((const xxh_u8*)input, len, (const xxh_u8*)secret, secretSize);
}

XXH_PUBLIC_API XXH64_hash_t
XXH3_64bits_withSeed(const void* input, size_t len, XXH64_hash_t seed)
{
    if (len <= 16) return XXH3_len_0to16_64b((const xxh_u8*)input, len, kSecret, seed);
    if (len <= 128) return XXH3_len_17to128_64b((const xxh_u8*)input, len, kSecret, sizeof(kSecret), seed);
    if (len <= XXH3_MIDSIZE_MAX) return XXH3_len_129to240_64b((const xxh_u8*)input, len, kSecret, sizeof(kSecret), seed);
    return XXH3_hashLong_64b_withSeed((const xxh_u8*)input, len, seed);
}

/* ===   XXH3 streaming   === */

XXH_PUBLIC_API XXH3_state_t* XXH3_createState(void)
{
    return (XXH3_state_t*)XXH_malloc(sizeof(XXH3_state_t));
}

XXH_PUBLIC_API XXH_errorcode XXH3_freeState(XXH3_state_t* statePtr)
{
    XXH_free(statePtr);
    return XXH_OK;
}

XXH_PUBLIC_API void
XXH3_copyState(XXH3_state_t* dst_state, const XXH3_state_t* src_state)
{
    memcpy(dst_state, src_state, sizeof(*dst_state));
}

static void
XXH3_64bits_reset_internal(XXH3_state_t* statePtr,
                           XXH64_hash_t seed,
                           const xxh_u8* secret, size_t secretSize)
{
    XXH_ASSERT(statePtr != NULL);
    memset(statePtr, 0, sizeof(*statePtr));
    statePtr->acc[0] = PRIME32_3;
    statePtr->acc[1] = PRIME64_1;
    statePtr->acc[2] = PRIME64_2;
    statePtr->acc[3] = PRIME64_3;
    statePtr->acc[4] = PRIME64_4;
    statePtr->acc[5] = PRIME32_2;
    statePtr->acc[6] = PRIME64_5;
    statePtr->acc[7] = PRIME32_1;
    statePtr->seed = seed;
    XXH_ASSERT(secret != NULL);
    statePtr->secret = secret;
    XXH_ASSERT(secretSize >= XXH3_SECRET_SIZE_MIN);
    statePtr->secretLimit = (XXH32_hash_t)(secretSize - STRIPE_LEN);
    statePtr->nbStripesPerBlock = statePtr->secretLimit / XXH_SECRET_CONSUME_RATE;
}

XXH_PUBLIC_API XXH_errorcode
XXH3_64bits_reset(XXH3_state_t* statePtr)
{
    if (statePtr == NULL) return XXH_ERROR;
    XXH3_64bits_reset_internal(statePtr, 0, kSecret, XXH_SECRET_DEFAULT_SIZE);
    return XXH_OK;
}

XXH_PUBLIC_API XXH_errorcode
XXH3_64bits_reset_withSecret(XXH3_state_t* statePtr, const void* secret, size_t secretSize)
{
    if (statePtr == NULL) return XXH_ERROR;
    XXH3_64bits_reset_internal(statePtr, 0, (const xxh_u8*)secret, secretSize);
    if (secret == NULL) return XXH_ERROR;
    if (secretSize < XXH3_SECRET_SIZE_MIN) return XXH_ERROR;
    return XXH_OK;
}

XXH_PUBLIC_API XXH_errorcode
XXH3_64bits_reset_withSeed(XXH3_state_t* statePtr, XXH64_hash_t seed)
{
    if (statePtr == NULL) return XXH_ERROR;
    XXH3_64bits_reset_internal(statePtr, seed, kSecret, XXH_SECRET_DEFAULT_SIZE);
    XXH3_initCustomSecret(statePtr->customSecret, seed);
    statePtr->secret = statePtr->customSecret;
    return XXH_OK;
}

XXH_FORCE_INLINE void
XXH3_consumeStripes( xxh_u64* acc,
                    XXH32_hash_t* nbStripesSoFarPtr, XXH32_hash_t nbStripesPerBlock,
                    const xxh_u8* input, size_t totalStripes,
                    const xxh_u8* secret, size_t secretLimit,
                    XXH3_accWidth_e accWidth)
{
    XXH_ASSERT(*nbStripesSoFarPtr < nbStripesPerBlock);
    if (nbStripesPerBlock - *nbStripesSoFarPtr <= totalStripes) {
        /* need a scrambling operation */
        size_t const nbStripes = nbStripesPerBlock - *nbStripesSoFarPtr;
        XXH3_accumulate(acc, input, secret + nbStripesSoFarPtr[0] * XXH_SECRET_CONSUME_RATE, nbStripes, accWidth);
        XXH3_scrambleAcc(acc, secret + secretLimit);
        XXH3_accumulate(acc, input + nbStripes * STRIPE_LEN, secret, totalStripes - nbStripes, accWidth);
        *nbStripesSoFarPtr = (XXH32_hash_t)(totalStripes - nbStripes);
    } else {
        XXH3_accumulate(acc, input, secret + nbStripesSoFarPtr[0] * XXH_SECRET_CONSUME_RATE, totalStripes, accWidth);
        *nbStripesSoFarPtr += (XXH32_hash_t)totalStripes;
    }
}

/*
 * Both XXH3_64bits_update and XXH3_128bits_update use this routine.
 */
XXH_FORCE_INLINE XXH_errorcode
XXH3_update(XXH3_state_t* state, const xxh_u8* input, size_t len, XXH3_accWidth_e accWidth)
{
    if (input==NULL)
#if defined(XXH_ACCEPT_NULL_INPUT_POINTER) && (XXH_ACCEPT_NULL_INPUT_POINTER>=1)
        return XXH_OK;
#else
        return XXH_ERROR;
#endif

    {   const xxh_u8* const bEnd = input + len;

        state->totalLen += len;

        if (state->bufferedSize + len <= XXH3_INTERNALBUFFER_SIZE) {  /* fill in tmp buffer */
            XXH_memcpy(state->buffer + state->bufferedSize, input, len);
            state->bufferedSize += (XXH32_hash_t)len;
            return XXH_OK;
        }
        /* input is now > XXH3_INTERNALBUFFER_SIZE */

        #define XXH3_INTERNALBUFFER_STRIPES (XXH3_INTERNALBUFFER_SIZE / STRIPE_LEN)
        XXH_STATIC_ASSERT(XXH3_INTERNALBUFFER_SIZE % STRIPE_LEN == 0);   /* clean multiple */

        /*
         * There is some input left inside the internal buffer.
         * Fill it, then consume it.
         */
        if (state->bufferedSize) {
            size_t const loadSize = XXH3_INTERNALBUFFER_SIZE - state->bufferedSize;
            XXH_memcpy(state->buffer + state->bufferedSize, input, loadSize);
            input += loadSize;
            XXH3_consumeStripes(state->acc,
                               &state->nbStripesSoFar, state->nbStripesPerBlock,
                                state->buffer, XXH3_INTERNALBUFFER_STRIPES,
                                state->secret, state->secretLimit,
                                accWidth);
            state->bufferedSize = 0;
        }

        /* Consume input by full buffer quantities */
        if (input+XXH3_INTERNALBUFFER_SIZE <= bEnd) {
            const xxh_u8* const limit = bEnd - XXH3_INTERNALBUFFER_SIZE;
            do {
                XXH3_consumeStripes(state->acc,
                                   &state->nbStripesSoFar, state->nbStripesPerBlock,
                                    input, XXH3_INTERNALBUFFER_STRIPES,
                                    state->secret, state->secretLimit,
                                    accWidth);
                input += XXH3_INTERNALBUFFER_SIZE;
            } while (input<=limit);
        }

        if (input < bEnd) { /* Some remaining input: buffer it */
            XXH_memcpy(state->buffer, input, (size_t)(bEnd-input));
            state->bufferedSize = (XXH32_hash_t)(bEnd-input);
        }
    }

    return XXH_OK;
}

XXH_PUBLIC_API XXH_errorcode
XXH3_64bits_update(XXH3_state_t* state, const void* input, size_t len)
{
    return XXH3_update(state, (const xxh_u8*)input, len, XXH3_acc_64bits);
}


XXH_FORCE_INLINE void
XXH3_digest_long (XXH64_hash_t* acc, const XXH3_state_t* state, XXH3_accWidth_e accWidth)
{
    /*
     * Digest on a local copy. This way, the state remains unaltered, and it can
     * continue ingesting more input afterwards.
     */
    memcpy(acc, state->acc, sizeof(state->acc));
    if (state->bufferedSize >= STRIPE_LEN) {
        size_t const totalNbStripes = state->bufferedSize / STRIPE_LEN;
        XXH32_hash_t nbStripesSoFar = state->nbStripesSoFar;
        XXH3_consumeStripes(acc,
                           &nbStripesSoFar, state->nbStripesPerBlock,
                            state->buffer, totalNbStripes,
                            state->secret, state->secretLimit,
                            accWidth);
        if (state->bufferedSize % STRIPE_LEN) {  /* one last partial stripe */
            XXH3_accumulate_512(acc,
                                state->buffer + state->bufferedSize - STRIPE_LEN,
                                state->secret + state->secretLimit - XXH_SECRET_LASTACC_START,
                                accWidth);
        }
    } else {  /* bufferedSize < STRIPE_LEN */
        if (state->bufferedSize) { /* one last stripe */
            xxh_u8 lastStripe[STRIPE_LEN];
            size_t const catchupSize = STRIPE_LEN - state->bufferedSize;
            memcpy(lastStripe, state->buffer + sizeof(state->buffer) - catchupSize, catchupSize);
            memcpy(lastStripe + catchupSize, state->buffer, state->bufferedSize);
            XXH3_accumulate_512(acc,
                                lastStripe,
                                state->secret + state->secretLimit - XXH_SECRET_LASTACC_START,
                                accWidth);
    }   }
}

XXH_PUBLIC_API XXH64_hash_t XXH3_64bits_digest (const XXH3_state_t* state)
{
    if (state->totalLen > XXH3_MIDSIZE_MAX) {
        XXH_ALIGN(XXH_ACC_ALIGN) XXH64_hash_t acc[ACC_NB];
        XXH3_digest_long(acc, state, XXH3_acc_64bits);
        return XXH3_mergeAccs(acc, state->secret + XXH_SECRET_MERGEACCS_START, (xxh_u64)state->totalLen * PRIME64_1);
    }
    /* len <= XXH3_MIDSIZE_MAX : short code */
    if (state->seed)
        return XXH3_64bits_withSeed(state->buffer, (size_t)state->totalLen, state->seed);
    return XXH3_64bits_withSecret(state->buffer, (size_t)(state->totalLen), state->secret, state->secretLimit + STRIPE_LEN);
}

/* ==========================================
 * XXH3 128 bits (=> XXH128)
 * ========================================== */

XXH_FORCE_INLINE XXH128_hash_t
XXH3_len_1to3_128b(const xxh_u8* input, size_t len, const xxh_u8* secret, XXH64_hash_t seed)
{
    XXH_ASSERT(input != NULL);
    XXH_ASSERT(1 <= len && len <= 3);
    XXH_ASSERT(secret != NULL);
    /*
     * len = 1: combinedl = { input[0], 0x01, input[0], input[0] }
     * len = 2: combinedl = { input[1], 0x02, input[0], input[1] }
     * len = 3: combinedl = { input[2], 0x03, input[0], input[1] }
     */
    {   xxh_u8 const c1 = input[0];
        xxh_u8 const c2 = input[len >> 1];
        xxh_u8 const c3 = input[len - 1];
        xxh_u32 const combinedl = ((xxh_u32)c1<<16) | (((xxh_u32)c2) << 24) | (((xxh_u32)c3) << 0) | (((xxh_u32)len) << 8);
        xxh_u32 const combinedh = XXH_rotl32(XXH_swap32(combinedl), 13);
        xxh_u64 const bitflipl = (XXH_readLE32(secret) ^ XXH_readLE32(secret+4)) + seed;
        xxh_u64 const bitfliph = (XXH_readLE32(secret+8) ^ XXH_readLE32(secret+12)) - seed;
        xxh_u64 const keyed_lo = (xxh_u64)combinedl ^ bitflipl;
        xxh_u64 const keyed_hi = (xxh_u64)combinedh ^ bitfliph;
        xxh_u64 const mixedl = keyed_lo * PRIME64_1;
        xxh_u64 const mixedh = keyed_hi * PRIME64_5;
        XXH128_hash_t const h128 = { XXH3_avalanche(mixedl) /*low64*/, XXH3_avalanche(mixedh) /*high64*/ };
        return h128;
    }
}

XXH_FORCE_INLINE XXH128_hash_t
XXH3_len_4to8_128b(const xxh_u8* input, size_t len, const xxh_u8* secret, XXH64_hash_t seed)
{
    XXH_ASSERT(input != NULL);
    XXH_ASSERT(secret != NULL);
    XXH_ASSERT(4 <= len && len <= 8);
    seed ^= (xxh_u64)XXH_swap32((xxh_u32)seed) << 32;
    {   xxh_u32 const input_lo = XXH_readLE32(input);
        xxh_u32 const input_hi = XXH_readLE32(input + len - 4);
        xxh_u64 const input_64 = input_lo + ((xxh_u64)input_hi << 32);
        xxh_u64 const bitflip = (XXH_readLE64(secret+16) ^ XXH_readLE64(secret+24)) + seed;
        xxh_u64 const keyed = input_64 ^ bitflip;

        /* Shift len to the left to ensure it is even, this avoids even multiplies. */
        XXH128_hash_t m128 = XXH_mult64to128(keyed, PRIME64_1 + (len << 2));

        m128.high64 += (m128.low64 << 1);
        m128.low64  ^= (m128.high64 >> 3);

        m128.low64   = XXH_xorshift64(m128.low64, 35);
        m128.low64  *= 0x9FB21C651E98DF25ULL;
        m128.low64   = XXH_xorshift64(m128.low64, 28);
        m128.high64  = XXH3_avalanche(m128.high64);
        return m128;
    }
}

XXH_FORCE_INLINE XXH128_hash_t
XXH3_len_9to16_128b(const xxh_u8* input, size_t len, const xxh_u8* secret, XXH64_hash_t seed)
{
    XXH_ASSERT(input != NULL);
    XXH_ASSERT(secret != NULL);
    XXH_ASSERT(9 <= len && len <= 16);
    {   xxh_u64 const bitflipl = (XXH_readLE64(secret+32) ^ XXH_readLE64(secret+40)) - seed;
        xxh_u64 const bitfliph = (XXH_readLE64(secret+48) ^ XXH_readLE64(secret+56)) + seed;
        xxh_u64 const input_lo = XXH_readLE64(input);
        xxh_u64       input_hi = XXH_readLE64(input + len - 8);
        XXH128_hash_t m128 = XXH_mult64to128(input_lo ^ input_hi ^ bitflipl, PRIME64_1);
        /*
         * Put len in the middle of m128 to ensure that the length gets mixed to
         * both the low and high bits in the 128x64 multiply below.
         */
        m128.low64  += (xxh_u64)(len - 1) << 54;
        input_hi ^= bitfliph;
        /*
         * Add the high 32 bits of input_hi to the high 32 bits of m128, then
         * add the long product of the low 32 bits of input_hi and PRIME32_2 to
         * the high 64 bits of m128.
         *
         * The best approach to this operation is different on 32-bit and 64-bit.
         */
        if (sizeof(void *) < sizeof(xxh_u64)) { /* 32-bit */
            /*
             * 32-bit optimized version, which is more readable.
             *
             * On 32-bit, it removes an ADC and delays a dependency between the two
             * halves of m128.high64, but it generates an extra mask on 64-bit.
             */
            m128.high64 += (input_hi & 0xFFFFFFFF00000000) + XXH_mult32to64((xxh_u32)input_hi, PRIME32_2);
        } else {
            /*
             * 64-bit optimized (albeit more confusing) version.
             *
             * Uses some properties of addition and multiplication to remove the mask:
             *
             * Let:
             *    a = input_hi.lo = (input_hi & 0x00000000FFFFFFFF)
             *    b = input_hi.hi = (input_hi & 0xFFFFFFFF00000000)
             *    c = PRIME32_2
             *
             *    a + (b * c)
             * Inverse Property: x + y - x == y
             *    a + (b * (1 + c - 1))
             * Distributive Property: x * (y + z) == (x * y) + (x * z)
             *    a + (b * 1) + (b * (c - 1))
             * Identity Property: x * 1 == x
             *    a + b + (b * (c - 1))
             *
             * Substitute a, b, and c:
             *    input_hi.hi + input_hi.lo + ((xxh_u64)input_hi.lo * (PRIME32_2 - 1))
             *
             * Since input_hi.hi + input_hi.lo == input_hi, we get this:
             *    input_hi + ((xxh_u64)input_hi.lo * (PRIME32_2 - 1))
             */
            m128.high64 += input_hi + XXH_mult32to64((xxh_u32)input_hi, PRIME32_2 - 1);
        }
        /* m128 ^= XXH_swap64(m128 >> 64); */
        m128.low64  ^= XXH_swap64(m128.high64);

        {   /* 128x64 multiply: h128 = m128 * PRIME64_2; */
            XXH128_hash_t h128 = XXH_mult64to128(m128.low64, PRIME64_2);
            h128.high64 += m128.high64 * PRIME64_2;

            h128.low64   = XXH3_avalanche(h128.low64);
            h128.high64  = XXH3_avalanche(h128.high64);
            return h128;
    }   }
}

/* Assumption : `secret` size is >= 16
 * Note : it should be >= XXH3_SECRET_SIZE_MIN anyway */
XXH_FORCE_INLINE XXH128_hash_t
XXH3_len_0to16_128b(const xxh_u8* input, size_t len, const xxh_u8* secret, XXH64_hash_t seed)
{
    XXH_ASSERT(len <= 16);
    {   if (len > 8) return XXH3_len_9to16_128b(input, len, secret, seed);
        if (len >= 4) return XXH3_len_4to8_128b(input, len, secret, seed);
        if (len) return XXH3_len_1to3_128b(input, len, secret, seed);
        {   XXH128_hash_t h128;
            xxh_u64 const bitflipl = XXH_readLE64(secret+64) ^ XXH_readLE64(secret+72);
            xxh_u64 const bitfliph = XXH_readLE64(secret+80) ^ XXH_readLE64(secret+88);
            h128.low64 = XXH3_avalanche((PRIME64_1 + seed) ^ bitflipl);
            h128.high64 = XXH3_avalanche((PRIME64_2 - seed) ^ bitfliph);
            return h128;
    }   }
}

/*
 * A bit slower than XXH3_mix16B, but handles multiply by zero better.
 */
XXH_FORCE_INLINE XXH128_hash_t
XXH128_mix32B(XXH128_hash_t acc, const xxh_u8* input_1, const xxh_u8* input_2, const xxh_u8* secret, XXH64_hash_t seed)
{
    acc.low64  += XXH3_mix16B (input_1, secret+0, seed);
    acc.low64  ^= XXH_readLE64(input_2) + XXH_readLE64(input_2 + 8);
    acc.high64 += XXH3_mix16B (input_2, secret+16, seed);
    acc.high64 ^= XXH_readLE64(input_1) + XXH_readLE64(input_1 + 8);
    return acc;
}


XXH_FORCE_INLINE XXH128_hash_t
XXH3_len_17to128_128b(const xxh_u8* XXH_RESTRICT input, size_t len,
                      const xxh_u8* XXH_RESTRICT secret, size_t secretSize,
                      XXH64_hash_t seed)
{
    XXH_ASSERT(secretSize >= XXH3_SECRET_SIZE_MIN); (void)secretSize;
    XXH_ASSERT(16 < len && len <= 128);

    {   XXH128_hash_t acc;
        acc.low64 = len * PRIME64_1;
        acc.high64 = 0;
        if (len > 32) {
            if (len > 64) {
                if (len > 96) {
                    acc = XXH128_mix32B(acc, input+48, input+len-64, secret+96, seed);
                }
                acc = XXH128_mix32B(acc, input+32, input+len-48, secret+64, seed);
            }
            acc = XXH128_mix32B(acc, input+16, input+len-32, secret+32, seed);
        }
        acc = XXH128_mix32B(acc, input, input+len-16, secret, seed);
        {   xxh_u64 const low64 = acc.low64 + acc.high64;
            xxh_u64 const high64 = (acc.low64 * PRIME64_1) + (acc.high64 * PRIME64_4) + ((len - seed) * PRIME64_2);
            XXH128_hash_t const h128 = { XXH3_avalanche(low64), (XXH64_hash_t)0 - XXH3_avalanche(high64) };
            return h128;
        }
    }
}

XXH_NO_INLINE XXH128_hash_t
XXH3_len_129to240_128b(const xxh_u8* XXH_RESTRICT input, size_t len,
                       const xxh_u8* XXH_RESTRICT secret, size_t secretSize,
                       XXH64_hash_t seed)
{
    XXH_ASSERT(secretSize >= XXH3_SECRET_SIZE_MIN); (void)secretSize;
    XXH_ASSERT(128 < len && len <= XXH3_MIDSIZE_MAX);

    {   XXH128_hash_t acc;
        int const nbRounds = (int)len / 32;
        int i;
        acc.low64 = len * PRIME64_1;
        acc.high64 = 0;
        for (i=0; i<4; i++) {
            acc = XXH128_mix32B(acc, input+(32*i), input+(32*i)+16, secret+(32*i), seed);
        }
        acc.low64 = XXH3_avalanche(acc.low64);
        acc.high64 = XXH3_avalanche(acc.high64);
        XXH_ASSERT(nbRounds >= 4);
        for (i=4 ; i < nbRounds; i++) {
            acc = XXH128_mix32B(acc, input+(32*i), input+(32*i)+16, secret+XXH3_MIDSIZE_STARTOFFSET+(32*(i-4)), seed);
        }
        /* last bytes */
        acc = XXH128_mix32B(acc, input + len - 16, input + len - 32, secret + XXH3_SECRET_SIZE_MIN - XXH3_MIDSIZE_LASTOFFSET - 16, 0ULL - seed);

        {   xxh_u64 const low64 = acc.low64 + acc.high64;
            xxh_u64 const high64 = (acc.low64 * PRIME64_1) + (acc.high64 * PRIME64_4) + ((len - seed) * PRIME64_2);
            XXH128_hash_t const h128 = { XXH3_avalanche(low64), (XXH64_hash_t)0 - XXH3_avalanche(high64) };
            return h128;
        }
    }
}

XXH_FORCE_INLINE XXH128_hash_t
XXH3_hashLong_128b_internal(const xxh_u8* XXH_RESTRICT input, size_t len,
                            const xxh_u8* XXH_RESTRICT secret, size_t secretSize)
{
    XXH_ALIGN(XXH_ACC_ALIGN) xxh_u64 acc[ACC_NB] = XXH3_INIT_ACC;

    XXH3_hashLong_internal_loop(acc, input, len, secret, secretSize, XXH3_acc_128bits);

    /* converge into final hash */
    XXH_STATIC_ASSERT(sizeof(acc) == 64);
    XXH_ASSERT(secretSize >= sizeof(acc) + XXH_SECRET_MERGEACCS_START);
    {   xxh_u64 const low64 = XXH3_mergeAccs(acc, secret + XXH_SECRET_MERGEACCS_START, (xxh_u64)len * PRIME64_1);
        xxh_u64 const high64 = XXH3_mergeAccs(acc, secret + secretSize - sizeof(acc) - XXH_SECRET_MERGEACCS_START, ~((xxh_u64)len * PRIME64_2));
        XXH128_hash_t const h128 = { low64, high64 };
        return h128;
    }
}

/*
 * It's important for performance that XXH3_hashLong is not inlined. Not sure
 * why (uop cache maybe?), but the difference is large and easily measurable.
 */
XXH_NO_INLINE XXH128_hash_t
XXH3_hashLong_128b_defaultSecret(const xxh_u8* input, size_t len)
{
    return XXH3_hashLong_128b_internal(input, len, kSecret, sizeof(kSecret));
}

/*
 * It's important for performance that XXH3_hashLong is not inlined. Not sure
 * why (uop cache maybe?), but the difference is large and easily measurable.
 */
XXH_NO_INLINE XXH128_hash_t
XXH3_hashLong_128b_withSecret(const xxh_u8* input, size_t len,
                              const xxh_u8* secret, size_t secretSize)
{
    return XXH3_hashLong_128b_internal(input, len, secret, secretSize);
}

/*
 * It's important for performance that XXH3_hashLong is not inlined. Not sure
 * why (uop cache maybe?), but the difference is large and easily measurable.
 */
XXH_NO_INLINE XXH128_hash_t
XXH3_hashLong_128b_withSeed(const xxh_u8* input, size_t len, XXH64_hash_t seed)
{
    XXH_ALIGN(8) xxh_u8 secret[XXH_SECRET_DEFAULT_SIZE];
    if (seed == 0) return XXH3_hashLong_128b_defaultSecret(input, len);
    XXH3_initCustomSecret(secret, seed);
    return XXH3_hashLong_128b_internal(input, len, secret, sizeof(secret));
}


XXH_PUBLIC_API XXH128_hash_t XXH3_128bits(const void* input, size_t len)
{
    if (len <= 16) return XXH3_len_0to16_128b((const xxh_u8*)input, len, kSecret, 0);
    if (len <= 128) return XXH3_len_17to128_128b((const xxh_u8*)input, len, kSecret, sizeof(kSecret), 0);
    if (len <= XXH3_MIDSIZE_MAX) return XXH3_len_129to240_128b((const xxh_u8*)input, len, kSecret, sizeof(kSecret), 0);
    return XXH3_hashLong_128b_defaultSecret((const xxh_u8*)input, len);
}

XXH_PUBLIC_API XXH128_hash_t
XXH3_128bits_withSecret(const void* input, size_t len, const void* secret, size_t secretSize)
{
    XXH_ASSERT(secretSize >= XXH3_SECRET_SIZE_MIN);
    /*
     * If an action is to be taken if `secret` conditions are not respected,
     * it should be done here.
     * For now, it's a contract pre-condition.
     * Adding a check and a branch here would cost performance at every hash.
     */
     if (len <= 16) return XXH3_len_0to16_128b((const xxh_u8*)input, len, (const xxh_u8*)secret, 0);
     if (len <= 128) return XXH3_len_17to128_128b((const xxh_u8*)input, len, (const xxh_u8*)secret, secretSize, 0);
     if (len <= XXH3_MIDSIZE_MAX) return XXH3_len_129to240_128b((const xxh_u8*)input, len, (const xxh_u8*)secret, secretSize, 0);
     return XXH3_hashLong_128b_withSecret((const xxh_u8*)input, len, (const xxh_u8*)secret, secretSize);
}

XXH_PUBLIC_API XXH128_hash_t
XXH3_128bits_withSeed(const void* input, size_t len, XXH64_hash_t seed)
{
    if (len <= 16) return XXH3_len_0to16_128b((const xxh_u8*)input, len, kSecret, seed);
    if (len <= 128) return XXH3_len_17to128_128b((const xxh_u8*)input, len, kSecret, sizeof(kSecret), seed);
    if (len <= XXH3_MIDSIZE_MAX) return XXH3_len_129to240_128b((const xxh_u8*)input, len, kSecret, sizeof(kSecret), seed);
    return XXH3_hashLong_128b_withSeed((const xxh_u8*)input, len, seed);
}

XXH_PUBLIC_API XXH128_hash_t
XXH128(const void* input, size_t len, XXH64_hash_t seed)
{
    return XXH3_128bits_withSeed(input, len, seed);
}


/* ===   XXH3 128-bit streaming   === */

/* all the functions are actually the same as for 64-bit streaming variant,
   just the reset one is different (different initial acc values for 0,5,6,7),
   and near the end of the digest function */

static void
XXH3_128bits_reset_internal(XXH3_state_t* statePtr,
                           XXH64_hash_t seed,
                           const xxh_u8* secret, size_t secretSize)
{
    XXH3_64bits_reset_internal(statePtr, seed, secret, secretSize);
}

XXH_PUBLIC_API XXH_errorcode
XXH3_128bits_reset(XXH3_state_t* statePtr)
{
    if (statePtr == NULL) return XXH_ERROR;
    XXH3_128bits_reset_internal(statePtr, 0, kSecret, XXH_SECRET_DEFAULT_SIZE);
    return XXH_OK;
}

XXH_PUBLIC_API XXH_errorcode
XXH3_128bits_reset_withSecret(XXH3_state_t* statePtr, const void* secret, size_t secretSize)
{
    if (statePtr == NULL) return XXH_ERROR;
    XXH3_128bits_reset_internal(statePtr, 0, (const xxh_u8*)secret, secretSize);
    if (secret == NULL) return XXH_ERROR;
    if (secretSize < XXH3_SECRET_SIZE_MIN) return XXH_ERROR;
    return XXH_OK;
}

XXH_PUBLIC_API XXH_errorcode
XXH3_128bits_reset_withSeed(XXH3_state_t* statePtr, XXH64_hash_t seed)
{
    if (statePtr == NULL) return XXH_ERROR;
    XXH3_128bits_reset_internal(statePtr, seed, kSecret, XXH_SECRET_DEFAULT_SIZE);
    XXH3_initCustomSecret(statePtr->customSecret, seed);
    statePtr->secret = statePtr->customSecret;
    return XXH_OK;
}

XXH_PUBLIC_API XXH_errorcode
XXH3_128bits_update(XXH3_state_t* state, const void* input, size_t len)
{
    return XXH3_update(state, (const xxh_u8*)input, len, XXH3_acc_128bits);
}

XXH_PUBLIC_API XXH128_hash_t XXH3_128bits_digest (const XXH3_state_t* state)
{
    if (state->totalLen > XXH3_MIDSIZE_MAX) {
        XXH_ALIGN(XXH_ACC_ALIGN) XXH64_hash_t acc[ACC_NB];
        XXH3_digest_long(acc, state, XXH3_acc_128bits);
        XXH_ASSERT(state->secretLimit + STRIPE_LEN >= sizeof(acc) + XXH_SECRET_MERGEACCS_START);
        {   xxh_u64 const low64 = XXH3_mergeAccs(acc, state->secret + XXH_SECRET_MERGEACCS_START, (xxh_u64)state->totalLen * PRIME64_1);
            xxh_u64 const high64 = XXH3_mergeAccs(acc, state->secret + state->secretLimit + STRIPE_LEN - sizeof(acc) - XXH_SECRET_MERGEACCS_START, ~((xxh_u64)state->totalLen * PRIME64_2));
            XXH128_hash_t const h128 = { low64, high64 };
            return h128;
        }
    }
    /* len <= XXH3_MIDSIZE_MAX : short code */
    if (state->seed)
        return XXH3_128bits_withSeed(state->buffer, (size_t)state->totalLen, state->seed);
    return XXH3_128bits_withSecret(state->buffer, (size_t)(state->totalLen), state->secret, state->secretLimit + STRIPE_LEN);
}

/* 128-bit utility functions */

#include <string.h>   /* memcmp, memcpy */

/* return : 1 is equal, 0 if different */
XXH_PUBLIC_API int XXH128_isEqual(XXH128_hash_t h1, XXH128_hash_t h2)
{
    /* note : XXH128_hash_t is compact, it has no padding byte */
    return !(memcmp(&h1, &h2, sizeof(h1)));
}

/* This prototype is compatible with stdlib's qsort().
 * return : >0 if *h128_1  > *h128_2
 *          <0 if *h128_1  < *h128_2
 *          =0 if *h128_1 == *h128_2  */
XXH_PUBLIC_API int XXH128_cmp(const void* h128_1, const void* h128_2)
{
    XXH128_hash_t const h1 = *(const XXH128_hash_t*)h128_1;
    XXH128_hash_t const h2 = *(const XXH128_hash_t*)h128_2;
    int const hcmp = (h1.high64 > h2.high64) - (h2.high64 > h1.high64);
    /* note : bets that, in most cases, hash values are different */
    if (hcmp) return hcmp;
    return (h1.low64 > h2.low64) - (h2.low64 > h1.low64);
}


/*======   Canonical representation   ======*/
XXH_PUBLIC_API void
XXH128_canonicalFromHash(XXH128_canonical_t* dst, XXH128_hash_t hash)
{
    XXH_STATIC_ASSERT(sizeof(XXH128_canonical_t) == sizeof(XXH128_hash_t));
    if (XXH_CPU_LITTLE_ENDIAN) {
        hash.high64 = XXH_swap64(hash.high64);
        hash.low64  = XXH_swap64(hash.low64);
    }
    memcpy(dst, &hash.high64, sizeof(hash.high64));
    memcpy((char*)dst + sizeof(hash.high64), &hash.low64, sizeof(hash.low64));
}

XXH_PUBLIC_API XXH128_hash_t
XXH128_hashFromCanonical(const XXH128_canonical_t* src)
{
    XXH128_hash_t h;
    h.high64 = XXH_readBE64(src);
    h.low64  = XXH_readBE64(src->digest + 8);
    return h;
}

/* Pop our optimization override from above */
#if XXH_VECTOR == XXH_AVX2 /* AVX2 */ \
  && defined(__GNUC__) && !defined(__clang__) /* GCC, not Clang */ \
  && defined(__OPTIMIZE__) && !defined(__OPTIMIZE_SIZE__) /* respect -O0 and -Os */
#  pragma GCC pop_options
#endif

#endif  /* XXH3_H_1397135465 */
