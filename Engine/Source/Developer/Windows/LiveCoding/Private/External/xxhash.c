/*
 * xxHash - Extremely Fast Hash algorithm
 * Copyright (C) 2012-present, Yann Collet
 *
 * BSD 2-Clause License (https://www.opensource.org/licenses/bsd-license.php)
 *
 * Redistribution and use in source and binary forms, with or without
 * modification, are permitted provided that the following conditions are
 * met:
 *
 *    * Redistributions of source code must retain the above copyright
 *      notice, this list of conditions and the following disclaimer.
 *    * Redistributions in binary form must reproduce the above
 *      copyright notice, this list of conditions and the following disclaimer
 *      in the documentation and/or other materials provided with the
 *      distribution.
 *
 * THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS
 * "AS IS" AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT
 * LIMITED TO, THE IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR
 * A PARTICULAR PURPOSE ARE DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT
 * OWNER OR CONTRIBUTORS BE LIABLE FOR ANY DIRECT, INDIRECT, INCIDENTAL,
 * SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES (INCLUDING, BUT NOT
 * LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES; LOSS OF USE,
 * DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON ANY
 * THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
 * (INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE
 * OF THIS SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
 *
 * You can contact the author at:
 *   - xxHash homepage: https://www.xxhash.com
 *   - xxHash source repository: https://github.com/Cyan4973/xxHash
 */


/*
 * xxhash.c instantiates functions defined in xxhash.h
 */


#define XXH_STATIC_LINKING_ONLY   /* access advanced declarations */
#define XXH_IMPLEMENTATION   /* access definitions */

 // BEGIN EPIC MODS
#pragma warning(push)
#pragma warning(disable:6385) // warning C6385: Reading invalid data from '&state'
#pragma warning(disable:4668) // warning C4668: '__cplusplus' is not defined as a preprocessor macro, replacing with '0' for '#if/#elif'
#pragma warning(disable:28251) // warning C28251: Inconsistent annotation for '_setjmp'
// END EPIC MODS

#include "xxhash.h"

// BEGIN EPIC MODS
#pragma warning(pop)
// END EPIC MODS
