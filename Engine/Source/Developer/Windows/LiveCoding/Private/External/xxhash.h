/*
 * xxHash - Extremely Fast Hash algorithm
 * Header File
 * Copyright (C) 2012-present, Yann Collet.
 *
 * BSD 2-Clause License (https://www.opensource.org/licenses/bsd-license.php)
 *
 * Redistribution and use in source and binary forms, with or without
 * modification, are permitted provided that the following conditions are
 * met:
 *
 *    * Redistributions of source code must retain the above copyright
 *      notice, this list of conditions and the following disclaimer.
 *    * Redistributions in binary form must reproduce the above
 *      copyright notice, this list of conditions and the following disclaimer
 *      in the documentation and/or other materials provided with the
 *      distribution.
 *
 * THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS
 * "AS IS" AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT
 * LIMITED TO, THE IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR
 * A PARTICULAR PURPOSE ARE DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT
 * OWNER OR CONTRIBUTORS BE LIABLE FOR ANY DIRECT, INDIRECT, INCIDENTAL,
 * SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES (INCLUDING, BUT NOT
 * LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES; LOSS OF USE,
 * DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON ANY
 * THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
 * (INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE
 * OF THIS SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
 *
 * You can contact the author at:
 *   - xxHash homepage: https://www.xxhash.com
 *   - xxHash source repository: https://github.com/Cyan4973/xxHash
 */

/* TODO: update */
/* Notice extracted from xxHash homepage:

xxHash is an extremely fast hash algorithm, running at RAM speed limits.
It also successfully passes all tests from the SMHasher suite.

Comparison (single thread, Windows Seven 32 bits, using SMHasher on a Core 2 Duo @3GHz)

Name            Speed       Q.Score   Author
xxHash          5.4 GB/s     10
CrapWow         3.2 GB/s      2       Andrew
MumurHash 3a    2.7 GB/s     10       Austin Appleby
SpookyHash      2.0 GB/s     10       Bob Jenkins
SBox            1.4 GB/s      9       Bret Mulvey
Lookup3         1.2 GB/s      9       Bob Jenkins
SuperFastHash   1.2 GB/s      1       Paul Hsieh
CityHash64      1.05 GB/s    10       Pike & Alakuijala
FNV             0.55 GB/s     5       Fowler, Noll, Vo
CRC32           0.43 GB/s     9
MD5-32          0.33 GB/s    10       Ronald L. Rivest
SHA1-32         0.28 GB/s    10

Q.Score is a measure of quality of the hash function.
It depends on successfully passing SMHasher test set.
10 is a perfect score.

Note: SMHasher's CRC32 implementation is not the fastest one.
Other speed-oriented implementations can be faster,
especially in combination with PCLMUL instruction:
https://fastcompression.blogspot.com/2019/03/presenting-xxh3.html?showComment=1552696407071#c3490092340461170735

A 64-bit version, named XXH64, is available since r35.
It offers much better speed, but for 64-bit applications only.
Name     Speed on 64 bits    Speed on 32 bits
XXH64       13.8 GB/s            1.9 GB/s
XXH32        6.8 GB/s            6.0 GB/s
*/

// MM: tuning
#define XXH_FORCE_MEMORY_ACCESS 2
#define XXH_FORCE_ALIGN_CHECK 0
#define XXH_CPU_LITTLE_ENDIAN 1
#define XXH_INLINE_ALL

#if defined (__cplusplus)
extern "C" {
#endif

/* ****************************
 *  INLINE mode
 ******************************/
/*!
 * XXH_INLINE_ALL (and XXH_PRIVATE_API)
 * Use these build macros to inline xxhash into the target unit.
 * Inlining improves performance on small inputs, especially when the length is
 * expressed as a compile-time constant:
 *
 *      https://fastcompression.blogspot.com/2018/03/xxhash-for-small-keys-impressive-power.html
 *
 * It also keeps xxHash symbols private to the unit, so they are not exported.
 *
 * Usage:
 *     #define XXH_INLINE_ALL
 *     #include "xxhash.h"
 *
 * Do not compile and link xxhash.o as a separate object, as it is not useful.
 */
#if (defined(XXH_INLINE_ALL) || defined(XXH_PRIVATE_API)) \
    && !defined(XXH_INLINE_ALL_31684351384)
   /* this section should be traversed only once */
#  define XXH_INLINE_ALL_31684351384
   /* give access to the advanced API, required to compile implementations */
#  undef XXH_STATIC_LINKING_ONLY   /* avoid macro redef */
#  define XXH_STATIC_LINKING_ONLY
   /* make all functions private */
#  undef XXH_PUBLIC_API
#  if defined(__GNUC__)
#    define XXH_PUBLIC_API static __inline __attribute__((unused))
#  elif defined (__cplusplus) || (defined (__STDC_VERSION__) && (__STDC_VERSION__ >= 199901L) /* C99 */)
#    define XXH_PUBLIC_API static inline
#  elif defined(_MSC_VER)
#    define XXH_PUBLIC_API static __inline
#  else
     /* note: this version may generate warnings for unused static functions */
#    define XXH_PUBLIC_API static
#  endif

   /*
    * This part deals with the special case where a unit wants to inline xxHash,
    * but "xxhash.h" has previously been included without XXH_INLINE_ALL, such
    * as part of some previously included *.h header file.
    * Without further action, the new include would just be ignored,
    * and functions would effectively _not_ be inlined (silent failure).
    * The following macros solve this situation by prefixing all inlined names,
    * avoiding naming collision with previous inclusions.
    */
#  ifdef XXH_NAMESPACE
#    error "XXH_INLINE_ALL with XXH_NAMESPACE is not supported"
     /*
      * Note: Alternative: #undef all symbols (it's a pretty large list).
      * Without #error: it compiles, but functions are actually not inlined.
      */
#  endif
#  define XXH_NAMESPACE XXH_INLINE_
   /*
    * Some identifiers (enums, type names) are not symbols, but they must
    * still be renamed to avoid redeclaration.
    * Alternative solution: do not redeclare them.
    * However, this requires some #ifdefs, and is a more dispersed action.
    * Meanwhile, renaming can be achieved in a single block
    */
#  define XXH_IPREF(Id)   XXH_INLINE_ ## Id
#  define XXH_OK XXH_IPREF(XXH_OK)
#  define XXH_ERROR XXH_IPREF(XXH_ERROR)
#  define XXH_errorcode XXH_IPREF(XXH_errorcode)
#  define XXH32_canonical_t  XXH_IPREF(XXH32_canonical_t)
#  define XXH64_canonical_t  XXH_IPREF(XXH64_canonical_t)
#  define XXH128_canonical_t XXH_IPREF(XXH128_canonical_t)
#  define XXH32_state_s XXH_IPREF(XXH32_state_s)
#  define XXH32_state_t XXH_IPREF(XXH32_state_t)
#  define XXH64_state_s XXH_IPREF(XXH64_state_s)
#  define XXH64_state_t XXH_IPREF(XXH64_state_t)
#  define XXH3_state_s  XXH_IPREF(XXH3_state_s)
#  define XXH3_state_t  XXH_IPREF(XXH3_state_t)
#  define XXH128_hash_t XXH_IPREF(XXH128_hash_t)
   /* Ensure the header is parsed again, even if it was previously included */
#  undef XXHASH_H_5627135585666179
#  undef XXHASH_H_STATIC_13879238742
#endif /* XXH_INLINE_ALL || XXH_PRIVATE_API */



/* ****************************************************************
 *  Stable API
 *****************************************************************/
#ifndef XXHASH_H_5627135585666179
#define XXHASH_H_5627135585666179 1

/* specific declaration modes for Windows */
#if !defined(XXH_INLINE_ALL) && !defined(XXH_PRIVATE_API)
#  if defined(WIN32) && defined(_MSC_VER) && (defined(XXH_IMPORT) || defined(XXH_EXPORT))
#    ifdef XXH_EXPORT
#      define XXH_PUBLIC_API __declspec(dllexport)
#    elif XXH_IMPORT
#      define XXH_PUBLIC_API __declspec(dllimport)
#    endif
#  else
#    define XXH_PUBLIC_API   /* do nothing */
#  endif
#endif

/*!
 * XXH_NAMESPACE, aka Namespace Emulation:
 *
 * If you want to include _and expose_ xxHash functions from within your own
 * library, but also want to avoid symbol collisions with other libraries which
 * may also include xxHash, you can use XXH_NAMESPACE to automatically prefix
 * any public symbol from xxhash library with the value of XXH_NAMESPACE
 * (therefore, avoid empty or numeric values).
 *
 * Note that no change is required within the calling program as long as it
 * includes `xxhash.h`: Regular symbol names will be automatically translated
 * by this header.
 */
#ifdef XXH_NAMESPACE
#  define XXH_CAT(A,B) A##B
#  define XXH_NAME2(A,B) XXH_CAT(A,B)
#  define XXH_versionNumber XXH_NAME2(XXH_NAMESPACE, XXH_versionNumber)
#  define XXH32 XXH_NAME2(XXH_NAMESPACE, XXH32)
#  define XXH32_createState XXH_NAME2(XXH_NAMESPACE, XXH32_createState)
#  define XXH32_freeState XXH_NAME2(XXH_NAMESPACE, XXH32_freeState)
#  define XXH32_reset XXH_NAME2(XXH_NAMESPACE, XXH32_reset)
#  define XXH32_update XXH_NAME2(XXH_NAMESPACE, XXH32_update)
#  define XXH32_digest XXH_NAME2(XXH_NAMESPACE, XXH32_digest)
#  define XXH32_copyState XXH_NAME2(XXH_NAMESPACE, XXH32_copyState)
#  define XXH32_canonicalFromHash XXH_NAME2(XXH_NAMESPACE, XXH32_canonicalFromHash)
#  define XXH32_hashFromCanonical XXH_NAME2(XXH_NAMESPACE, XXH32_hashFromCanonical)
#  define XXH64 XXH_NAME2(XXH_NAMESPACE, XXH64)
#  define XXH64_createState XXH_NAME2(XXH_NAMESPACE, XXH64_createState)
#  define XXH64_freeState XXH_NAME2(XXH_NAMESPACE, XXH64_freeState)
#  define XXH64_reset XXH_NAME2(XXH_NAMESPACE, XXH64_reset)
#  define XXH64_update XXH_NAME2(XXH_NAMESPACE, XXH64_update)
#  define XXH64_digest XXH_NAME2(XXH_NAMESPACE, XXH64_digest)
#  define XXH64_copyState XXH_NAME2(XXH_NAMESPACE, XXH64_copyState)
#  define XXH64_canonicalFromHash XXH_NAME2(XXH_NAMESPACE, XXH64_canonicalFromHash)
#  define XXH64_hashFromCanonical XXH_NAME2(XXH_NAMESPACE, XXH64_hashFromCanonical)
#endif


/* *************************************
*  Version
***************************************/
#define XXH_VERSION_MAJOR    0
#define XXH_VERSION_MINOR    7
#define XXH_VERSION_RELEASE  3
#define XXH_VERSION_NUMBER  (XXH_VERSION_MAJOR *100*100 + XXH_VERSION_MINOR *100 + XXH_VERSION_RELEASE)
XXH_PUBLIC_API unsigned XXH_versionNumber (void);


/* ****************************
*  Definitions
******************************/
#include <stddef.h>   /* size_t */
typedef enum { XXH_OK=0, XXH_ERROR } XXH_errorcode;


/*-**********************************************************************
*  32-bit hash
************************************************************************/
#if !defined (__VMS) \
  && (defined (__cplusplus) \
  || (defined (__STDC_VERSION__) && (__STDC_VERSION__ >= 199901L) /* C99 */) )
#   include <stdint.h>
    typedef uint32_t XXH32_hash_t;
#else
#   include <limits.h>
#   if UINT_MAX == 0xFFFFFFFFUL
      typedef unsigned int XXH32_hash_t;
#   else
#     if ULONG_MAX == 0xFFFFFFFFUL
        typedef unsigned long XXH32_hash_t;
#     else
#       error "unsupported platform: need a 32-bit type"
#     endif
#   endif
#endif

/*!
 * XXH32():
 *  Calculate the 32-bit hash of sequence "length" bytes stored at memory address "input".
 *  The memory between input & input+length must be valid (allocated and read-accessible).
 *  "seed" can be used to alter the result predictably.
 *  Speed on Core 2 Duo @ 3 GHz (single thread, SMHasher benchmark): 5.4 GB/s
 */
XXH_PUBLIC_API XXH32_hash_t XXH32 (const void* input, size_t length, XXH32_hash_t seed);

/*******   Streaming   *******/

/*
 * Streaming functions generate the xxHash value from an incrememtal input.
 * This method is slower than single-call functions, due to state management.
 * For small inputs, prefer `XXH32()` and `XXH64()`, which are better optimized.
 *
 * An XXH state must first be allocated using `XXH*_createState()`.
 *
 * Start a new hash by initializing the state with a seed using `XXH*_reset()`.
 *
 * Then, feed the hash state by calling `XXH*_update()` as many times as necessary.
 *
 * The function returns an error code, with 0 meaning OK, and any other value
 * meaning there is an error.
 *
 * Finally, a hash value can be produced anytime, by using `XXH*_digest()`.
 * This function returns the nn-bits hash as an int or long long.
 *
 * It's still possible to continue inserting input into the hash state after a
 * digest, and generate new hash values later on by invoking `XXH*_digest()`.
 *
 * When done, release the state using `XXH*_freeState()`.
 */

typedef struct XXH32_state_s XXH32_state_t;   /* incomplete type */
XXH_PUBLIC_API XXH32_state_t* XXH32_createState(void);
XXH_PUBLIC_API XXH_errorcode  XXH32_freeState(XXH32_state_t* statePtr);
XXH_PUBLIC_API void XXH32_copyState(XXH32_state_t* dst_state, const XXH32_state_t* src_state);

XXH_PUBLIC_API XXH_errorcode XXH32_reset  (XXH32_state_t* statePtr, XXH32_hash_t seed);
XXH_PUBLIC_API XXH_errorcode XXH32_update (XXH32_state_t* statePtr, const void* input, size_t length);
XXH_PUBLIC_API XXH32_hash_t  XXH32_digest (const XXH32_state_t* statePtr);

/*******   Canonical representation   *******/

/*
 * The default return values from XXH functions are unsigned 32 and 64 bit
 * integers.
 * This the simplest and fastest format for further post-processing.
 *
 * However, this leaves open the question of what is the order on the byte level,
 * since little and big endian conventions will store the same number differently.
 *
 * The canonical representation settles this issue by mandating big-endian
 * convention, the same convention as human-readable numbers (large digits first).
 *
 * When writing hash values to storage, sending them over a network, or printing
 * them, it's highly recommended to use the canonical representation to ensure
 * portability across a wider range of systems, present and future.
 *
 * The following functions allow transformation of hash values to and from
 * canonical format.
 */

typedef struct { unsigned char digest[4]; } XXH32_canonical_t;
XXH_PUBLIC_API void XXH32_canonicalFromHash(XXH32_canonical_t* dst, XXH32_hash_t hash);
XXH_PUBLIC_API XXH32_hash_t XXH32_hashFromCanonical(const XXH32_canonical_t* src);


#ifndef XXH_NO_LONG_LONG
/*-**********************************************************************
*  64-bit hash
************************************************************************/
#if !defined (__VMS) \
  && (defined (__cplusplus) \
  || (defined (__STDC_VERSION__) && (__STDC_VERSION__ >= 199901L) /* C99 */) )
#   include <stdint.h>
    typedef uint64_t XXH64_hash_t;
#else
    /* the following type must have a width of 64-bit */
    typedef unsigned long long XXH64_hash_t;
#endif

/*!
 * XXH64():
 * Returns the 64-bit hash of sequence of length @length stored at memory
 * address @input.
 * @seed can be used to alter the result predictably.
 * This function usually runs faster on 64-bit systems, but slower on 32-bit
 * systems (see benchmark).
 */
XXH_PUBLIC_API XXH64_hash_t XXH64 (const void* input, size_t length, XXH64_hash_t seed);

/*******   Streaming   *******/
typedef struct XXH64_state_s XXH64_state_t;   /* incomplete type */
XXH_PUBLIC_API XXH64_state_t* XXH64_createState(void);
XXH_PUBLIC_API XXH_errorcode  XXH64_freeState(XXH64_state_t* statePtr);
XXH_PUBLIC_API void XXH64_copyState(XXH64_state_t* dst_state, const XXH64_state_t* src_state);

XXH_PUBLIC_API XXH_errorcode XXH64_reset  (XXH64_state_t* statePtr, XXH64_hash_t seed);
XXH_PUBLIC_API XXH_errorcode XXH64_update (XXH64_state_t* statePtr, const void* input, size_t length);
XXH_PUBLIC_API XXH64_hash_t  XXH64_digest (const XXH64_state_t* statePtr);

/*******   Canonical representation   *******/
typedef struct { unsigned char digest[8]; } XXH64_canonical_t;
XXH_PUBLIC_API void XXH64_canonicalFromHash(XXH64_canonical_t* dst, XXH64_hash_t hash);
XXH_PUBLIC_API XXH64_hash_t XXH64_hashFromCanonical(const XXH64_canonical_t* src);


#endif  /* XXH_NO_LONG_LONG */

#endif /* XXHASH_H_5627135585666179 */



#if defined(XXH_STATIC_LINKING_ONLY) && !defined(XXHASH_H_STATIC_13879238742)
#define XXHASH_H_STATIC_13879238742
/* ****************************************************************************
 * This section contains declarations which are not guaranteed to remain stable.
 * They may change in future versions, becoming incompatible with a different
 * version of the library.
 * These declarations should only be used with static linking.
 * Never use them in association with dynamic linking!
 ***************************************************************************** */

/*
 * These definitions are only present to allow static allocation of an XXH
 * state, for example, on the stack or in a struct.
 * Never **ever** access members directly.
 */

struct XXH32_state_s {
   XXH32_hash_t total_len_32;
   XXH32_hash_t large_len;
   XXH32_hash_t v1;
   XXH32_hash_t v2;
   XXH32_hash_t v3;
   XXH32_hash_t v4;
   XXH32_hash_t mem32[4];
   XXH32_hash_t memsize;
   XXH32_hash_t reserved;   /* never read nor write, might be removed in a future version */
};   /* typedef'd to XXH32_state_t */


#ifndef XXH_NO_LONG_LONG  /* defined when there is no 64-bit support */

struct XXH64_state_s {
   XXH64_hash_t total_len;
   XXH64_hash_t v1;
   XXH64_hash_t v2;
   XXH64_hash_t v3;
   XXH64_hash_t v4;
   XXH64_hash_t mem64[4];
   XXH32_hash_t memsize;
   XXH32_hash_t reserved32;  /* required for padding anyway */
   XXH64_hash_t reserved64;  /* never read nor write, might be removed in a future version */
};   /* typedef'd to XXH64_state_t */


/*-**********************************************************************
*  XXH3
*  New experimental hash
************************************************************************/

/* ************************************************************************
 * XXH3 is a new hash algorithm featuring:
 *  - Improved speed for both small and large inputs
 *  - True 64-bit and 128-bit outputs
 *  - SIMD acceleration
 *  - Improved 32-bit viability
 *
 * Speed analysis methodology is explained here:
 *
 *    https://fastcompression.blogspot.com/2019/03/presenting-xxh3.html
 *
 * In general, expect XXH3 to run about ~2x faster on large inputs and >3x
 * faster on small ones compared to XXH64, though exact differences depend on
 * the platform.
 *
 * The algorithm is portable: Like XXH32 and XXH64, it generates the same hash
 * on all platforms.
 *
 * It benefits greatly from SIMD and 64-bit arithmetic, but does not require it.
 *
 * Almost all 32-bit and 64-bit targets that can run XXH32 smoothly can run
 * XXH3 at usable speeds, even if XXH64 runs slowly. Further details are
 * explained in the implementation.
 *
 * Optimized implementations are provided for AVX2, SSE2, NEON, POWER8, ZVector,
 * and scalar targets. This can be controlled with the XXH_VECTOR macro.
 *
 * XXH3 offers 2 variants, _64bits and _128bits.
 * When only 64 bits are needed, prefer calling the _64bits variant, as it
 * reduces the amount of mixing, resulting in faster speed on small inputs.
 *
 * It's also generally simpler to manipulate a scalar return type than a struct.
 *
 * The 128-bit version adds additional strength, but it is slightly slower.
 *
 * The XXH3 algorithm is still in development.
 * The results it produces may still change in future versions.
 *
 * Results produced by v0.7.x are not comparable with results from v0.7.y.
 * However, the API is completely stable, and it can safely be used for
 * ephemeral data (local sessions).
 *
 * Avoid storing values in long-term storage until the algorithm is finalized.
 *
 * The API supports one-shot hashing, streaming mode, and custom secrets.
 */

#ifdef XXH_NAMESPACE
#  define XXH3_64bits XXH_NAME2(XXH_NAMESPACE, XXH3_64bits)
#  define XXH3_64bits_withSecret XXH_NAME2(XXH_NAMESPACE, XXH3_64bits_withSecret)
#  define XXH3_64bits_withSeed XXH_NAME2(XXH_NAMESPACE, XXH3_64bits_withSeed)

#  define XXH3_createState XXH_NAME2(XXH_NAMESPACE, XXH3_createState)
#  define XXH3_freeState XXH_NAME2(XXH_NAMESPACE, XXH3_freeState)
#  define XXH3_copyState XXH_NAME2(XXH_NAMESPACE, XXH3_copyState)

#  define XXH3_64bits_reset XXH_NAME2(XXH_NAMESPACE, XXH3_64bits_reset)
#  define XXH3_64bits_reset_withSeed XXH_NAME2(XXH_NAMESPACE, XXH3_64bits_reset_withSeed)
#  define XXH3_64bits_reset_withSecret XXH_NAME2(XXH_NAMESPACE, XXH3_64bits_reset_withSecret)
#  define XXH3_64bits_update XXH_NAME2(XXH_NAMESPACE, XXH3_64bits_update)
#  define XXH3_64bits_digest XXH_NAME2(XXH_NAMESPACE, XXH3_64bits_digest)
#endif

/* XXH3_64bits():
 * default 64-bit variant, using default secret and default seed of 0.
 * It's the fastest variant. */
XXH_PUBLIC_API XXH64_hash_t XXH3_64bits(const void* data, size_t len);

/*
 * XXH3_64bits_withSecret():
 * It's possible to provide any blob of bytes as a "secret" to generate the hash.
 * This makes it more difficult for an external actor to prepare an intentional
 * collision.
 * The secret *must* be large enough (>= XXH3_SECRET_SIZE_MIN).
 * It should consist of random bytes.
 * Avoid trivial sequences, such as repeating sequences and especially '\0',
 * as this can cancel out itself.
 * Failure to respect these conditions will result in a poor quality hash.
 */
#define XXH3_SECRET_SIZE_MIN 136
XXH_PUBLIC_API XXH64_hash_t XXH3_64bits_withSecret(const void* data, size_t len, const void* secret, size_t secretSize);

/*
 * XXH3_64bits_withSeed():
 * This variant generates a custom secret on the fly based on the default
 * secret, altered using the `seed` value.
 * While this operation is decently fast, note that it's not completely free.
 * Note: seed==0 produces the same results as XXH3_64bits().
 */
XXH_PUBLIC_API XXH64_hash_t XXH3_64bits_withSeed(const void* data, size_t len, XXH64_hash_t seed);


/* streaming 64-bit */

#if defined (__STDC_VERSION__) && (__STDC_VERSION__ >= 201112L)   /* C11+ */
#  include <stdalign.h>
#  define XXH_ALIGN(n)      alignas(n)
#elif defined(__GNUC__)
#  define XXH_ALIGN(n)      __attribute__ ((aligned(n)))
#elif defined(_MSC_VER)
#  define XXH_ALIGN(n)      __declspec(align(n))
#else
#  define XXH_ALIGN(n)   /* disabled */
#endif

/* Old GCC versions only accept the attribute after the type in structures. */
#if !(defined(__STDC_VERSION__) && (__STDC_VERSION__ >= 201112L))   /* C11+ */ \
    && defined(__GNUC__)
#   define XXH_ALIGN_MEMBER(align, type) type XXH_ALIGN(align)
#else
#   define XXH_ALIGN_MEMBER(align, type) XXH_ALIGN(align) type
#endif

typedef struct XXH3_state_s XXH3_state_t;

#define XXH3_SECRET_DEFAULT_SIZE 192   /* minimum XXH3_SECRET_SIZE_MIN */
#define XXH3_INTERNALBUFFER_SIZE 256
struct XXH3_state_s {
   XXH_ALIGN_MEMBER(64, XXH64_hash_t acc[8]);
  /* used to store a custom secret generated from the seed. Makes state larger.
   * Design might change */
   XXH_ALIGN_MEMBER(64, unsigned char customSecret[XXH3_SECRET_DEFAULT_SIZE]);
   XXH_ALIGN_MEMBER(64, unsigned char buffer[XXH3_INTERNALBUFFER_SIZE]);
   XXH32_hash_t bufferedSize;
   XXH32_hash_t nbStripesPerBlock;
   XXH32_hash_t nbStripesSoFar;
   XXH32_hash_t secretLimit;
   XXH32_hash_t reserved32;
   XXH32_hash_t reserved32_2;
   XXH64_hash_t totalLen;
   XXH64_hash_t seed;
   XXH64_hash_t reserved64;
   /* note: there is some padding after due to alignment on 64 bytes */
   const unsigned char* secret;
};   /* typedef'd to XXH3_state_t */

#undef XXH_ALIGN_MEMBER

/*
 * Streaming requires state maintenance.
 * This operation costs memory and CPU.
 * As a consequence, streaming is slower than one-shot hashing.
 * For better performance, prefer one-shot functions whenever possible.
 */
XXH_PUBLIC_API XXH3_state_t* XXH3_createState(void);
XXH_PUBLIC_API XXH_errorcode XXH3_freeState(XXH3_state_t* statePtr);
XXH_PUBLIC_API void XXH3_copyState(XXH3_state_t* dst_state, const XXH3_state_t* src_state);


/*
 * XXH3_64bits_reset():
 * Initialize with the default parameters.
 * The result will be equivalent to `XXH3_64bits()`.
 */
XXH_PUBLIC_API XXH_errorcode XXH3_64bits_reset(XXH3_state_t* statePtr);
/*
 * XXH3_64bits_reset_withSeed():
 * Generate a custom secret from `seed`, and store it into `statePtr`.
 * digest will be equivalent to `XXH3_64bits_withSeed()`.
 */
XXH_PUBLIC_API XXH_errorcode XXH3_64bits_reset_withSeed(XXH3_state_t* statePtr, XXH64_hash_t seed);
/*
 * XXH3_64bits_reset_withSecret():
 * `secret` is referenced, and must outlive the hash streaming session, so
 * be careful when using stack arrays.
 * `secretSize` must be >= `XXH3_SECRET_SIZE_MIN`.
 */
XXH_PUBLIC_API XXH_errorcode XXH3_64bits_reset_withSecret(XXH3_state_t* statePtr, const void* secret, size_t secretSize);

XXH_PUBLIC_API XXH_errorcode XXH3_64bits_update (XXH3_state_t* statePtr, const void* input, size_t length);
XXH_PUBLIC_API XXH64_hash_t  XXH3_64bits_digest (const XXH3_state_t* statePtr);


/* 128-bit */

#ifdef XXH_NAMESPACE
#  define XXH128 XXH_NAME2(XXH_NAMESPACE, XXH128)
#  define XXH3_128bits XXH_NAME2(XXH_NAMESPACE, XXH3_128bits)
#  define XXH3_128bits_withSeed XXH_NAME2(XXH_NAMESPACE, XXH3_128bits_withSeed)
#  define XXH3_128bits_withSecret XXH_NAME2(XXH_NAMESPACE, XXH3_128bits_withSecret)

#  define XXH3_128bits_reset XXH_NAME2(XXH_NAMESPACE, XXH3_128bits_reset)
#  define XXH3_128bits_reset_withSeed XXH_NAME2(XXH_NAMESPACE, XXH3_128bits_reset_withSeed)
#  define XXH3_128bits_reset_withSecret XXH_NAME2(XXH_NAMESPACE, XXH3_128bits_reset_withSecret)
#  define XXH3_128bits_update XXH_NAME2(XXH_NAMESPACE, XXH3_128bits_update)
#  define XXH3_128bits_digest XXH_NAME2(XXH_NAMESPACE, XXH3_128bits_digest)

#  define XXH128_isEqual XXH_NAME2(XXH_NAMESPACE, XXH128_isEqual)
#  define XXH128_cmp     XXH_NAME2(XXH_NAMESPACE, XXH128_cmp)
#  define XXH128_canonicalFromHash XXH_NAME2(XXH_NAMESPACE, XXH128_canonicalFromHash)
#  define XXH128_hashFromCanonical XXH_NAME2(XXH_NAMESPACE, XXH128_hashFromCanonical)
#endif

typedef struct {
    XXH64_hash_t low64;
    XXH64_hash_t high64;
} XXH128_hash_t;

XXH_PUBLIC_API XXH128_hash_t XXH128(const void* data, size_t len, XXH64_hash_t seed);
XXH_PUBLIC_API XXH128_hash_t XXH3_128bits(const void* data, size_t len);
XXH_PUBLIC_API XXH128_hash_t XXH3_128bits_withSeed(const void* data, size_t len, XXH64_hash_t seed);  /* == XXH128() */
XXH_PUBLIC_API XXH128_hash_t XXH3_128bits_withSecret(const void* data, size_t len, const void* secret, size_t secretSize);

XXH_PUBLIC_API XXH_errorcode XXH3_128bits_reset(XXH3_state_t* statePtr);
XXH_PUBLIC_API XXH_errorcode XXH3_128bits_reset_withSeed(XXH3_state_t* statePtr, XXH64_hash_t seed);
XXH_PUBLIC_API XXH_errorcode XXH3_128bits_reset_withSecret(XXH3_state_t* statePtr, const void* secret, size_t secretSize);

XXH_PUBLIC_API XXH_errorcode XXH3_128bits_update (XXH3_state_t* statePtr, const void* input, size_t length);
XXH_PUBLIC_API XXH128_hash_t XXH3_128bits_digest (const XXH3_state_t* statePtr);


/* Note: For better performance, these functions can be inlined using XXH_INLINE_ALL */

/*!
 * XXH128_isEqual():
 * Return: 1 if `h1` and `h2` are equal, 0 if they are not.
 */
XXH_PUBLIC_API int XXH128_isEqual(XXH128_hash_t h1, XXH128_hash_t h2);

/*!
 * XXH128_cmp():
 *
 * This comparator is compatible with stdlib's `qsort()`/`bsearch()`.
 *
 * return: >0 if *h128_1  > *h128_2
 *         <0 if *h128_1  < *h128_2
 *         =0 if *h128_1 == *h128_2
 */
XXH_PUBLIC_API int XXH128_cmp(const void* h128_1, const void* h128_2);


/*******   Canonical representation   *******/
typedef struct { unsigned char digest[16]; } XXH128_canonical_t;
XXH_PUBLIC_API void XXH128_canonicalFromHash(XXH128_canonical_t* dst, XXH128_hash_t hash);
XXH_PUBLIC_API XXH128_hash_t XXH128_hashFromCanonical(const XXH128_canonical_t* src);


#endif  /* XXH_NO_LONG_LONG */

#if defined(XXH_INLINE_ALL) || defined(XXH_PRIVATE_API)
#  define XXH_IMPLEMENTATION
#endif

#endif  /* defined(XXH_STATIC_LINKING_ONLY) && !defined(XXHASH_H_STATIC_13879238742) */


/* ======================================================================== */
/* ======================================================================== */
/* ======================================================================== */


/*-**********************************************************************
 * xxHash implementation
 *-**********************************************************************
 * xxHash's implementation used to be found in xxhash.c.
 *
 * However, code inlining requires the implementation to be visible to the
 * compiler, usually within the header.
 *
 * As a workaround, xxhash.c used to be included within xxhash.h. This caused
 * some issues with some build systems, especially ones which treat .c files
 * as source files.
 *
 * Therefore, the implementation is now directly integrated within xxhash.h.
 * Another small advantage is that xxhash.c is no longer needed in /include.
 ************************************************************************/

#if ( defined(XXH_INLINE_ALL) || defined(XXH_PRIVATE_API) \
   || defined(XXH_IMPLEMENTATION) ) && !defined(XXH_IMPLEM_13a8737387)
#  define XXH_IMPLEM_13a8737387

/* *************************************
*  Tuning parameters
***************************************/
/*!
 * XXH_FORCE_MEMORY_ACCESS:
 * By default, access to unaligned memory is controlled by `memcpy()`, which is
 * safe and portable.
 *
 * Unfortunately, on some target/compiler combinations, the generated assembly
 * is sub-optimal.
 *
 * The below switch allow to select a different access method for improved
 * performance.
 * Method 0 (default):
 *     Use `memcpy()`. Safe and portable.
 * Method 1:
 *     `__attribute__((packed))` statement. It depends on compiler extensions
 *     and is therefore not portable.
 *     This method is safe if your compiler supports it, and *generally* as
 *     fast or faster than `memcpy`.
 * Method 2:
 *     Direct access via cast. This method doesn't depend on the compiler but
 *     violates the C standard.
 *     It can generate buggy code on targets which do not support unaligned
 *     memory accesses.
 *     But in some circumstances, it's the only known way to get the most
 *     performance (ie GCC + ARMv6)
 * Method 3:
 *     Byteshift. This can generate the best code on old compilers which don't
 *     inline small `memcpy()` calls, and it might also be faster on big-endian
 *     systems which lack a native byteswap instruction.
 * See https://stackoverflow.com/a/32095106/646947 for details.
 * Prefer these methods in priority order (0 > 1 > 2 > 3)
 */
#ifndef XXH_FORCE_MEMORY_ACCESS   /* can be defined externally, on command line for example */
#  if !defined(__clang__) && defined(__GNUC__) && defined(__ARM_FEATURE_UNALIGNED) && defined(__ARM_ARCH) && (__ARM_ARCH == 6)
#    define XXH_FORCE_MEMORY_ACCESS 2
#  elif !defined(__clang__) && ((defined(__INTEL_COMPILER) && !defined(_WIN32)) || \
  (defined(__GNUC__) && (defined(__ARM_ARCH) && __ARM_ARCH >= 7)))
#    define XXH_FORCE_MEMORY_ACCESS 1
#  endif
#endif

/*!
 *XXH_ACCEPT_NULL_INPUT_POINTER:
 * If the input pointer is NULL, xxHash's default behavior is to dereference it,
 * triggering a segfault.
 * When this macro is enabled, xxHash actively checks the input for a null pointer.
 * If it is, the result for null input pointers is the same as a zero-length input.
 */
#ifndef XXH_ACCEPT_NULL_INPUT_POINTER   /* can be defined externally */
#  define XXH_ACCEPT_NULL_INPUT_POINTER 0
#endif

/*!
 * XXH_FORCE_ALIGN_CHECK:
 * This is a minor performance trick, only useful with lots of very small keys.
 * It means: check for aligned/unaligned input.
 * The check costs one initial branch per hash;
 * Set it to 0 when the input is guaranteed to be aligned or when alignment
 * doesn't matter for performance.
 *
 * This option does not affect XXH3.
 */
#ifndef XXH_FORCE_ALIGN_CHECK /* can be defined externally */
#  if defined(__i386) || defined(_M_IX86) || defined(__x86_64__) || defined(_M_X64)
#    define XXH_FORCE_ALIGN_CHECK 0
#  else
#    define XXH_FORCE_ALIGN_CHECK 1
#  endif
#endif

/*!
 * XXH_NO_INLINE_HINTS:
 *
 * By default, xxHash tries to force the compiler to inline almost all internal
 * functions.
 *
 * This can usually improve performance due to reduced jumping and improved
 * constant folding, but significantly increases the size of the binary which
 * might not be favorable.
 *
 * Additionally, sometimes the forced inlining can be detrimental to performance,
 * depending on the architecture.
 *
 * XXH_NO_INLINE_HINTS marks all internal functions as static, giving the
 * compiler full control on whether to inline or not.
 *
 * When not optimizing (-O0), optimizing for size (-Os, -Oz), or using
 * -fno-inline with GCC or Clang, this will automatically be defined.
 */
#ifndef XXH_NO_INLINE_HINTS
#  if defined(__OPTIMIZE_SIZE__) /* -Os, -Oz */ \
   || defined(__NO_INLINE__)     /* -O0, -fno-inline */
#    define XXH_NO_INLINE_HINTS 1
#  else
#    define XXH_NO_INLINE_HINTS 0
#  endif
#endif

/*!
 * XXH_REROLL:
 * Whether to reroll XXH32_finalize, and XXH64_finalize,
 * instead of using an unrolled jump table/if statement loop.
 *
 * This is automatically defined on -Os/-Oz on GCC and Clang.
 */
#ifndef XXH_REROLL
#  if defined(__OPTIMIZE_SIZE__)
#    define XXH_REROLL 1
#  else
#    define XXH_REROLL 0
#  endif
#endif


/* *************************************
*  Includes & Memory related functions
***************************************/
/*!
 * Modify the local functions below should you wish to use some other memory
 * routines for malloc() and free()
 */
#include <stdlib.h>
static void* XXH_malloc(size_t s) { return malloc(s); }
static void  XXH_free  (void* p)  { free(p); }
/*! and for memcpy() */
#include <string.h>
static void* XXH_memcpy(void* dest, const void* src, size_t size)
{
    return memcpy(dest,src,size);
}

#include <limits.h>   /* ULLONG_MAX */


/* *************************************
*  Compiler Specific Options
***************************************/
#ifdef _MSC_VER /* Visual Studio warning fix */
#  pragma warning(disable : 4127) /* disable: C4127: conditional expression is constant */
#endif

#if XXH_NO_INLINE_HINTS /* disable inlining hints */
#  define XXH_FORCE_INLINE static
#  define XXH_NO_INLINE static
#elif defined(_MSC_VER)    /* Visual Studio */
#  define XXH_FORCE_INLINE static __forceinline
#  define XXH_NO_INLINE static __declspec(noinline)
#else
#  if defined (__cplusplus) \
    || defined (__STDC_VERSION__) && __STDC_VERSION__ >= 199901L   /* C99 */
#    ifdef __GNUC__
#      define XXH_FORCE_INLINE static inline __attribute__((always_inline))
#      define XXH_NO_INLINE static __attribute__((noinline))
#    else
#      define XXH_FORCE_INLINE static inline
#      define XXH_NO_INLINE static
#    endif
#  else
#    define XXH_FORCE_INLINE static
#    define XXH_NO_INLINE static
#  endif /* __STDC_VERSION__ */
#endif



/* *************************************
*  Debug
***************************************/
/*
 * DEBUGLEVEL is expected to be defined externally, typically via the compiler's
 * command line options. The value must be a number.
 */
#ifndef DEBUGLEVEL
#  define DEBUGLEVEL 0
#endif

#if (DEBUGLEVEL>=1)
#  include <assert.h>   /* note: can still be disabled with NDEBUG */
#  define XXH_ASSERT(c)   assert(c)
#else
#  define XXH_ASSERT(c)   ((void)0)
#endif

/* note: use after variable declarations */
#define XXH_STATIC_ASSERT(c)  { enum { XXH_sa = 1/(int)(!!(c)) }; }


/* *************************************
*  Basic Types
***************************************/
#if !defined (__VMS) \
 && (defined (__cplusplus) \
 || (defined (__STDC_VERSION__) && (__STDC_VERSION__ >= 199901L) /* C99 */) )
# include <stdint.h>
  typedef uint8_t  xxh_u8;
#else
  typedef unsigned char      xxh_u8;
#endif
typedef XXH32_hash_t xxh_u32;


/* ***   Memory access   *** */

#if (defined(XXH_FORCE_MEMORY_ACCESS) && (XXH_FORCE_MEMORY_ACCESS==3))
/*
 * Manual byteshift. Best for old compilers which don't inline memcpy.
 * We actually directly use XXH_readLE32 and XXH_readBE32.
 */
#elif (defined(XXH_FORCE_MEMORY_ACCESS) && (XXH_FORCE_MEMORY_ACCESS==2))

/*
 * Force direct memory access. Only works on CPU which support unaligned memory
 * access in hardware.
 */
static xxh_u32 XXH_read32(const void* memPtr) { return *(const xxh_u32*) memPtr; }

#elif (defined(XXH_FORCE_MEMORY_ACCESS) && (XXH_FORCE_MEMORY_ACCESS==1))

/*
 * __pack instructions are safer but compiler specific, hence potentially
 * problematic for some compilers.
 *
 * Currently only defined for GCC and ICC.
 */
typedef union { xxh_u32 u32; } __attribute__((packed)) unalign;
static xxh_u32 XXH_read32(const void* ptr) { return ((const unalign*)ptr)->u32; }

#else

/*
 * Portable and safe solution. Generally efficient.
 * see: https://stackoverflow.com/a/32095106/646947
 */
static xxh_u32 XXH_read32(const void* memPtr)
{
    xxh_u32 val;
    memcpy(&val, memPtr, sizeof(val));
    return val;
}

#endif   /* XXH_FORCE_DIRECT_MEMORY_ACCESS */


/* ***   Endianess   *** */
typedef enum { XXH_bigEndian=0, XXH_littleEndian=1 } XXH_endianess;

/*!
 * XXH_CPU_LITTLE_ENDIAN:
 * Defined to 1 if the target is little endian, or 0 if it is big endian.
 * It can be defined externally, for example on the compiler command line.
 *
 * If it is not defined, a runtime check (which is usually constant folded)
 * is used instead.
 */
#ifndef XXH_CPU_LITTLE_ENDIAN
/*
 * Try to detect endianness automatically, to avoid the nonstandard behavior
 * in `XXH_isLittleEndian()`
 */
#  if defined(_WIN32) /* Windows is always little endian */ \
     || defined(__LITTLE_ENDIAN__) \
     || (defined(__BYTE_ORDER__) && __BYTE_ORDER__ == __ORDER_LITTLE_ENDIAN__)
#    define XXH_CPU_LITTLE_ENDIAN 1
#  elif defined(__BIG_ENDIAN__) \
     || (defined(__BYTE_ORDER__) && __BYTE_ORDER__ == __ORDER_BIG_ENDIAN__)
#    define XXH_CPU_LITTLE_ENDIAN 0
#  else
static int XXH_isLittleEndian(void)
{
    /*
     * Nonstandard, but well-defined behavior in practice.
     * Don't use static: it is detrimental to performance.
     */
    const union { xxh_u32 u; xxh_u8 c[4]; } one = { 1 };
    return one.c[0];
}
#   define XXH_CPU_LITTLE_ENDIAN   XXH_isLittleEndian()
#  endif
#endif




/* ****************************************
*  Compiler-specific Functions and Macros
******************************************/
#define XXH_GCC_VERSION (__GNUC__ * 100 + __GNUC_MINOR__)

#ifndef __has_builtin
#  define __has_builtin(x) 0
#endif

#if !defined(NO_CLANG_BUILTIN) && __has_builtin(__builtin_rotateleft32) \
                               && __has_builtin(__builtin_rotateleft64)
#  define XXH_rotl32 __builtin_rotateleft32
#  define XXH_rotl64 __builtin_rotateleft64
/* Note: although _rotl exists for minGW (GCC under windows), performance seems poor */
#elif defined(_MSC_VER)
#  define XXH_rotl32(x,r) _rotl(x,r)
#  define XXH_rotl64(x,r) _rotl64(x,r)
#else
#  define XXH_rotl32(x,r) (((x) << (r)) | ((x) >> (32 - (r))))
#  define XXH_rotl64(x,r) (((x) << (r)) | ((x) >> (64 - (r))))
#endif

#if defined(_MSC_VER)     /* Visual Studio */
#  define XXH_swap32 _byteswap_ulong
#elif XXH_GCC_VERSION >= 403
#  define XXH_swap32 __builtin_bswap32
#else
static xxh_u32 XXH_swap32 (xxh_u32 x)
{
    return  ((x << 24) & 0xff000000 ) |
            ((x <<  8) & 0x00ff0000 ) |
            ((x >>  8) & 0x0000ff00 ) |
            ((x >> 24) & 0x000000ff );
}
#endif


/* ***************************
*  Memory reads
*****************************/
typedef enum { XXH_aligned, XXH_unaligned } XXH_alignment;

/*
 * XXH_FORCE_MEMORY_ACCESS==3 is an endian-independent byteshift load.
 *
 * This is ideal for older compilers which don't inline memcpy.
 */
#if (defined(XXH_FORCE_MEMORY_ACCESS) && (XXH_FORCE_MEMORY_ACCESS==3))

XXH_FORCE_INLINE xxh_u32 XXH_readLE32(const void* memPtr)
{
    const xxh_u8* bytePtr = (const xxh_u8 *)memPtr;
    return bytePtr[0]
         | ((xxh_u32)bytePtr[1] << 8)
         | ((xxh_u32)bytePtr[2] << 16)
         | ((xxh_u32)bytePtr[3] << 24);
}

XXH_FORCE_INLINE xxh_u32 XXH_readBE32(const void* memPtr)
{
    const xxh_u8* bytePtr = (const xxh_u8 *)memPtr;
    return bytePtr[3]
         | ((xxh_u32)bytePtr[2] << 8)
         | ((xxh_u32)bytePtr[1] << 16)
         | ((xxh_u32)bytePtr[0] << 24);
}

#else
XXH_FORCE_INLINE xxh_u32 XXH_readLE32(const void* ptr)
{
    return XXH_CPU_LITTLE_ENDIAN ? XXH_read32(ptr) : XXH_swap32(XXH_read32(ptr));
}

static xxh_u32 XXH_readBE32(const void* ptr)
{
    return XXH_CPU_LITTLE_ENDIAN ? XXH_swap32(XXH_read32(ptr)) : XXH_read32(ptr);
}
#endif

XXH_FORCE_INLINE xxh_u32
XXH_readLE32_align(const void* ptr, XXH_alignment align)
{
    if (align==XXH_unaligned) {
        return XXH_readLE32(ptr);
    } else {
        return XXH_CPU_LITTLE_ENDIAN ? *(const xxh_u32*)ptr : XXH_swap32(*(const xxh_u32*)ptr);
    }
}


/* *************************************
*  Misc
***************************************/
XXH_PUBLIC_API unsigned XXH_versionNumber (void) { return XXH_VERSION_NUMBER; }


/* *******************************************************************
*  32-bit hash functions
*********************************************************************/
static const xxh_u32 PRIME32_1 = 0x9E3779B1U;   /* 0b10011110001101110111100110110001 */
static const xxh_u32 PRIME32_2 = 0x85EBCA77U;   /* 0b10000101111010111100101001110111 */
static const xxh_u32 PRIME32_3 = 0xC2B2AE3DU;   /* 0b11000010101100101010111000111101 */
static const xxh_u32 PRIME32_4 = 0x27D4EB2FU;   /* 0b00100111110101001110101100101111 */
static const xxh_u32 PRIME32_5 = 0x165667B1U;   /* 0b00010110010101100110011110110001 */

static xxh_u32 XXH32_round(xxh_u32 acc, xxh_u32 input)
{
    acc += input * PRIME32_2;
    acc  = XXH_rotl32(acc, 13);
    acc *= PRIME32_1;
#if defined(__GNUC__) && defined(__SSE4_1__) && !defined(XXH_ENABLE_AUTOVECTORIZE)
    /*
     * UGLY HACK:
     * This inline assembly hack forces acc into a normal register. This is the
     * only thing that prevents GCC and Clang from autovectorizing the XXH32
     * loop (pragmas and attributes don't work for some resason) without globally
     * disabling SSE4.1.
     *
     * The reason we want to avoid vectorization is because despite working on
     * 4 integers at a time, there are multiple factors slowing XXH32 down on
     * SSE4:
     * - There's a ridiculous amount of lag from pmulld (10 cycles of latency on
     *   newer chips!) making it slightly slower to multiply four integers at
     *   once compared to four integers independently. Even when pmulld was
     *   fastest, Sandy/Ivy Bridge, it is still not worth it to go into SSE
     *   just to multiply unless doing a long operation.
     *
     * - Four instructions are required to rotate,
     *      movqda tmp,  v // not required with VEX encoding
     *      pslld  tmp, 13 // tmp <<= 13
     *      psrld  v,   19 // x >>= 19
     *      por    v,  tmp // x |= tmp
     *   compared to one for scalar:
     *      roll   v, 13    // reliably fast across the board
     *      shldl  v, v, 13 // Sandy Bridge and later prefer this for some reason
     *
     * - Instruction level parallelism is actually more beneficial here because
     *   the SIMD actually serializes this operation: While v1 is rotating, v2
     *   can load data, while v3 can multiply. SSE forces them to operate
     *   together.
     *
     * How this hack works:
     * __asm__(""       // Declare an assembly block but don't declare any instructions
     *          :       // However, as an Input/Output Operand,
     *          "+r"    // constrain a read/write operand (+) as a general purpose register (r).
     *          (acc)   // and set acc as the operand
     * );
     *
     * Because of the 'r', the compiler has promised that seed will be in a
     * general purpose register and the '+' says that it will be 'read/write',
     * so it has to assume it has changed. It is like volatile without all the
     * loads and stores.
     *
     * Since the argument has to be in a normal register (not an SSE register),
     * each time XXH32_round is called, it is impossible to vectorize.
     */
    __asm__("" : "+r" (acc));
#endif
    return acc;
}

/* mix all bits */
static xxh_u32 XXH32_avalanche(xxh_u32 h32)
{
    h32 ^= h32 >> 15;
    h32 *= PRIME32_2;
    h32 ^= h32 >> 13;
    h32 *= PRIME32_3;
    h32 ^= h32 >> 16;
    return(h32);
}

#define XXH_get32bits(p) XXH_readLE32_align(p, align)

static xxh_u32
XXH32_finalize(xxh_u32 h32, const xxh_u8* ptr, size_t len, XXH_alignment align)
{
#define PROCESS1               \
    h32 += (*ptr++) * PRIME32_5; \
    h32 = XXH_rotl32(h32, 11) * PRIME32_1 ;

#define PROCESS4                         \
    h32 += XXH_get32bits(ptr) * PRIME32_3; \
    ptr+=4;                                \
    h32  = XXH_rotl32(h32, 17) * PRIME32_4 ;

    /* Compact rerolled version */
    if (XXH_REROLL) {
        len &= 15;
        while (len >= 4) {
            PROCESS4;
            len -= 4;
        }
        while (len > 0) {
            PROCESS1;
            --len;
        }
        return XXH32_avalanche(h32);
    } else {
         switch(len&15) /* or switch(bEnd - p) */ {
           case 12:      PROCESS4;
                         /* fallthrough */
           case 8:       PROCESS4;
                         /* fallthrough */
           case 4:       PROCESS4;
                         return XXH32_avalanche(h32);

           case 13:      PROCESS4;
                         /* fallthrough */
           case 9:       PROCESS4;
                         /* fallthrough */
           case 5:       PROCESS4;
                         PROCESS1;
                         return XXH32_avalanche(h32);

           case 14:      PROCESS4;
                         /* fallthrough */
           case 10:      PROCESS4;
                         /* fallthrough */
           case 6:       PROCESS4;
                         PROCESS1;
                         PROCESS1;
                         return XXH32_avalanche(h32);

           case 15:      PROCESS4;
                         /* fallthrough */
           case 11:      PROCESS4;
                         /* fallthrough */
           case 7:       PROCESS4;
                         /* fallthrough */
           case 3:       PROCESS1;
                         /* fallthrough */
           case 2:       PROCESS1;
                         /* fallthrough */
           case 1:       PROCESS1;
                         /* fallthrough */
           case 0:       return XXH32_avalanche(h32);
        }
        XXH_ASSERT(0);
        return h32;   /* reaching this point is deemed impossible */
    }
}

XXH_FORCE_INLINE xxh_u32
XXH32_endian_align(const xxh_u8* input, size_t len, xxh_u32 seed, XXH_alignment align)
{
    const xxh_u8* bEnd = input + len;
    xxh_u32 h32;

#if defined(XXH_ACCEPT_NULL_INPUT_POINTER) && (XXH_ACCEPT_NULL_INPUT_POINTER>=1)
    if (input==NULL) {
        len=0;
        bEnd=input=(const xxh_u8*)(size_t)16;
    }
#endif

    if (len>=16) {
        const xxh_u8* const limit = bEnd - 15;
        xxh_u32 v1 = seed + PRIME32_1 + PRIME32_2;
        xxh_u32 v2 = seed + PRIME32_2;
        xxh_u32 v3 = seed + 0;
        xxh_u32 v4 = seed - PRIME32_1;

        do {
            v1 = XXH32_round(v1, XXH_get32bits(input)); input += 4;
            v2 = XXH32_round(v2, XXH_get32bits(input)); input += 4;
            v3 = XXH32_round(v3, XXH_get32bits(input)); input += 4;
            v4 = XXH32_round(v4, XXH_get32bits(input)); input += 4;
        } while (input < limit);

        h32 = XXH_rotl32(v1, 1)  + XXH_rotl32(v2, 7)
            + XXH_rotl32(v3, 12) + XXH_rotl32(v4, 18);
    } else {
        h32  = seed + PRIME32_5;
    }

    h32 += (xxh_u32)len;

    return XXH32_finalize(h32, input, len&15, align);
}


XXH_PUBLIC_API XXH32_hash_t XXH32 (const void* input, size_t len, XXH32_hash_t seed)
{
#if 0
    /* Simple version, good for code maintenance, but unfortunately slow for small inputs */
    XXH32_state_t state;
    XXH32_reset(&state, seed);
    XXH32_update(&state, (const xxh_u8*)input, len);
    return XXH32_digest(&state);

#else

    if (XXH_FORCE_ALIGN_CHECK) {
        if ((((size_t)input) & 3) == 0) {   /* Input is 4-bytes aligned, leverage the speed benefit */
            return XXH32_endian_align((const xxh_u8*)input, len, seed, XXH_aligned);
    }   }

    return XXH32_endian_align((const xxh_u8*)input, len, seed, XXH_unaligned);
#endif
}



/*******   Hash streaming   *******/

XXH_PUBLIC_API XXH32_state_t* XXH32_createState(void)
{
    return (XXH32_state_t*)XXH_malloc(sizeof(XXH32_state_t));
}
XXH_PUBLIC_API XXH_errorcode XXH32_freeState(XXH32_state_t* statePtr)
{
    XXH_free(statePtr);
    return XXH_OK;
}

XXH_PUBLIC_API void XXH32_copyState(XXH32_state_t* dstState, const XXH32_state_t* srcState)
{
    memcpy(dstState, srcState, sizeof(*dstState));
}

XXH_PUBLIC_API XXH_errorcode XXH32_reset(XXH32_state_t* statePtr, XXH32_hash_t seed)
{
    XXH32_state_t state;   /* using a local state to memcpy() in order to avoid strict-aliasing warnings */
    memset(&state, 0, sizeof(state));
    state.v1 = seed + PRIME32_1 + PRIME32_2;
    state.v2 = seed + PRIME32_2;
    state.v3 = seed + 0;
    state.v4 = seed - PRIME32_1;
    /* do not write into reserved, planned to be removed in a future version */
	// BEGIN EPIC MOD
    memcpy(statePtr, &state, sizeof(state) - sizeof(state.reserved)); //-V512_UNDERFLOW_OFF PVS
	// END EPIC MOD
    return XXH_OK;
}


XXH_PUBLIC_API XXH_errorcode
XXH32_update(XXH32_state_t* state, const void* input, size_t len)
{
    if (input==NULL)
#if defined(XXH_ACCEPT_NULL_INPUT_POINTER) && (XXH_ACCEPT_NULL_INPUT_POINTER>=1)
        return XXH_OK;
#else
        return XXH_ERROR;
#endif

    {   const xxh_u8* p = (const xxh_u8*)input;
        const xxh_u8* const bEnd = p + len;

        state->total_len_32 += (XXH32_hash_t)len;
        state->large_len |= (XXH32_hash_t)((len>=16) | (state->total_len_32>=16));

        if (state->memsize + len < 16)  {   /* fill in tmp buffer */
            XXH_memcpy((xxh_u8*)(state->mem32) + state->memsize, input, len);
            state->memsize += (XXH32_hash_t)len;
            return XXH_OK;
        }

        if (state->memsize) {   /* some data left from previous update */
            XXH_memcpy((xxh_u8*)(state->mem32) + state->memsize, input, 16-state->memsize);
            {   const xxh_u32* p32 = state->mem32;
                state->v1 = XXH32_round(state->v1, XXH_readLE32(p32)); p32++;
                state->v2 = XXH32_round(state->v2, XXH_readLE32(p32)); p32++;
                state->v3 = XXH32_round(state->v3, XXH_readLE32(p32)); p32++;
                state->v4 = XXH32_round(state->v4, XXH_readLE32(p32));
            }
            p += 16-state->memsize;
            state->memsize = 0;
        }

        if (p <= bEnd-16) {
            const xxh_u8* const limit = bEnd - 16;
            xxh_u32 v1 = state->v1;
            xxh_u32 v2 = state->v2;
            xxh_u32 v3 = state->v3;
            xxh_u32 v4 = state->v4;

            do {
                v1 = XXH32_round(v1, XXH_readLE32(p)); p+=4;
                v2 = XXH32_round(v2, XXH_readLE32(p)); p+=4;
                v3 = XXH32_round(v3, XXH_readLE32(p)); p+=4;
                v4 = XXH32_round(v4, XXH_readLE32(p)); p+=4;
            } while (p<=limit);

            state->v1 = v1;
            state->v2 = v2;
            state->v3 = v3;
            state->v4 = v4;
        }

        if (p < bEnd) {
            XXH_memcpy(state->mem32, p, (size_t)(bEnd-p));
            state->memsize = (unsigned)(bEnd-p);
        }
    }

    return XXH_OK;
}


XXH_PUBLIC_API XXH32_hash_t XXH32_digest (const XXH32_state_t* state)
{
    xxh_u32 h32;

    if (state->large_len) {
        h32 = XXH_rotl32(state->v1, 1)
            + XXH_rotl32(state->v2, 7)
            + XXH_rotl32(state->v3, 12)
            + XXH_rotl32(state->v4, 18);
    } else {
        h32 = state->v3 /* == seed */ + PRIME32_5;
    }

    h32 += state->total_len_32;

    return XXH32_finalize(h32, (const xxh_u8*)state->mem32, state->memsize, XXH_aligned);
}


/*******   Canonical representation   *******/

/*
 * The default return values from XXH functions are unsigned 32 and 64 bit
 * integers.
 *
 * The canonical representation uses big endian convention, the same convention
 * as human-readable numbers (large digits first).
 *
 * This way, hash values can be written into a file or buffer, remaining
 * comparable across different systems.
 *
 * The following functions allow transformation of hash values to and from their
 * canonical format.
 */
XXH_PUBLIC_API void XXH32_canonicalFromHash(XXH32_canonical_t* dst, XXH32_hash_t hash)
{
    XXH_STATIC_ASSERT(sizeof(XXH32_canonical_t) == sizeof(XXH32_hash_t));
    if (XXH_CPU_LITTLE_ENDIAN) hash = XXH_swap32(hash);
    memcpy(dst, &hash, sizeof(*dst));
}

XXH_PUBLIC_API XXH32_hash_t XXH32_hashFromCanonical(const XXH32_canonical_t* src)
{
    return XXH_readBE32(src);
}


#ifndef XXH_NO_LONG_LONG

/* *******************************************************************
*  64-bit hash functions
*********************************************************************/

/*******   Memory access   *******/

typedef XXH64_hash_t xxh_u64;


/*!
 * XXH_REROLL_XXH64:
 * Whether to reroll the XXH64_finalize() loop.
 *
 * Just like XXH32, we can unroll the XXH64_finalize() loop. This can be a
 * performance gain on 64-bit hosts, as only one jump is required.
 *
 * However, on 32-bit hosts, because arithmetic needs to be done with two 32-bit
 * registers, and 64-bit arithmetic needs to be simulated, it isn't beneficial
 * to unroll. The code becomes ridiculously large (the largest function in the
 * binary on i386!), and rerolling it saves anywhere from 3kB to 20kB. It is
 * also slightly faster because it fits into cache better and is more likely
 * to be inlined by the compiler.
 *
 * If XXH_REROLL is defined, this is ignored and the loop is always rerolled.
 */
#ifndef XXH_REROLL_XXH64
#  if (defined(__ILP32__) || defined(_ILP32)) /* ILP32 is often defined on 32-bit GCC family */ \
   || !(defined(__x86_64__) || defined(_M_X64) || defined(_M_AMD64) /* x86-64 */ \
     || defined(_M_ARM64) || defined(_M_ARM64EC) || defined(__aarch64__) || defined(__arm64__) /* aarch64 */ /* EPIC MOD: Added _M_ARM64EC support */ \
     || defined(__PPC64__) || defined(__PPC64LE__) || defined(__ppc64__) || defined(__powerpc64__) /* ppc64 */ \
     || defined(__mips64__) || defined(__mips64)) /* mips64 */ \
   || (!defined(SIZE_MAX) || SIZE_MAX < ULLONG_MAX) /* check limits */
#    define XXH_REROLL_XXH64 1
#  else
#    define XXH_REROLL_XXH64 0
#  endif
#endif /* !defined(XXH_REROLL_XXH64) */

#if (defined(XXH_FORCE_MEMORY_ACCESS) && (XXH_FORCE_MEMORY_ACCESS==3))
/*
 * Manual byteshift. Best for old compilers which don't inline memcpy.
 * We actually directly use XXH_readLE64 and XXH_readBE64.
 */
#elif (defined(XXH_FORCE_MEMORY_ACCESS) && (XXH_FORCE_MEMORY_ACCESS==2))

/* Force direct memory access. Only works on CPU which support unaligned memory access in hardware */
static xxh_u64 XXH_read64(const void* memPtr) { return *(const xxh_u64*) memPtr; }

#elif (defined(XXH_FORCE_MEMORY_ACCESS) && (XXH_FORCE_MEMORY_ACCESS==1))

/*
 * __pack instructions are safer, but compiler specific, hence potentially
 * problematic for some compilers.
 *
 * Currently only defined for GCC and ICC.
 */
typedef union { xxh_u32 u32; xxh_u64 u64; } __attribute__((packed)) unalign64;
static xxh_u64 XXH_read64(const void* ptr) { return ((const unalign64*)ptr)->u64; }

#else

/*
 * Portable and safe solution. Generally efficient.
 * see: https://stackoverflow.com/a/32095106/646947
 */
static xxh_u64 XXH_read64(const void* memPtr)
{
    xxh_u64 val;
    memcpy(&val, memPtr, sizeof(val));
    return val;
}

#endif   /* XXH_FORCE_DIRECT_MEMORY_ACCESS */

#if defined(_MSC_VER)     /* Visual Studio */
#  define XXH_swap64 _byteswap_uint64
#elif XXH_GCC_VERSION >= 403
#  define XXH_swap64 __builtin_bswap64
#else
static xxh_u64 XXH_swap64 (xxh_u64 x)
{
    return  ((x << 56) & 0xff00000000000000ULL) |
            ((x << 40) & 0x00ff000000000000ULL) |
            ((x << 24) & 0x0000ff0000000000ULL) |
            ((x << 8)  & 0x000000ff00000000ULL) |
            ((x >> 8)  & 0x00000000ff000000ULL) |
            ((x >> 24) & 0x0000000000ff0000ULL) |
            ((x >> 40) & 0x000000000000ff00ULL) |
            ((x >> 56) & 0x00000000000000ffULL);
}
#endif


/* XXH_FORCE_MEMORY_ACCESS==3 is an endian-independent byteshift load. */
#if (defined(XXH_FORCE_MEMORY_ACCESS) && (XXH_FORCE_MEMORY_ACCESS==3))

XXH_FORCE_INLINE xxh_u64 XXH_readLE64(const void* memPtr)
{
    const xxh_u8* bytePtr = (const xxh_u8 *)memPtr;
    return bytePtr[0]
         | ((xxh_u64)bytePtr[1] << 8)
         | ((xxh_u64)bytePtr[2] << 16)
         | ((xxh_u64)bytePtr[3] << 24)
         | ((xxh_u64)bytePtr[4] << 32)
         | ((xxh_u64)bytePtr[5] << 40)
         | ((xxh_u64)bytePtr[6] << 48)
         | ((xxh_u64)bytePtr[7] << 56);
}

XXH_FORCE_INLINE xxh_u64 XXH_readBE64(const void* memPtr)
{
    const xxh_u8* bytePtr = (const xxh_u8 *)memPtr;
    return bytePtr[7]
         | ((xxh_u64)bytePtr[6] << 8)
         | ((xxh_u64)bytePtr[5] << 16)
         | ((xxh_u64)bytePtr[4] << 24)
         | ((xxh_u64)bytePtr[3] << 32)
         | ((xxh_u64)bytePtr[2] << 40)
         | ((xxh_u64)bytePtr[1] << 48)
         | ((xxh_u64)bytePtr[0] << 56);
}

#else
XXH_FORCE_INLINE xxh_u64 XXH_readLE64(const void* ptr)
{
    return XXH_CPU_LITTLE_ENDIAN ? XXH_read64(ptr) : XXH_swap64(XXH_read64(ptr));
}

static xxh_u64 XXH_readBE64(const void* ptr)
{
    return XXH_CPU_LITTLE_ENDIAN ? XXH_swap64(XXH_read64(ptr)) : XXH_read64(ptr);
}
#endif

XXH_FORCE_INLINE xxh_u64
XXH_readLE64_align(const void* ptr, XXH_alignment align)
{
    if (align==XXH_unaligned)
        return XXH_readLE64(ptr);
    else
        return XXH_CPU_LITTLE_ENDIAN ? *(const xxh_u64*)ptr : XXH_swap64(*(const xxh_u64*)ptr);
}


/*******   xxh64   *******/

static const xxh_u64 PRIME64_1 = 0x9E3779B185EBCA87ULL;   /* 0b1001111000110111011110011011000110000101111010111100101010000111 */
static const xxh_u64 PRIME64_2 = 0xC2B2AE3D27D4EB4FULL;   /* 0b1100001010110010101011100011110100100111110101001110101101001111 */
static const xxh_u64 PRIME64_3 = 0x165667B19E3779F9ULL;   /* 0b0001011001010110011001111011000110011110001101110111100111111001 */
static const xxh_u64 PRIME64_4 = 0x85EBCA77C2B2AE63ULL;   /* 0b1000010111101011110010100111011111000010101100101010111001100011 */
static const xxh_u64 PRIME64_5 = 0x27D4EB2F165667C5ULL;   /* 0b0010011111010100111010110010111100010110010101100110011111000101 */

static xxh_u64 XXH64_round(xxh_u64 acc, xxh_u64 input)
{
    acc += input * PRIME64_2;
    acc  = XXH_rotl64(acc, 31);
    acc *= PRIME64_1;
    return acc;
}

static xxh_u64 XXH64_mergeRound(xxh_u64 acc, xxh_u64 val)
{
    val  = XXH64_round(0, val);
    acc ^= val;
    acc  = acc * PRIME64_1 + PRIME64_4;
    return acc;
}

static xxh_u64 XXH64_avalanche(xxh_u64 h64)
{
    h64 ^= h64 >> 33;
    h64 *= PRIME64_2;
    h64 ^= h64 >> 29;
    h64 *= PRIME64_3;
    h64 ^= h64 >> 32;
    return h64;
}


#define XXH_get64bits(p) XXH_readLE64_align(p, align)

static xxh_u64
XXH64_finalize(xxh_u64 h64, const xxh_u8* ptr, size_t len, XXH_alignment align)
{
#define PROCESS1_64            \
    h64 ^= (*ptr++) * PRIME64_5; \
    h64 = XXH_rotl64(h64, 11) * PRIME64_1;

#define PROCESS4_64          \
    h64 ^= (xxh_u64)(XXH_get32bits(ptr)) * PRIME64_1; \
    ptr+=4;                    \
    h64 = XXH_rotl64(h64, 23) * PRIME64_2 + PRIME64_3;

#define PROCESS8_64 {        \
    xxh_u64 const k1 = XXH64_round(0, XXH_get64bits(ptr)); \
    ptr+=8;                    \
    h64 ^= k1;               \
    h64  = XXH_rotl64(h64,27) * PRIME64_1 + PRIME64_4; \
}

    /* Rerolled version for 32-bit targets is faster and much smaller. */
    if (XXH_REROLL || XXH_REROLL_XXH64) {
        len &= 31;
        while (len >= 8) {
            PROCESS8_64;
            len -= 8;
        }
        if (len >= 4) {
            PROCESS4_64;
            len -= 4;
        }
        while (len > 0) {
            PROCESS1_64;
            --len;
        }
         return  XXH64_avalanche(h64);
    } else {
        switch(len & 31) {
           case 24: PROCESS8_64;
                         /* fallthrough */
           case 16: PROCESS8_64;
                         /* fallthrough */
           case  8: PROCESS8_64;
                    return XXH64_avalanche(h64);

           case 28: PROCESS8_64;
                         /* fallthrough */
           case 20: PROCESS8_64;
                         /* fallthrough */
           case 12: PROCESS8_64;
                         /* fallthrough */
           case  4: PROCESS4_64;
                    return XXH64_avalanche(h64);

           case 25: PROCESS8_64;
                         /* fallthrough */
           case 17: PROCESS8_64;
                         /* fallthrough */
           case  9: PROCESS8_64;
                    PROCESS1_64;
                    return XXH64_avalanche(h64);

           case 29: PROCESS8_64;
                         /* fallthrough */
           case 21: PROCESS8_64;
                         /* fallthrough */
           case 13: PROCESS8_64;
                         /* fallthrough */
           case  5: PROCESS4_64;
                    PROCESS1_64;
                    return XXH64_avalanche(h64);

           case 26: PROCESS8_64;
                         /* fallthrough */
           case 18: PROCESS8_64;
                         /* fallthrough */
           case 10: PROCESS8_64;
                    PROCESS1_64;
                    PROCESS1_64;
                    return XXH64_avalanche(h64);

           case 30: PROCESS8_64;
                         /* fallthrough */
           case 22: PROCESS8_64;
                         /* fallthrough */
           case 14: PROCESS8_64;
                         /* fallthrough */
           case  6: PROCESS4_64;
                    PROCESS1_64;
                    PROCESS1_64;
                    return XXH64_avalanche(h64);

           case 27: PROCESS8_64;
                         /* fallthrough */
           case 19: PROCESS8_64;
                         /* fallthrough */
           case 11: PROCESS8_64;
                    PROCESS1_64;
                    PROCESS1_64;
                    PROCESS1_64;
                    return XXH64_avalanche(h64);

           case 31: PROCESS8_64;
                         /* fallthrough */
           case 23: PROCESS8_64;
                         /* fallthrough */
           case 15: PROCESS8_64;
                         /* fallthrough */
           case  7: PROCESS4_64;
                         /* fallthrough */
           case  3: PROCESS1_64;
                         /* fallthrough */
           case  2: PROCESS1_64;
                         /* fallthrough */
           case  1: PROCESS1_64;
                         /* fallthrough */
           case  0: return XXH64_avalanche(h64);
        }
    }
    /* impossible to reach */
    XXH_ASSERT(0);
    return 0;  /* unreachable, but some compilers complain without it */
}

XXH_FORCE_INLINE xxh_u64
XXH64_endian_align(const xxh_u8* input, size_t len, xxh_u64 seed, XXH_alignment align)
{
    const xxh_u8* bEnd = input + len;
    xxh_u64 h64;

#if defined(XXH_ACCEPT_NULL_INPUT_POINTER) && (XXH_ACCEPT_NULL_INPUT_POINTER>=1)
    if (input==NULL) {
        len=0;
        bEnd=input=(const xxh_u8*)(size_t)32;
    }
#endif

    if (len>=32) {
        const xxh_u8* const limit = bEnd - 32;
        xxh_u64 v1 = seed + PRIME64_1 + PRIME64_2;
        xxh_u64 v2 = seed + PRIME64_2;
        xxh_u64 v3 = seed + 0;
        xxh_u64 v4 = seed - PRIME64_1;

        do {
            v1 = XXH64_round(v1, XXH_get64bits(input)); input+=8;
            v2 = XXH64_round(v2, XXH_get64bits(input)); input+=8;
            v3 = XXH64_round(v3, XXH_get64bits(input)); input+=8;
            v4 = XXH64_round(v4, XXH_get64bits(input)); input+=8;
        } while (input<=limit);

        h64 = XXH_rotl64(v1, 1) + XXH_rotl64(v2, 7) + XXH_rotl64(v3, 12) + XXH_rotl64(v4, 18);
        h64 = XXH64_mergeRound(h64, v1);
        h64 = XXH64_mergeRound(h64, v2);
        h64 = XXH64_mergeRound(h64, v3);
        h64 = XXH64_mergeRound(h64, v4);

    } else {
        h64  = seed + PRIME64_5;
    }

    h64 += (xxh_u64) len;

    return XXH64_finalize(h64, input, len, align);
}


XXH_PUBLIC_API XXH64_hash_t XXH64 (const void* input, size_t len, XXH64_hash_t seed)
{
#if 0
    /* Simple version, good for code maintenance, but unfortunately slow for small inputs */
    XXH64_state_t state;
    XXH64_reset(&state, seed);
    XXH64_update(&state, (const xxh_u8*)input, len);
    return XXH64_digest(&state);

#else

    if (XXH_FORCE_ALIGN_CHECK) {
        if ((((size_t)input) & 7)==0) {  /* Input is aligned, let's leverage the speed advantage */
            return XXH64_endian_align((const xxh_u8*)input, len, seed, XXH_aligned);
    }   }

    return XXH64_endian_align((const xxh_u8*)input, len, seed, XXH_unaligned);

#endif
}

/*******   Hash Streaming   *******/

XXH_PUBLIC_API XXH64_state_t* XXH64_createState(void)
{
    return (XXH64_state_t*)XXH_malloc(sizeof(XXH64_state_t));
}
XXH_PUBLIC_API XXH_errorcode XXH64_freeState(XXH64_state_t* statePtr)
{
    XXH_free(statePtr);
    return XXH_OK;
}

XXH_PUBLIC_API void XXH64_copyState(XXH64_state_t* dstState, const XXH64_state_t* srcState)
{
    memcpy(dstState, srcState, sizeof(*dstState));
}

XXH_PUBLIC_API XXH_errorcode XXH64_reset(XXH64_state_t* statePtr, XXH64_hash_t seed)
{
    XXH64_state_t state;   /* use a local state to memcpy() in order to avoid strict-aliasing warnings */
    memset(&state, 0, sizeof(state));
    state.v1 = seed + PRIME64_1 + PRIME64_2;
    state.v2 = seed + PRIME64_2;
    state.v3 = seed + 0;
    state.v4 = seed - PRIME64_1;
     /* do not write into reserved64, might be removed in a future version */
	// BEGIN EPIC MOD
	memcpy(statePtr, &state, sizeof(state) - sizeof(state.reserved64)); //-V512_UNDERFLOW_OFF PVS
	// END EPIC MOD
    return XXH_OK;
}

XXH_PUBLIC_API XXH_errorcode
XXH64_update (XXH64_state_t* state, const void* input, size_t len)
{
    if (input==NULL)
#if defined(XXH_ACCEPT_NULL_INPUT_POINTER) && (XXH_ACCEPT_NULL_INPUT_POINTER>=1)
        return XXH_OK;
#else
        return XXH_ERROR;
#endif

    {   const xxh_u8* p = (const xxh_u8*)input;
        const xxh_u8* const bEnd = p + len;

        state->total_len += len;

        if (state->memsize + len < 32) {  /* fill in tmp buffer */
            XXH_memcpy(((xxh_u8*)state->mem64) + state->memsize, input, len);
            state->memsize += (xxh_u32)len;
            return XXH_OK;
        }

        if (state->memsize) {   /* tmp buffer is full */
            XXH_memcpy(((xxh_u8*)state->mem64) + state->memsize, input, 32-state->memsize);
            state->v1 = XXH64_round(state->v1, XXH_readLE64(state->mem64+0));
            state->v2 = XXH64_round(state->v2, XXH_readLE64(state->mem64+1));
            state->v3 = XXH64_round(state->v3, XXH_readLE64(state->mem64+2));
            state->v4 = XXH64_round(state->v4, XXH_readLE64(state->mem64+3));
            p += 32-state->memsize;
            state->memsize = 0;
        }

        if (p+32 <= bEnd) {
            const xxh_u8* const limit = bEnd - 32;
            xxh_u64 v1 = state->v1;
            xxh_u64 v2 = state->v2;
            xxh_u64 v3 = state->v3;
            xxh_u64 v4 = state->v4;

            do {
                v1 = XXH64_round(v1, XXH_readLE64(p)); p+=8;
                v2 = XXH64_round(v2, XXH_readLE64(p)); p+=8;
                v3 = XXH64_round(v3, XXH_readLE64(p)); p+=8;
                v4 = XXH64_round(v4, XXH_readLE64(p)); p+=8;
            } while (p<=limit);

            state->v1 = v1;
            state->v2 = v2;
            state->v3 = v3;
            state->v4 = v4;
        }

        if (p < bEnd) {
            XXH_memcpy(state->mem64, p, (size_t)(bEnd-p));
            state->memsize = (unsigned)(bEnd-p);
        }
    }

    return XXH_OK;
}


XXH_PUBLIC_API XXH64_hash_t XXH64_digest (const XXH64_state_t* state)
{
    xxh_u64 h64;

    if (state->total_len >= 32) {
        xxh_u64 const v1 = state->v1;
        xxh_u64 const v2 = state->v2;
        xxh_u64 const v3 = state->v3;
        xxh_u64 const v4 = state->v4;

        h64 = XXH_rotl64(v1, 1) + XXH_rotl64(v2, 7) + XXH_rotl64(v3, 12) + XXH_rotl64(v4, 18);
        h64 = XXH64_mergeRound(h64, v1);
        h64 = XXH64_mergeRound(h64, v2);
        h64 = XXH64_mergeRound(h64, v3);
        h64 = XXH64_mergeRound(h64, v4);
    } else {
        h64  = state->v3 /*seed*/ + PRIME64_5;
    }

    h64 += (xxh_u64) state->total_len;

    return XXH64_finalize(h64, (const xxh_u8*)state->mem64, (size_t)state->total_len, XXH_aligned);
}


/******* Canonical representation   *******/

XXH_PUBLIC_API void XXH64_canonicalFromHash(XXH64_canonical_t* dst, XXH64_hash_t hash)
{
    XXH_STATIC_ASSERT(sizeof(XXH64_canonical_t) == sizeof(XXH64_hash_t));
    if (XXH_CPU_LITTLE_ENDIAN) hash = XXH_swap64(hash);
    memcpy(dst, &hash, sizeof(*dst));
}

XXH_PUBLIC_API XXH64_hash_t XXH64_hashFromCanonical(const XXH64_canonical_t* src)
{
    return XXH_readBE64(src);
}



/* *********************************************************************
*  XXH3
*  New generation hash designed for speed on small keys and vectorization
************************************************************************ */

#include "xxh3.h"


#endif  /* XXH_NO_LONG_LONG */


#endif  /* XXH_IMPLEMENTATION */


#if defined (__cplusplus)
}
#endif
