// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveCodingLog.h"

DEFINE_LOG_CATEGORY(LogLiveCoding);
