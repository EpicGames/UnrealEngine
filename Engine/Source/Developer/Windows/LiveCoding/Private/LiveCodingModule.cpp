// Copyright Epic Games, Inc. All Rights Reserved.

#if LC_VERSION == 1

#include "LiveCodingModule.h"
#include "BuildSettings.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "LiveCodingLog.h"
#include "External/LC_Commands.h"
#include "External/LC_EntryPoint.h"
#include "External/LC_API.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "Windows/WindowsHWrapper.h"
#include "Algo/Sort.h"
#include "Algo/BinarySearch.h"
#include "HAL/LowLevelMemTracker.h"
#include "Async/TaskGraphInterfaces.h"
#if WITH_EDITOR
	#include "Editor.h"
	#include "Kismet2/ReloadUtilities.h"
	#include "Widgets/Notifications/SNotificationList.h"
	#include "Framework/Notifications/NotificationManager.h"
#else
	#include "UObject/Reload.h"
#endif
#if WITH_ENGINE
	#include "Engine/Engine.h"
	#include "UObject/UObjectIterator.h"
	#include "UObject/StrongObjectPtr.h"
#endif

#include "Windows/AllowWindowsPlatformTypes.h"
#include <psapi.h> // for GetModuleFileNameEx
#ifdef NT_SUCCESS
#undef NT_SUCCESS
#endif
#include <subauth.h> // for UNICODE_STRING

IMPLEMENT_MODULE(FLiveCodingModule, LiveCoding)

#define LOCTEXT_NAMESPACE "LiveCodingModule"

LLM_DEFINE_TAG(LiveCoding);

bool GIsCompileActive = false;
bool GTriggerReload = false;
bool GHasLoadedPatch = false;
commands::PostCompileResult GPostCompileResult = commands::PostCompileResult::Success;
FString GLiveCodingConsolePath;
FString GLiveCodingConsoleArguments;
FLiveCodingModule* GLiveCodingModule = nullptr;

#if !WITH_EDITOR
class FNullReload : public IReload
{
public:
	FNullReload(FLiveCodingModule& InLiveCodingModule)
		: LiveCodingModule(InLiveCodingModule)
	{
		BeginReload(EActiveReloadType::LiveCoding, *this);
	}

	~FNullReload()
	{
		EndReload();
	}

	virtual EActiveReloadType GetType() const
	{
		return EActiveReloadType::LiveCoding;
	}


	virtual const TCHAR* GetPrefix() const
	{
		return TEXT("LIVECODING");
	}

	virtual void NotifyFunctionRemap(FNativeFuncPtr NewFunctionPointer, FNativeFuncPtr OldFunctionPointer)
	{
	}

	virtual void NotifyChange(UClass* New, UClass* Old) override
	{
	}

	virtual void NotifyChange(UEnum* New, UEnum* Old) override
	{
	}

	virtual void NotifyChange(UScriptStruct* New, UScriptStruct* Old) override
	{
	}

	virtual void NotifyChange(UPackage* New, UPackage* Old) override
	{
	}

	virtual bool GetEnableReinstancing(bool bHasChanged) const
	{
		if (bHasChanged && !bEnabledMessage)
		{
			bEnabledMessage = true;
			bHasReinstancingOccurred = true;
			static const TCHAR* Message = TEXT("Object structure changes detected.  LiveCoding re-instancing isn't supported in builds without the editor");
			UE_LOGF(LogLiveCoding, Error, "%ls", Message);
#if WITH_ENGINE
			if (GEngine)
			{
				GEngine->AddOnScreenDebugMessage(uint64(uintptr_t(&LiveCodingModule)), 5.f, FColor::Red, Message);
			}
#endif
		}
		return false;
	}

	virtual void Reinstance()
	{
	}

	virtual UObject* GetReinstancedCDO(UObject* CDO) override
	{
		return CDO;
	}

	virtual const UObject* GetReinstancedCDO(const UObject* CDO) override
	{
		return CDO;
	}

	virtual const TMap<UClass*, UClass*>& GetReinstancedClasses() override
	{
		static const TMap<UClass*, UClass*> Empty;
		return Empty;
	}

	virtual const TArray<UClass*>& GetNewClasses() override
	{
		static const TArray<UClass*> Empty;
		return Empty;
	}


	bool HasReinstancingOccurred() const
	{
		return bHasReinstancingOccurred;
	}

	void Reset()
	{
		bHasReinstancingOccurred = false;
	}

private:
	FLiveCodingModule& LiveCodingModule;
	mutable bool bEnabledMessage = false;
	mutable bool bHasReinstancingOccurred = false;
};
#endif

namespace LivePP
{

	// Based on liveProcess::GetLowerBoundIn4GBRange
	uintptr_t GetLowerBound(uintptr_t ModuleBase, uint64 Padding)
	{
		constexpr uintptr_t LowestPossibleAddress = 64ull * 1024ull;
		return ModuleBase >= LowestPossibleAddress + Padding ? ModuleBase - Padding : LowestPossibleAddress;
	}

	// Based on liveProcess::GetUpperBoundIn4GBRange
	uintptr_t GetUpperBound(uintptr_t ModuleBase, uint64 Padding)
	{
		constexpr uintptr_t HighestPossibleAddress = 0x00007FFFFFFF0000ull;
		return ModuleBase <= HighestPossibleAddress - Padding ? ModuleBase + Padding : HighestPossibleAddress;
	}

	// Based on VirtualMemoryRange::ReservePages
	void ReservePages(TArray<uintptr_t>& ReservedPages, uintptr_t AddressStart, uintptr_t AddressEnd, uint64 PageAlignment)
	{
		HANDLE ProcessHandle = GetCurrentProcess();

		// reserve all free pages in the virtual memory range.
		// pages must be aligned to the given alignment.
		for (uintptr_t Address = AddressStart; Address < AddressEnd; /* nothing */)
		{
			// align address to be scanned
			Address = Align(Address, PageAlignment);

			if (Address < AddressStart)
			{
				// overflow happened because we scanned too far
				break;
			}

			::MEMORY_BASIC_INFORMATION MemoryInfo = {};
			const size_t BytesReturned = ::VirtualQueryEx(ProcessHandle, (const void*)Address, &MemoryInfo, sizeof(::MEMORY_BASIC_INFORMATION));
			if (BytesReturned == 0)
			{
				break;
			}

			// we are only interested in free pages
			if (MemoryInfo.State == MEM_FREE)
			{
				// work out the maximum size of the page allocation.
				// we should not allocate past the end of the range.
				const uint64 BytesLeft = AddressEnd - (uintptr_t)MemoryInfo.BaseAddress;
				const uint64 Size = std::min<uint64>(MemoryInfo.RegionSize, BytesLeft);

				// try to reserve this page.
				// if we are really unlucky, the process might have allocated this region in the meantime.
				void* BaseAddress = ::VirtualAllocEx(ProcessHandle, MemoryInfo.BaseAddress, Size, MEM_RESERVE, PAGE_NOACCESS);
				if (BaseAddress)
				{
					ReservedPages.Add((uintptr_t)BaseAddress);
				}
			}

			// keep on searching
			Address = (uintptr_t)MemoryInfo.BaseAddress + MemoryInfo.RegionSize;
		}
	}
}

namespace
{
	constexpr uint64 DefaultPadding = 128ull * 1024ull * 1024ull; // LivePP used 2ull * 1024ull * 1024ull * 1024ull;
	constexpr uint64 DefaultPageAlignment = 64ull * 1024ull;
	constexpr uint32 NewModuleCountThreshhold = 128;

	void ReservePagesBefore(TArray<uintptr_t>& ReservedPages, uintptr_t ModuleBase, uint64 Padding, uint32 PageAlignment)
	{
		LivePP::ReservePages(ReservedPages, LivePP::GetLowerBound(ModuleBase, Padding), ModuleBase, PageAlignment);
	}

	void ReservePagesBetween(TArray<uintptr_t>& ReservedPages, uintptr_t StartModuleBase, uintptr_t EndModuleBase, uint64 Padding, uint32 PageAlignment)
	{
		LivePP::ReservePages(ReservedPages, LivePP::GetLowerBound(StartModuleBase, Padding), LivePP::GetUpperBound(EndModuleBase, Padding), PageAlignment);
	}

	void ReservePagesAfter(TArray<uintptr_t>& ReservedPages, uintptr_t ModuleBase, uint64 Padding, uint32 PageAlignment)
	{
		LivePP::ReservePages(ReservedPages, ModuleBase, LivePP::GetUpperBound(ModuleBase, Padding), PageAlignment);
	}

	void ReservePages(TArray<uintptr_t>& ReservedPages, const TArray<uintptr_t>& ModuleBases, uint64 Padding, uint64 PageAlignment)
	{
		if (ModuleBases.IsEmpty())
		{
			return;
		}

		// We use an ordered list to avoid rescanning the same ranges over and over again
		TArray<uintptr_t> SortedModuleBases(ModuleBases);
		SortedModuleBases.Sort();

		// Reserve prior to the first module
		ReservePagesBefore(ReservedPages, SortedModuleBases[0], Padding, PageAlignment);

		// Loop through the pairs of modules
		for (int i = 0; i < SortedModuleBases.Num() - 1; ++i)
		{
			uintptr_t StartModuleBase = SortedModuleBases[i];
			uintptr_t EndModuleBase = SortedModuleBases[i + 1];
			if (EndModuleBase - StartModuleBase < Padding * 2)
			{
				ReservePagesBetween(ReservedPages, StartModuleBase, EndModuleBase, Padding, PageAlignment);
			}
			else
			{
				ReservePagesAfter(ReservedPages, StartModuleBase, Padding, PageAlignment);
				ReservePagesBefore(ReservedPages, EndModuleBase, Padding, PageAlignment);
			}
		}

		// Reserve after the last module
		ReservePagesAfter(ReservedPages, SortedModuleBases.Last(), Padding, PageAlignment);
	}
}

// Helper structure to load the NTDLL library and work with an API
struct FNtDllFunction
{
	FARPROC Addr;

	FNtDllFunction(const char* Name)
	{
		HMODULE NtDll = LoadLibraryW(L"ntdll.dll");
		check(NtDll);
		Addr = GetProcAddress(NtDll, Name);
	}

	template <typename... ArgTypes>
	unsigned int operator () (ArgTypes... Args)
	{
		typedef unsigned int (NTAPI* Prototype)(ArgTypes...);
		return (Prototype((void*)Addr))(Args...);
	}
};

FLiveCodingModule::FLiveCodingModule()
	: FullEngineDir(FPaths::ConvertRelativePathToFull(FPaths::EngineDir()))
	, FullEnginePluginsDir(FPaths::ConvertRelativePathToFull(FPaths::EnginePluginsDir()))
	, FullProjectDir(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()))
	, FullProjectPluginsDir(FPaths::ConvertRelativePathToFull(FPaths::ProjectPluginsDir()))
{
	GLiveCodingModule = this;

	const FString ExecutablePath = FPaths::GetPath(FPlatformProcess::ExecutablePath());
	FullEngineDirFromExecutable = ExecutablePath / FPaths::EngineDir();
	FPaths::NormalizeDirectoryName(FullEngineDirFromExecutable);
	FPaths::CollapseRelativeDirectories(FullEngineDirFromExecutable);
	FullEnginePluginsDirFromExecutable = ExecutablePath / FPaths::EnginePluginsDir();
	FPaths::NormalizeDirectoryName(FullEnginePluginsDirFromExecutable);
	FPaths::CollapseRelativeDirectories(FullEnginePluginsDirFromExecutable);
	FullProjectDirFromExecutable = ExecutablePath / FPaths::ProjectDir();
	FPaths::NormalizeDirectoryName(FullProjectDirFromExecutable);
	FPaths::CollapseRelativeDirectories(FullProjectDirFromExecutable);
	FullProjectPluginsDirFromExecutable = ExecutablePath / FPaths::ProjectPluginsDir();
	FPaths::NormalizeDirectoryName(FullProjectPluginsDirFromExecutable);
	FPaths::CollapseRelativeDirectories(FullProjectPluginsDirFromExecutable);
}

FLiveCodingModule::~FLiveCodingModule()
{
	GLiveCodingModule = nullptr;
}

#if USE_DEBUG_LIVE_CODING_CONSOLE
	static const TCHAR* DefaultConsolePath = TEXT("Binaries/Win64/LiveCodingConsole-Win64-Debug.exe");
#else
	static const TCHAR* DefaultConsolePath = TEXT("Binaries/Win64/LiveCodingConsole.exe");
#endif 

void FLiveCodingModule::StartupModule()
{
	LLM_SCOPE_BYTAG(LiveCoding);

	// Register with NT to get dll notifications
	FNtDllFunction RegisterFunc("LdrRegisterDllNotification");
	RegisterFunc(0, OnDllNotification, this, &CallbackCookie);

	// Get the creating process main executable name.  We want to skip this file when iterating through the dll list
	FString MainModuleName;
	{
		TCHAR Scratch[MAX_PATH];
		int Length = GetModuleFileNameEx(::GetCurrentProcess(), NULL, Scratch, MAX_PATH);
		MainModuleName = FString::ConstructFromPtrSize(Scratch, Length);
	}

	// https://docs.microsoft.com/en-us/windows/win32/api/winternl/ns-winternl-peb_ldr_data
	typedef struct _PEB_LDR_DATA {
		BYTE       Reserved1[8];
		PVOID      Reserved2[3];
		LIST_ENTRY InMemoryOrderModuleList;
	} PEB_LDR_DATA, * PPEB_LDR_DATA;

	// https://docs.microsoft.com/en-us/windows/win32/api/winternl/ns-winternl-peb
	typedef struct _PEB {
		BYTE                          Reserved1[2];
		BYTE                          BeingDebugged;
		BYTE                          Reserved2[1];
		PVOID                         Reserved3[2];
		PPEB_LDR_DATA                 Ldr;
		//...
	} PEB, * PPEB;

	// https://docs.microsoft.com/en-us/windows/win32/api/winternl/ns-winternl-teb
	typedef struct _TEB {
		PVOID Reserved1[12];
		PPEB  ProcessEnvironmentBlock;
		//...
	} TEB, * PTEB;

	// https://docs.microsoft.com/en-us/windows/win32/api/winternl/ns-winternl-peb_ldr_data
	typedef struct _LDR_DATA_TABLE_ENTRY {
		PVOID Reserved1[2];
		LIST_ENTRY InMemoryOrderLinks;
		PVOID Reserved2[2];
		PVOID DllBase;
		PVOID EntryPoint;
		PVOID Reserved3;
		UNICODE_STRING FullDllName;
		//...
	} LDR_DATA_TABLE_ENTRY, * PLDR_DATA_TABLE_ENTRY;

	// Enumerate already loaded modules.
	const TEB* ThreadEnvBlock = reinterpret_cast<TEB*>(NtCurrentTeb());
	const PEB* ProcessEnvBlock = ThreadEnvBlock->ProcessEnvironmentBlock;
	const LIST_ENTRY* ModuleIter = ProcessEnvBlock->Ldr->InMemoryOrderModuleList.Flink;
	const LIST_ENTRY* ModuleIterEnd = ModuleIter->Blink;
	do
	{
		const auto& ModuleData = *(LDR_DATA_TABLE_ENTRY*)(ModuleIter - 1);
		if (ModuleData.DllBase == 0)
		{
			break;
		}

		FString FullPath = FString::ConstructFromPtrSize(ModuleData.FullDllName.Buffer, ModuleData.FullDllName.Length / sizeof(ModuleData.FullDllName.Buffer[0]));
		if (!FullPath.Equals(MainModuleName, ESearchCase::IgnoreCase))
		{
			FPaths::NormalizeFilename(FullPath);
			OnDllLoaded(FullPath);
		}
		ModuleIter = ModuleIter->Flink;
	} while (ModuleIter != ModuleIterEnd);

	Settings = GetMutableDefault<ULiveCodingSettings>();

	IConsoleManager& ConsoleManager = IConsoleManager::Get();

	EnableCommand = ConsoleManager.RegisterConsoleCommand(
		TEXT("LiveCoding"),
		TEXT("Enables live coding support"),
		FConsoleCommandWithOutputDeviceDelegate::CreateRaw(this, &FLiveCodingModule::EnableConsoleCommand),
		ECVF_Cheat
	);

	CompileCommand = ConsoleManager.RegisterConsoleCommand(
		TEXT("LiveCoding.Compile"),
		TEXT("Initiates a live coding compile"),
		FConsoleCommandDelegate::CreateLambda([this] { Compile(ELiveCodingCompileFlags::None, nullptr); }),
		ECVF_Cheat
	);
	
	CompileSyncCommand = ConsoleManager.RegisterConsoleCommand(
		TEXT("LiveCoding.CompileSync"),
		TEXT("Initiates a live coding compile and waits for completion"),
		FConsoleCommandDelegate::CreateLambda([this] { Compile(ELiveCodingCompileFlags::WaitForCompletion, nullptr); }),
		ECVF_Cheat
	);

#if IS_MONOLITHIC
	FString DefaultEngineDir = BuildSettings::GetLiveCodingEngineDir();
#else
	FString DefaultEngineDir = FPaths::EngineDir();
#endif
	ConsolePathVariable = ConsoleManager.RegisterConsoleVariable(
		TEXT("LiveCoding.ConsolePath"),
		FPaths::ConvertRelativePathToFull(DefaultEngineDir / DefaultConsolePath),
		TEXT("Path to the live coding console application"),
		ECVF_Cheat
	);

#if IS_MONOLITHIC
	const TCHAR* LiveCodingProject = BuildSettings::GetLiveCodingProject();
	FString SourceProject = (LiveCodingProject != nullptr)? LiveCodingProject : TEXT("");
#else
	FString SourceProject = FPaths::IsProjectFilePathSet() ? FPaths::GetProjectFilePath() : TEXT("");
#endif
	if (SourceProject.Len() > 0)
	{
		SourceProject = FPaths::ConvertRelativePathToFull(SourceProject);
	}
	SourceProjectVariable = ConsoleManager.RegisterConsoleVariable(
		TEXT("LiveCoding.SourceProject"),
		SourceProject,
		TEXT("Path to the project that this target was built from"),
		ECVF_Cheat
	);

	EndFrameDelegateHandle = FCoreDelegates::OnEndFrame.AddRaw(this, &FLiveCodingModule::Tick);

	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		SettingsSection = SettingsModule->RegisterSettings("Editor", "General", "Live Coding",
			LOCTEXT("LiveCodingSettingsName", "Live Coding"),
			LOCTEXT("LiveCodintSettingsDescription", "Settings for recompiling C++ code while the engine is running."),
			GetMutableDefault<ULiveCodingSettings>()
		);
	}

	LppStartup();

	bool bAllowAutoStart = true;
	bool bForceStart = false;

	if (bool bCommandLineEnable; FParse::Bool(FCommandLine::Get(), TEXT("-LiveCoding="), bCommandLineEnable))
	{
		bAllowAutoStart &= bCommandLineEnable;
		bForceStart = bCommandLineEnable;
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("LiveCoding")))
	{
		bForceStart = true;
	}

	bSettingsEnabledLastTick = Settings->bEnabled;
	if (Settings->bEnabled && Settings->Startup != ELiveCodingStartupMode::Manual && !FApp::IsUnattended() && bAllowAutoStart)
	{
		StartLiveCodingAsync(Settings->Startup);
	}
	else if (bForceStart)
	{
		StartLiveCodingAsync(ELiveCodingStartupMode::Manual);
	}

	bEnableReinstancingLastTick = IsReinstancingEnabled();
}

void FLiveCodingModule::ShutdownModule()
{
	StopLiveCoding();
}

void FLiveCodingModule::EnableByDefault(bool bEnable)
{
	if (Settings->bEnabled != bEnable)
	{
		Settings->bEnabled = bEnable;
		if (SettingsSection.IsValid())
		{
			SettingsSection->Save();
		}
	}
	EnableForSession(bEnable);
}

bool FLiveCodingModule::IsEnabledByDefault() const
{
	return Settings->bEnabled;
}

void FLiveCodingModule::EnableConsoleCommand(FOutputDevice& out)
{
	EnableForSession(true);

	// For packaged builds, by default it is unlikely that UE_LOG messages will be seen in the console.
	// So log any error directly.
	if (!EnableErrorText.IsEmpty())
	{
		out.Log(EnableErrorText);
	}
}

void FLiveCodingModule::EnableForSession(bool bEnable)
{
	if (bEnable)
	{
		EnableErrorText = FText::GetEmpty();
		switch (State)
		{
		case EState::NotRunning:
			if (!SetupConsolePath())
			{
				return;
			}
			StartLiveCoding(ELiveCodingStartupMode::Manual); // State set in this method
			ShowConsole();
			break;

		case EState::Starting:
			WaitForStartup();
			break;

		case EState::Running:
		case EState::RunningAndEnabled:
			State = EState::RunningAndEnabled;
			ShowConsole();
			break;
		}
	}
	else 
	{
		switch (State)
		{
		case EState::NotRunning:
		case EState::Starting:
			break;

		case EState::Running:
		case EState::RunningAndEnabled:
			HideConsole();
			State = EState::Running;
			break;
		}
	}
}

void FLiveCodingModule::WaitForStartup()
{
	if (State == EState::Starting)
	{
		UE_LOGF(LogLiveCoding, Display, "Waiting for console to start.");
		FPlatformProcess::ConditionalSleep([this]() { return State != EState::Starting; });
	}
}

bool FLiveCodingModule::IsEnabledForSession() const
{
	switch (State)
	{
	case EState::NotRunning:
	case EState::Starting:
	case EState::Running:
		return false;

	case EState::RunningAndEnabled:
		return true;

	default:
		return false;
	}
}

const FText& FLiveCodingModule::GetEnableErrorText() const
{
	return EnableErrorText;
}

bool FLiveCodingModule::CanEnableForSession() const
{
#if !IS_MONOLITHIC
	FModuleManager& ModuleManager = FModuleManager::Get();
	if(ModuleManager.HasAnyOverridenModuleFilename())
	{
		return false;
	}
#endif
	return true;
}

bool FLiveCodingModule::HasStarted() const
{
	return HasStarted(false);
}

bool FLiveCodingModule::HasStarted(bool bAllowStarting) const
{
	switch (State)
	{
	case EState::NotRunning:
		return false;

	case EState::Starting:
		return bAllowStarting;

	case EState::Running:
	case EState::RunningAndEnabled:
		return true;

	default:
		return false;
	}
}

void FLiveCodingModule::ShowConsole()
{
	ShowConsole(false);
}

void FLiveCodingModule::ShowConsole(bool bAllowStarting)
{
	if (HasStarted(bAllowStarting))
	{
		LppSetVisible(true);
		LppSetActive(true);
		LppShowConsole();
	}
}

void FLiveCodingModule::HideConsole()
{
	if (HasStarted())
	{
		UE_LOGF(LogLiveCoding, Display, "Console will be hidden but remain running in the background. Restart to disable completely.");
		LppSetActive(false);
		LppSetVisible(false);
	}
}

void FLiveCodingModule::Compile()
{
	Compile(ELiveCodingCompileFlags::None, nullptr);
}

inline bool ReturnResults(ELiveCodingCompileResult InResult, ELiveCodingCompileResult* OutResult)
{
	if (OutResult != nullptr)
	{
		*OutResult = InResult;
	}
	return InResult == ELiveCodingCompileResult::Success || InResult == ELiveCodingCompileResult::NoChanges || InResult == ELiveCodingCompileResult::InProgress;
}

bool FLiveCodingModule::Compile(ELiveCodingCompileFlags CompileFlags, ELiveCodingCompileResult* Result)
{
	if (GIsCompileActive)
	{
		return ReturnResults(ELiveCodingCompileResult::CompileStillActive, Result);
	}

	EnableForSession(true);
	if (!HasStarted())
	{
		return ReturnResults(ELiveCodingCompileResult::NotStarted, Result);
	}

	// Need to do this immediately rather than waiting until next tick
	UpdateModules(false); 

	// Trigger the recompile
	GIsCompileActive = true;
	LastResults = ELiveCodingCompileResult::Failure;
	LppTriggerRecompile();

	// If we aren't waiting, just return now
	if (!EnumHasAnyFlags(CompileFlags, ELiveCodingCompileFlags::WaitForCompletion))
	{
		return ReturnResults(ELiveCodingCompileResult::InProgress, Result);
	}

	// Wait until we are no longer compiling.  Cancellation is handled via other mechanisms and
	// need not be detected in this loop.  GIsCompileActive will be cleared.
	FText StatusUpdate = LOCTEXT("CompileStatusMessage", "Compiling...");
	FScopedSlowTask SlowTask(0, StatusUpdate, GIsSlowTask);
	SlowTask.MakeDialog();

	// Wait until the compile completes
	while (GIsCompileActive)
	{
		SlowTask.EnterProgressFrame(0.0f);
		AttemptSyncLivePatching();
		FPlatformProcess::Sleep(0.01f);
	}

	// A final sync to get the result and complete the process
	AttemptSyncLivePatching();

	return ReturnResults(LastResults, Result);
}

bool FLiveCodingModule::IsCompiling() const
{
	return GIsCompileActive;
}

void FLiveCodingModule::Tick()
{

	// Check for a change in the last requested enable state if we are in automatic mode
	if (Settings->bEnabled != bSettingsEnabledLastTick && Settings->Startup != ELiveCodingStartupMode::Manual)
	{
		switch (State)
		{
		case EState::NotRunning:
			bSettingsEnabledLastTick = Settings->bEnabled;
			if (Settings->bEnabled)
			{
				EnableForSession(true);
				if (!IsEnabledForSession())
				{
					FMessageDialog::Open(EAppMsgType::Ok, EnableErrorText);
				}
			}
			break;

		case EState::Starting:
		case EState::Running:
			break;

		case EState::RunningAndEnabled:
			bSettingsEnabledLastTick = Settings->bEnabled;
			if (!Settings->bEnabled)
			{
				EnableForSession(false);
			}
			break;
		}
	}

	if (HasStarted())
	{
		if (LppWantsRestart())
		{
			LppRestart(lpp::LPP_RESTART_BEHAVIOR_REQUEST_EXIT, 0);
		}

		if (HasStarted() && IsReinstancingEnabled() != bEnableReinstancingLastTick)
		{
			bEnableReinstancingLastTick = IsReinstancingEnabled();
			LppSetReinstancingFlow(bEnableReinstancingLastTick);
		}

		if (bUpdateModulesInTick)
		{
			UpdateModules(false);
			bUpdateModulesInTick = false;
		}

		AttemptSyncLivePatching();
	}
}

void FLiveCodingModule::AttemptSyncLivePatching()
{

	// We use to wait for all commands to finish, but that causes a lock up if starting PIE after a compilation 
	// request caused another command to be sent to the live coding console.  For example, the registering of 
	// another lazy load module at PIE start would cause this problem.
	for (int Index = LppPendingTokens.Num(); Index-- > 0;)
	{
		if (LppTryWaitForToken(LppPendingTokens[Index]))
		{
			LppPendingTokens.RemoveAt(Index);
		}
	}

	// Needs to happen after updating modules, since "Quick Restart" functionality may try to install patch immediately
	extern void LppSyncPoint();
	LppSyncPoint();

	if ((!GIsCompileActive || GTriggerReload) && Reload.IsValid())
	{
		if (GHasLoadedPatch)
		{
#if WITH_COREUOBJECT && WITH_ENGINE

			// Collect the existing objects
			TArray<UObject*> StartingObjects;
			if (Reload->GetEnableReinstancing(false))
			{
				StartingObjects.Reserve(1024); // Arbitrary
				for (TObjectIterator<UObject> It(EObjectFlags::RF_NoFlags); It; ++It)
				{
					StartingObjects.Add(*It);
				}
				Algo::Sort(StartingObjects);
			}

			// During the module loading process, the list of changed classes will be recorded.  Invoking this method will 
			// result in the RegisterForReinstancing method being invoked which in turn records the classes in the ClassesToReinstance
			// member variable being populated.
			ProcessNewlyLoadedUObjects();

			// Complete the process of re-instancing without doing a GC
#if WITH_EDITOR
			Reload->Finalize(false);
#endif

			TArray<TStrongObjectPtr<UObject>> NewObjects;
			if (Reload->GetEnableReinstancing(false))
			{

				// Loop through the objects again looking for anything new that isn't associated with a
				// reinstanced class.
				for (TObjectIterator<UObject> It(EObjectFlags::RF_NoFlags); It; ++It)
				{
					if (Algo::BinarySearch(StartingObjects, *It) == INDEX_NONE)
					{
						if (!It->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists))
						{
							NewObjects.Add(TStrongObjectPtr<UObject>(*It));
						}
					}
				}

				// Loop through all of the classes looking for classes that have been re-instanced.  Reset the CDO
				// to something that will never change.  Since these classes have been replaced, they should NEVER
				// have their CDos accessed again.  In the future we should try to figure out a better solution the issue
				// where the reinstanced crashes recreating the default object probably due to a mismatch between then
				// new constructor being invoked and the blueprint data associated with the old class.  With LC, the
				// old constructor has been replaced.
				static UObject* DummyDefaultObject = UObject::StaticClass()->GetDefaultObject(false);
				for (TObjectIterator<UClass> It; It; ++It)
				{
					UClass* Class = *It;
					if (Class->GetName().StartsWith(TEXT("LIVECODING_")) ||
						Class->GetName().StartsWith(TEXT("REINST_")))
					{
						Class->SetDefaultObject(DummyDefaultObject);
					}
				}
			}

			// Broadcast event prior to GC.  Otherwise some things are holding onto references
			FCoreUObjectDelegates::ReloadCompleteDelegate.Broadcast(EReloadCompleteReason::None);

			// Perform the GC to try and destruct all the objects which will be invoking the old destructors.
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
#endif

			// Second sync point to finish off the patching
			if (GTriggerReload)
			{
				LppSyncPoint();
			}

#if WITH_COREUOBJECT && WITH_ENGINE
			// Remove the reference to any new objects
			NewObjects.Empty();
#endif

			FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::LiveCodingReload);

			OnPatchCompleteDelegate.Broadcast();
			GHasLoadedPatch = false;

			bHasReinstancingOccurred |= Reload->HasReinstancingOccurred();
		}
		else if (GTriggerReload)
		{
			LppSyncPoint();
		}

		if (!GIsCompileActive)
		{
			static const FString Success("Live coding succeeded");

			// Reset this first so it does its logging first
			Reload.Reset();

			switch (GPostCompileResult)
			{
			case commands::PostCompileResult::Success:
				LastResults = ELiveCodingCompileResult::Success;
				if (bHasReinstancingOccurred)
				{
					if (!IsReinstancingEnabled())
					{
						UE_LOGF(LogLiveCoding, Warning, "%ls, %ls", *Success, TEXT("data type changes with re-instancing disabled is not supported and will likely lead to a crash"));
					}
					else
					{
#if WITH_EDITOR
						UE_LOGF(LogLiveCoding, Warning, "%ls, %ls", *Success, TEXT("data type changes may cause packaging to fail if assets reference the new or updated data types"));
#else
						UE_LOGF(LogLiveCoding, Warning, "%ls, %ls", *Success, TEXT("data type changes may cause unexpected failures"));
#endif
					}
				}
				else
				{
					UE_LOGF(LogLiveCoding, Display, "%ls", *Success);
				}
				break;
			case commands::PostCompileResult::NoChanges:
				LastResults = ELiveCodingCompileResult::NoChanges;
				UE_LOGF(LogLiveCoding, Display, "%ls, %ls", *Success, TEXT("no code changes detected"));
				break;
			case commands::PostCompileResult::Cancelled:
				LastResults = ELiveCodingCompileResult::Cancelled;
				UE_LOGF(LogLiveCoding, Error, "Live coding canceled");
				break;
			case commands::PostCompileResult::Failure:
				LastResults = ELiveCodingCompileResult::Failure;
				UE_LOGF(LogLiveCoding, Error, "Live coding failed, please see Live console for more information");
				break;
			default:
				LastResults = ELiveCodingCompileResult::Failure;
				check(false);
			}

#if WITH_EDITOR
			static const FText SuccessText = LOCTEXT("Success", "Live coding succeeded");
			static const FText NoChangesText = LOCTEXT("NoChanges", "No code changes were detected.");
			static const FText FailureText = LOCTEXT("Failed", "Live coding failed");
			static const FText FailureDetailText = LOCTEXT("FailureDetail", "Please see Live Coding console for more information.");
			static const FText CancelledText = LOCTEXT("Cancelled", "Live coding cancelled");
			static const FText ReinstancingText = LOCTEXT("Reinstancing", "Data type changes may cause packaging to fail if assets reference the new or updated data types.");
			static const FText DisabledText = LOCTEXT("ReinstancingDisabled", "Data type changes with re-instancing disabled is not supported and will likely lead to a crash.");

			switch (GPostCompileResult)
			{
			case commands::PostCompileResult::Success:
				if (bHasReinstancingOccurred)
				{
					if (!IsReinstancingEnabled())
					{
						ShowNotification(true, SuccessText, &DisabledText);
					}
					else
					{
						ShowNotification(true, SuccessText, &ReinstancingText);
					}
				}
				else
				{
					ShowNotification(true, SuccessText, nullptr);
				}
				break;
			case commands::PostCompileResult::NoChanges:
				ShowNotification(true, SuccessText, &NoChangesText);
				break;
			case commands::PostCompileResult::Cancelled:
				ShowNotification(false, CancelledText, nullptr);
				break;
			case commands::PostCompileResult::Failure:
				ShowNotification(false, FailureText, &FailureDetailText);
				break;
			default:
				check(false);
			}
#endif
		}
		else
		{
			Reload->Reset();
		}
	}
	GTriggerReload = false;
}

#if WITH_EDITOR
void FLiveCodingModule::ShowNotification(bool Success, const FText& Title, const FText* SubText)
{
	FNotificationInfo Info(Title);
	Info.ExpireDuration = 5.0f;
	Info.bUseSuccessFailIcons = true;
	if (SubText)
	{
		Info.SubText = *SubText;
	}
	if (TSharedPtr<SNotificationItem> CompileNotification = FSlateNotificationManager::Get().AddNotification(Info))
	{
		CompileNotification->SetCompletionState(Success ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
	}
}
#endif

ILiveCodingModule::FOnPatchCompleteDelegate& FLiveCodingModule::GetOnPatchCompleteDelegate()
{
	return OnPatchCompleteDelegate;
}

void FLiveCodingModule::StartLiveCodingAsync(ELiveCodingStartupMode StartupMode)
{
	// Make sure we can setup the console path correctly
	if (!SetupConsolePath())
	{
		State = EState::NotRunning;
		return;
	}

	if (IsRunningCommandlet())
	{
		StartLiveCoding(StartupMode);
	}
	else
	{
		auto Task = [this, StartupMode]()
		{
			StartLiveCoding(StartupMode);
		};

		FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(Task), TStatId());
	}
}

bool FLiveCodingModule::SetupConsolePath()
{
	// Setup the console path
	GLiveCodingConsolePath = ConsolePathVariable->GetString();
	if (!FPaths::FileExists(GLiveCodingConsolePath))
	{
		// Check from the executable as the user might have specified different base dir
		FString CodingConsolePathFromExecutable = FullEngineDirFromExecutable / DefaultConsolePath;
		FPaths::CollapseRelativeDirectories(CodingConsolePathFromExecutable);
		if (!FPaths::FileExists(CodingConsolePathFromExecutable))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Executable"), FText::FromString(GLiveCodingConsolePath));
			const static FText FormatString = LOCTEXT("LiveCodingMissingExecutable", "Unable to start live coding session. Missing executable '{Executable}'. Use the LiveCoding.ConsolePath console variable to modify.");
			EnableErrorText = FText::Format(FormatString, Args);
			UE_LOGF(LogLiveCoding, Error, "Unable to start live coding session. Missing executable '%ls'. Use the LiveCoding.ConsolePath console variable to modify.", *GLiveCodingConsolePath);
			return false;
		}

		GLiveCodingConsolePath = CodingConsolePathFromExecutable;

		// If we found the console from the executable path, chances are users wants the project from there as well
		const FString ExecutablePath = FPaths::GetPath(FPlatformProcess::ExecutablePath());
		FString SourceProjectFromExecutable = ExecutablePath / FPaths::GetProjectFilePath();
		FPaths::NormalizeDirectoryName(SourceProjectFromExecutable);
		FPaths::CollapseRelativeDirectories(SourceProjectFromExecutable);
		if(SourceProjectFromExecutable.Len() > 0 && FPaths::FileExists(SourceProjectFromExecutable))
		{
			SourceProjectVariable->Set(*SourceProjectFromExecutable);
		}
	}
	return true;
}

bool FLiveCodingModule::StartLiveCoding(ELiveCodingStartupMode StartupMode)
{
	EnableErrorText = FText::GetEmpty();
	if (!HasStarted())
	{
		State = EState::Starting;
		if (StartupMode == ELiveCodingStartupMode::AutomaticButHidden)
		{
			GLiveCodingConsoleArguments = L"-Hidden";
		}
		else
		{
			GLiveCodingConsoleArguments = L"";
		}

		// Make sure there aren't any hot reload modules already active
		if (!CanEnableForSession())
		{
			EnableErrorText = LOCTEXT("NoLiveCodingCompileAfterHotReload", "Live Coding cannot be enabled while hot-reloaded modules are active. Please close the editor and build from your IDE before restarting.");
			UE_LOGF(LogLiveCoding, Error, "Unable to start live coding session. Some modules have already been hot reloaded.");
			State = EState::NotRunning;
			return false;
		}

		// Get the source project filename
		const FString SourceProject = SourceProjectVariable->GetString();
		if (SourceProject.Len() > 0 && !FPaths::FileExists(SourceProject))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ProjectFile"), FText::FromString(SourceProject));
			const static FText FormatString = LOCTEXT("LiveCodingMissingProjectFile", "Unable to start live coding session. Unable to find source project file '{ProjectFile}'.");
			EnableErrorText = FText::Format(FormatString, Args);
			UE_LOGF(LogLiveCoding, Error, "Unable to start live coding session. Unable to find source project file '%ls'.", *SourceProject);
			State = EState::NotRunning;
			return false;
		}

		// Create a mutex that allows UBT to detect that we shouldn't hot-reload into this executable. The handle to it will be released automatically when the process exits.
		FString ExecutablePath = FPaths::ConvertRelativePathToFull(FPlatformProcess::ExecutablePath());

		FString MutexName = TEXT("Global\\LiveCoding_");
		for (int Idx = 0; Idx < ExecutablePath.Len(); Idx++)
		{
			TCHAR Character = ExecutablePath[Idx];
			if (Character == '/' || Character == '\\' || Character == ':')
			{
				MutexName += '+';
			}
			else
			{
				MutexName += Character;
			}
		}

		GlobalMutex = CreateMutex(NULL, FALSE, *MutexName);
		if (!GlobalMutex)
		{
			State = EState::NotRunning;
			return false;
		}

		UE_LOGF(LogLiveCoding, Display, "Starting LiveCoding");

		// Enable external build system
		LppUseExternalBuildSystem();

		const TCHAR* VfsPaths = BuildSettings::GetVfsPathsWide();
		if (*VfsPaths)
		{
			// We have to send each entry separately because LC settings strings must be < 256 bytes
			TStringBuilder<256> VfsEntry;

			if (SourceProject.Len() > 0)
			{
				const TCHAR* FirstSemi = FCString::Strchr(VfsPaths, ';');
				FString VfsRoot = FPaths::GetPath(FString(MakeStringView(VfsPaths, FirstSemi - VfsPaths)));
				VfsEntry.Append(VfsRoot).AppendChar('/').Append(FPaths::GetBaseFilename(SourceProject)).AppendChar(';').Append(FPaths::GetPath(SourceProject));
#if PLATFORM_WINDOWS
				VfsEntry.ToString(); // Ensure null terminated
				for (TCHAR* It=VfsEntry.GetData(); *It; ++It)
					if (*It == '/')
						*It = '\\';
#endif
				LppApplySettingString("Vfs", *VfsEntry);
				VfsEntry.Reset();
			}

			const TCHAR* LastSemi = VfsPaths;
			while (true)
			{
				const TCHAR* FirstSemi = FCString::Strchr(LastSemi, ';');
				if (!FirstSemi)
					break;
				const TCHAR* SecondSemi = FCString::Strchr(FirstSemi + 1, ';');
				if (!FirstSemi)
					break;
				VfsEntry.Append(LastSemi, SecondSemi - LastSemi);
				LppApplySettingString("Vfs", *VfsEntry);
				VfsEntry.Reset();
				LastSemi = SecondSemi + 1;
			}
		}

		// Enable the server
		FString ProjectPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()).ToLower();
		FString ProcessGroup = FString::Printf(TEXT("UE_%s_0x%08x"), FApp::GetProjectName(), GetTypeHash(ProjectPath));

		LppRegisterProcessGroup(TCHAR_TO_ANSI(*ProcessGroup));

		SetBuildArguments();

#if WITH_EDITOR
		if (IsReinstancingEnabled())
		{
			LppSetReinstancingFlow(true);
		}

		if (GEditor != nullptr)
		{
			LppDisableCompileFinishNotification();
		}
#endif

		if (StartupMode == ELiveCodingStartupMode::Automatic)
		{
			ShowConsole(true);
		}

		// Mark it as started
		State = EState::RunningAndEnabled;

		auto FinalizeStartup = [this]()
			{
				// Register a delegate to listen for new modules loaded from this point onwards
				ModulesChangedDelegateHandle = FModuleManager::Get().OnModulesChanged().AddRaw(this, &FLiveCodingModule::OnModulesChanged);

				bUpdateModulesInTick = true;

				FCoreDelegates::OnPreExit.AddLambda( [this]()
				{
					StopLiveCoding();
				});
			};

		// Configure all the current modules. For non-commandlets, schedule it to be done in the first Tick() so we can batch everything together.
		if (IsRunningCommandlet())
		{
			UpdateModules(true);
			FinalizeStartup();
		}
		else
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(FinalizeStartup), TStatId(), nullptr, ENamedThreads::GameThread);
		}
	}
	return true;
}

void FLiveCodingModule::StopLiveCoding()
{
	if (CallbackCookie)
	{
		LppShutdown();
		FCoreDelegates::OnEndFrame.Remove(EndFrameDelegateHandle);

		IConsoleManager& ConsoleManager = IConsoleManager::Get();
		ConsoleManager.UnregisterConsoleObject(SourceProjectVariable);
		ConsoleManager.UnregisterConsoleObject(ConsolePathVariable);
		ConsoleManager.UnregisterConsoleObject(CompileCommand);
		ConsoleManager.UnregisterConsoleObject(CompileSyncCommand);
		ConsoleManager.UnregisterConsoleObject(EnableCommand);

		// Unregister from the dll notifications
		FNtDllFunction UnregisterFunc("LdrUnregisterDllNotification");
		UnregisterFunc(CallbackCookie);
		CallbackCookie = nullptr;
	}

	if (GlobalMutex)
	{
		ReleaseMutex(GlobalMutex);
		GlobalMutex = nullptr;
	}
}

void FLiveCodingModule::SetBuildArguments()
{
	FScopeLock lock(&SetBuildArgumentsCs);

	// Build the command line
	FString KnownTargetName = FPlatformMisc::GetUBTTargetName();
	if (KnownTargetName != LastKnownTargetName)
	{
		LastKnownTargetName = KnownTargetName;
		FString Arguments = FString::Printf(TEXT("%s %s %s"),
			*KnownTargetName,
			FPlatformMisc::GetUBTPlatform(),
			LexToString(FApp::GetBuildConfiguration()));

		UE_LOGF(LogLiveCoding, Display, "LiveCodingConsole Arguments: %ls", *Arguments);

		FString SourceProject = SourceProjectVariable->GetString();
		if (SourceProject.Len() > 0)
		{
			Arguments += FString::Printf(TEXT(" -Project=\"%s\""), *FPaths::ConvertRelativePathToFull(SourceProject));
		}
		LppSetBuildArguments(*Arguments);
	}
}

void FLiveCodingModule::UpdateModules(bool bAllowStarting)
{
	if (HasStarted(bAllowStarting))
	{
		// The target name might have changed now that we are loading early
		SetBuildArguments();

		TArray<ModuleChange> Changes;
		TArray<uintptr_t> Pages;
		FGraphEventRef Task;
		{
			ReservePagesGlobalData& Global = GetReservePagesGlobalData();
			FScopeLock lock(&Global.ModuleChangeCs);
			Swap(Changes, Global.ModuleChanges);
			Swap(Pages, Global.ReservedPages);
			Task = Global.ReservePagesTaskRef;
			Global.LastReservePagesModuleCount = 0;
		}

		if (Task.IsValid())
		{
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(Task);
		}

#if IS_MONOLITHIC
		wchar_t FullFilePath[WINDOWS_MAX_PATH];
		verify(GetModuleFileName(hInstance, FullFilePath, UE_ARRAY_COUNT(FullFilePath)));
		LppEnableModule(FullFilePath);
#else
		// Collect the list of preloaded modules
		TSet<FName> PreloadedFileNames;
		{
			FModuleManager& Manager = FModuleManager::Get();
			for (FName ModuleName : Settings->PreloadNamedModules)
			{
				TArray<FName> OutModules;
				Manager.FindModules(*ModuleName.ToString(), OutModules);
				for (FName OutModule : OutModules)
				{
					FModuleStatus ModuleStatus;
					if (Manager.QueryModule(OutModule, ModuleStatus))
					{
						if (!ModuleStatus.FilePath.IsEmpty())
						{
							PreloadedFileNames.Add(FName(FPaths::GetBaseFilename(ModuleStatus.FilePath, true)));
						}
					}
				}
			}
		}

		TArray<FString> EnableModules;
		TArray<FString> LazyLoadModules;
		for (const ModuleChange& Change : Changes)
		{
			if (Change.bLoaded)
			{
				ConfiguredModules.Add(Change.FullName);
				FString FullFilePath(Change.FullName.ToString());
				if (ShouldPreloadModule(PreloadedFileNames, FullFilePath))
				{
					EnableModules.Add(FullFilePath);
				}
				else
				{
					LazyLoadModules.Add(FullFilePath);
				}
			}
			else
			{
				ConfiguredModules.Remove(Change.FullName);
			}
		}

		if (!EnableModules.IsEmpty() || !LazyLoadModules.IsEmpty())
		{
			TArray<const TCHAR*> EnableModuleFileNames;
			for (const FString& EnableModule : EnableModules)
			{
				EnableModuleFileNames.Add(*EnableModule);
			}

			TArray<const TCHAR*> LazyLoadModuleFileNames;
			for (const FString& LazyLoadModule : LazyLoadModules)
			{
				LazyLoadModuleFileNames.Add(*LazyLoadModule);
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(LppEnableModulesEx);
				void* LppEnableModulesExToken = LppEnableModulesEx(
					EnableModuleFileNames.GetData(), EnableModuleFileNames.Num(),
					LazyLoadModuleFileNames.GetData(), LazyLoadModuleFileNames.Num(),
					Pages.GetData(), Pages.Num());
				LppPendingTokens.Add(LppEnableModulesExToken);
			}
		}
#endif
	}
}

FLiveCodingModule::ReservePagesGlobalData& FLiveCodingModule::GetReservePagesGlobalData()
{
	static ReservePagesGlobalData Singleton;
	return Singleton;
}

void FLiveCodingModule::ReservePagesTask()
{
	for (;;)
	{
		ReservePagesGlobalData& Global = GetReservePagesGlobalData();
		TArray<ModuleChange> Changes;
		{
			FScopeLock lock(&Global.ModuleChangeCs);
			uint32 NewModules = Global.ModuleChanges.Num() - Global.LastReservePagesModuleCount;
			if (NewModules < NewModuleCountThreshhold)
			{
				Global.ReservePagesTaskRef = FGraphEventRef();
				return;
			}
			Changes.Append(Global.ModuleChanges.GetData() + Global.LastReservePagesModuleCount, NewModules);
			Global.LastReservePagesModuleCount = Global.ModuleChanges.Num();
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(LiveCodingReservePages);

		TArray<uintptr_t> ModuleBases;
		ModuleBases.Reserve(Changes.Num());
		for (const ModuleChange& Change : Changes)
		{
			if (Change.bLoaded)
			{
				HMODULE Handle = ::GetModuleHandle(*Change.FullName.ToString());
				ModuleBases.Add(reinterpret_cast<uintptr_t>(Handle));
			}
		}

		TArray<uintptr_t> Pages;
		ReservePages(Pages, ModuleBases, DefaultPadding, DefaultPageAlignment);

		{
			FScopeLock lock(&Global.ModuleChangeCs);
			Global.ReservedPages.Append(Pages);
		}
	}
}


void FLiveCodingModule::OnModulesChanged(FName ModuleName, EModuleChangeReason Reason)
{
#if !IS_MONOLITHIC
	if (Reason == EModuleChangeReason::ModuleLoaded)
	{
		// Assume that Tick() won't be called if we're running a commandlet
		if (IsRunningCommandlet())
		{
			UpdateModules(false);
		}
		else
		{
			bUpdateModulesInTick = true;
		}
	}
#endif
}

bool FLiveCodingModule::ShouldPreloadModule(const TSet<FName>& PreloadedFileNames, const FString& FullFilePath) const
{

	// Perform some name based checks
	{
		FString FileName = FPaths::GetBaseFilename(FullFilePath, true);
		FName Name(FileName, FNAME_Find);
		if (Name != NAME_None && PreloadedFileNames.Contains(Name))
		{
			return true;
		}
	}

	if (FullFilePath.StartsWith(FullProjectDir, ESearchCase::IgnoreCase))
	{
		if (Settings->bPreloadProjectModules == Settings->bPreloadProjectPluginModules)
		{
			return Settings->bPreloadProjectModules;
		}

		if(FullFilePath.StartsWith(FullProjectPluginsDir, ESearchCase::IgnoreCase))
		{
			return Settings->bPreloadProjectPluginModules;
		}
		else
		{
			return Settings->bPreloadProjectModules;
		}
	}
	else
	{
		if (FApp::IsEngineInstalled())
		{
			return false;
		}

		if (Settings->bPreloadEngineModules == Settings->bPreloadEnginePluginModules)
		{
			return Settings->bPreloadEngineModules;
		}

		if(FullFilePath.StartsWith(FullEnginePluginsDir, ESearchCase::IgnoreCase))
		{
			return Settings->bPreloadEnginePluginModules;
		}
		else
		{
			return Settings->bPreloadEngineModules;
		}
	}
}

void FLiveCodingModule::BeginReload()
{
	if (GLiveCodingModule != nullptr)
	{
		if (!GLiveCodingModule->Reload.IsValid())
		{
			GLiveCodingModule->bHasReinstancingOccurred = false;
			GLiveCodingModule->bHasPatchBeenLoaded = false;
			GPostCompileResult = commands::PostCompileResult::Success;
#if WITH_EDITOR && WITH_LIVE_CODING
			GLiveCodingModule->Reload.Reset(new FReload(EActiveReloadType::LiveCoding, TEXT("LIVECODING"), *GLog));
			GLiveCodingModule->Reload->SetEnableReinstancing(GLiveCodingModule->IsReinstancingEnabled());
			GLiveCodingModule->Reload->SetSendReloadCompleteNotification(false);
#elif WITH_LIVE_CODING
			GLiveCodingModule->Reload.Reset(new FNullReload(*GLiveCodingModule));
#endif
		}
	}
}

bool FLiveCodingModule::IsReinstancingEnabled() const
{
#if WITH_EDITOR
	return Settings->bEnableReinstancing;
#else
	return false;
#endif
}

bool FLiveCodingModule::AutomaticallyCompileNewClasses() const
{
	return Settings->bAutomaticallyCompileNewClasses;
}

void FLiveCodingModule::OnDllNotification(unsigned int Reason, const void* DataPtr, void* Context)
{
	FLiveCodingModule* This = reinterpret_cast<FLiveCodingModule*>(Context);

	enum
	{
		LDR_DLL_NOTIFICATION_REASON_LOADED = 1,
		LDR_DLL_NOTIFICATION_REASON_UNLOADED = 2,
	};

	struct FNotificationData
	{
		uint32 Flags;
		const UNICODE_STRING& FullPath;
		const UNICODE_STRING& BaseName;
		UPTRINT	Base;
	};
	const auto& Data = *(FNotificationData*)DataPtr;
	FString FullPath = FString::ConstructFromPtrSize(Data.FullPath.Buffer, Data.FullPath.Length / sizeof(Data.FullPath.Buffer[0]));
	FPaths::NormalizeFilename(FullPath);

	switch (Reason)
	{
	case LDR_DLL_NOTIFICATION_REASON_LOADED:
		This->OnDllLoaded(FullPath);
		break;

	case LDR_DLL_NOTIFICATION_REASON_UNLOADED:
		This->OnDllUnloaded(FullPath);
		break;
	}
}

void FLiveCodingModule::OnDllLoaded(const FString& FullPath)
{
	if (IsUEDll(FullPath))
	{
		ReservePagesGlobalData& Global = GetReservePagesGlobalData();
		FScopeLock lock(&Global.ModuleChangeCs);
		Global.ModuleChanges.Emplace(ModuleChange{ FName(FullPath), true });
		if (!Global.ReservePagesTaskRef.IsValid() && Global.ModuleChanges.Num() - Global.LastReservePagesModuleCount >= NewModuleCountThreshhold)
		{
			Global.ReservePagesTaskRef = FFunctionGraphTask::CreateAndDispatchWhenReady([this]() { ReservePagesTask(); });
		}
	}
}

void FLiveCodingModule::OnDllUnloaded(const FString& FullPath)
{
	if (IsUEDll(FullPath))
	{
		ReservePagesGlobalData& Global = GetReservePagesGlobalData();
		FScopeLock lock(&Global.ModuleChangeCs);
		Global.ModuleChanges.Emplace(ModuleChange{ FName(FullPath), false });
	}
}


bool FLiveCodingModule::IsUEDll(const FString& FullPath)
{
	// Ignore patches.
	if (IsPatchDll(FullPath))
	{
		return false;
	}

	// Dll must be in the engine or project dir
	if (!FullPath.StartsWith(FullEngineDir, ESearchCase::IgnoreCase) &&
		!FullPath.StartsWith(FullEnginePluginsDir, ESearchCase::IgnoreCase) &&
		!FullPath.StartsWith(FullProjectDir, ESearchCase::IgnoreCase) &&
		!FullPath.StartsWith(FullProjectPluginsDir, ESearchCase::IgnoreCase) &&
		!FullPath.StartsWith(FullEngineDirFromExecutable, ESearchCase::IgnoreCase) &&
		!FullPath.StartsWith(FullEnginePluginsDirFromExecutable, ESearchCase::IgnoreCase) &&
		!FullPath.StartsWith(FullProjectDirFromExecutable, ESearchCase::IgnoreCase) &&
		!FullPath.StartsWith(FullProjectPluginsDirFromExecutable, ESearchCase::IgnoreCase))
	{
		return false;
	}
	return true;
}

bool FLiveCodingModule::IsPatchDll(const FString& FullPath)
{
	// If the Dll ends with ".patch_#.dll", then ignore it
	FString Name = FPaths::GetBaseFilename(FullPath, true);
	FString Extension = FPaths::GetExtension(Name);
	if (!Extension.StartsWith("patch_", ESearchCase::IgnoreCase))
	{
		return false;
	}

	for (int Index = 6; Index < Extension.Len(); ++Index)
	{
		if (!FChar::IsDigit(Extension[Index]))
		{
			return false;
		}
	}
	return true;
}

// Invoked from LC_ClientCommandActions
void LiveCodingBeginPatch()
{
	GHasLoadedPatch = true;
	// If we are beginning a patch from a restart from the console, we need to create the reload object
	FLiveCodingModule::BeginReload();
}

// Invoked from LC_ClientCommandActions
void LiveCodingEndCompile()
{
	GIsCompileActive = false;
}

// Invoked from LC_ClientCommandActions
void LiveCodingPreCompile()
{
	UE_LOGF(LogLiveCoding, Display, "Starting Live Coding compile.");
	GIsCompileActive = true;
	if (GLiveCodingModule != nullptr)
	{
		GLiveCodingModule->BeginReload();
	}
}

// Invoked from LC_ClientCommandActions
void LiveCodingPostCompile(commands::PostCompileResult PostCompileResult)
{
	GPostCompileResult = PostCompileResult;
	GIsCompileActive = false;
}

// Invoked from LC_ClientCommandActions
void LiveCodingTriggerReload()
{
	GTriggerReload = true;
}

#include "Windows/HideWindowsPlatformTypes.h"
#undef LOCTEXT_NAMESPACE

#endif // LC_VERSION == 1
