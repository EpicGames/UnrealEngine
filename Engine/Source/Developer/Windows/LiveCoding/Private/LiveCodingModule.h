// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if LC_VERSION == 1

#include "ILiveCodingModule.h"
#include "LiveCodingSettings.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/Text.h"
#include "Async/TaskGraphFwd.h"
#include <atomic>
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"

struct IConsoleCommand;
class IConsoleVariable;
class ISettingsSection;
class ULiveCodingSettings;
class FOutputDevice;

#if WITH_EDITOR
class FReload;
#else
class FNullReload;
#endif

class FLiveCodingModule final : public ILiveCodingModule
{
public:
	FLiveCodingModule();
	~FLiveCodingModule();

	// IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// ILiveCodingModule implementation
	virtual void EnableByDefault(bool bInEnabled) override;
	virtual bool IsEnabledByDefault() const override;
	virtual void EnableForSession(bool bInEnabled) override;
	virtual bool IsEnabledForSession() const override;
	virtual const FText& GetEnableErrorText() const override;
	virtual bool AutomaticallyCompileNewClasses() const override;
	virtual bool CanEnableForSession() const override;
	virtual bool HasStarted() const override;
	virtual void ShowConsole() override;
	virtual void Compile() override;
	virtual bool Compile(ELiveCodingCompileFlags CompileFlags, ELiveCodingCompileResult* Result) override;
	virtual bool IsCompiling() const override;
	virtual void Tick() override;
	virtual FOnPatchCompleteDelegate& GetOnPatchCompleteDelegate() override;

	static void BeginReload();

private:
	void AttemptSyncLivePatching();
	static void OnDllNotification(unsigned int Reason, const void* DataPtr, void* Context);
	void OnDllLoaded(const FString& FullPath);
	void OnDllUnloaded(const FString& FullPath);
	bool IsUEDll(const FString& FullPath);
	bool IsPatchDll(const FString& FullPath);
	void HideConsole();
	void EnableConsoleCommand(FOutputDevice& out);
	void StartLiveCodingAsync(ELiveCodingStartupMode StartupMode);
	bool SetupConsolePath();
	bool StartLiveCoding(ELiveCodingStartupMode StartupMode);
	void StopLiveCoding();
	void OnModulesChanged(FName ModuleName, EModuleChangeReason Reason);
	void UpdateModules(bool bAllowStarting);
	bool ShouldPreloadModule(const TSet<FName>& PreloadedFileNames, const FString& FullFilePath) const;
	bool IsReinstancingEnabled() const;
	void WaitForStartup();
	bool HasStarted(bool bAllowStarting) const;
	void ShowConsole(bool bAllowStarting);
	void SetBuildArguments();

	struct ModuleChange
	{
		FName FullName;
		bool bLoaded;
	};
	struct ReservePagesGlobalData
	{
		FCriticalSection ModuleChangeCs;
		TArray<ModuleChange> ModuleChanges;
		TArray<uintptr_t> ReservedPages;
		int LastReservePagesModuleCount = 0;
		FGraphEventRef ReservePagesTaskRef;
	};
	static ReservePagesGlobalData& GetReservePagesGlobalData();
	static void ReservePagesTask();

#if WITH_EDITOR
	void ShowNotification(bool Success, const FText& Title, const FText* SubText);
#endif

private:
	enum class EState
	{
		NotRunning,
		Starting,
		Running,
		RunningAndEnabled,
	};

	ULiveCodingSettings* Settings;
	TSharedPtr<ISettingsSection> SettingsSection;
	bool bSettingsEnabledLastTick = false;
	bool bEnableReinstancingLastTick = false;
	bool bBuildArgumentsSet = false;
	std::atomic<EState> State = EState::NotRunning;
	std::atomic<bool> bUpdateModulesInTick = false;
	bool bHasReinstancingOccurred = false;
	bool bHasPatchBeenLoaded = false;
	ELiveCodingCompileResult LastResults = ELiveCodingCompileResult::Success;
	TSet<FName> ConfiguredModules;
	TArray<void*> LppPendingTokens;
	void* CallbackCookie = nullptr;
	FString LastKnownTargetName = FString();

	FText EnableErrorText;

	const FString FullEngineDir;
	const FString FullEnginePluginsDir;
	const FString FullProjectDir;
	const FString FullProjectPluginsDir;
	FString FullEngineDirFromExecutable;
	FString FullEnginePluginsDirFromExecutable;
	FString FullProjectDirFromExecutable;
	FString FullProjectPluginsDirFromExecutable;

	IConsoleCommand* EnableCommand;
	IConsoleCommand* CompileCommand;
	IConsoleCommand* CompileSyncCommand;
	IConsoleVariable* ConsolePathVariable;
	IConsoleVariable* SourceProjectVariable;
	FDelegateHandle EndFrameDelegateHandle;
	FDelegateHandle ModulesChangedDelegateHandle;
	FOnPatchCompleteDelegate OnPatchCompleteDelegate;

	FCriticalSection SetBuildArgumentsCs;

#if WITH_EDITOR
	TUniquePtr<FReload> Reload;
#else
	TUniquePtr<FNullReload> Reload;
#endif

	void* GlobalMutex = NULL;
};

#endif // LC_VERSION == 1
