// Copyright Epic Games, Inc. All Rights Reserved.

#if LC_VERSION == 2

#include "LiveCodingModule2.h"
#include "LiveCodingLog.h"
#include "BuildSettings.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "LPP_API_x64_CPP.h"
#include "Windows/HideWindowsPlatformTypes.h"

IMPLEMENT_MODULE(FLiveCodingModule, LiveCoding)

#define LOCTEXT_NAMESPACE "LiveCodingModule"

LLM_DEFINE_TAG(LiveCoding);

void PrecompileHook(lpp::LppPrecompileHookId, const wchar_t* const recompiledModulePath, unsigned int filesToCompileCount) { UE_LOGF(LogLiveCoding, Display, "Compiling..."); }
LPP_PRECOMPILE_HOOK(PrecompileHook);

void CompileSuccessHook(lpp::LppCompileSuccessHookId, const wchar_t* const recompiledModulePath, const wchar_t* const recompiledSourcePath) { UE_LOGF(LogLiveCoding, Display, "Compile success!"); } 
LPP_COMPILE_SUCCESS_HOOK(CompileSuccessHook);

void CompileErrorHook(lpp::LppCompileErrorHookId, const wchar_t* const recompiledModulePath, const wchar_t* const recompiledSourcePath, const wchar_t* const compilerOutput)
{
	UE_LOGF(LogLiveCoding, Display, "Compile error:"); 
	UE_LOGF(LogLiveCoding, Display, "%ls", WCHAR_TO_TCHAR(compilerOutput)); 
}
LPP_COMPILE_ERROR_HOOK(CompileErrorHook);


void LinkStartHook(lpp::LppLinkStartHookId, const wchar_t* const recompiledModulePath) { UE_LOGF(LogLiveCoding, Display, "Linking..."); }
LPP_LINK_START_HOOK(LinkStartHook);

void LinkSuccessHook(lpp::LppLinkSuccessHookId, const wchar_t* const recompiledModulePath) { UE_LOGF(LogLiveCoding, Display, "Link sucess!"); }
LPP_LINK_SUCCESS_HOOK(LinkSuccessHook);

void LinkErrorHook(lpp::LppLinkErrorHookId, const wchar_t* const recompiledModulePath, const wchar_t* const linkerOutput)
{
	UE_LOGF(LogLiveCoding, Display, "Link error:"); 
	UE_LOGF(LogLiveCoding, Display, "%ls", WCHAR_TO_TCHAR(linkerOutput)); 
}
LPP_LINK_ERROR_HOOK(LinkErrorHook);

//void PostcompileHook(lpp::LppPostcompileHookId, const wchar_t* const recompiledModulePath, unsigned int filesToCompileCount) { UE_LOGF(LogLiveCoding, Display, "Compile done..."); } 
//LPP_POSTCOMPILE_HOOK(PostcompileHook);

void PrePatchHook(lpp::LppHotReloadPrepatchHookId, const wchar_t* const recompiledModulePath, const wchar_t* const* const modifiedFiles, unsigned int modifiedFilesCount, const wchar_t* const* const modifiedClassLayouts, unsigned int modifiedClassLayoutsCount)
{
	UE_LOGF(LogLiveCoding, Display, "Patching...");
}
LPP_HOTRELOAD_PREPATCH_HOOK(PrePatchHook);

void PostPatchHook(lpp::LppHotReloadPostpatchHookId, const wchar_t* const recompiledModulePath, const wchar_t* const* const modifiedFiles, unsigned int modifiedFilesCount, const wchar_t* const* const modifiedClassLayouts, unsigned int modifiedClassLayoutsCount)
{
	UE_LOGF(LogLiveCoding, Display, "Patching done!");
}
LPP_HOTRELOAD_POSTPATCH_HOOK(PostPatchHook);


FLiveCodingModule::FLiveCodingModule()
{
}

FLiveCodingModule::~FLiveCodingModule()
{
}

uint32 StartBroker()
{
	// Check if broker is running.. Let's do it every time for now.. don't think it takes long time
	bool isBrokerRunning = false;
	FPlatformProcess::FProcEnumerator ProcessEnumerator;
	while (ProcessEnumerator.MoveNext())
	{
		const FString ProcessName = ProcessEnumerator.GetCurrent().GetName();
		if (ProcessName == TEXT("LPP_Broker.exe"))
		{
			return ProcessEnumerator.GetCurrent().GetPID();
		}
	}

	FString LivePlusPlusPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Binaries/") / FPlatformProcess::GetBinariesSubdirectory() / TEXT("LivePlusPlus"));

	UE_LOGF(LogLiveCoding, Display, "Starting LiveCoding Broker.");

	FString BrokerExe = LivePlusPlusPath / TEXT("LPP_Broker.exe");
	uint32 ProcessId;
	if (FPlatformProcess::CreateProc(*BrokerExe, TEXT(""), true, true, true, &ProcessId, 0, NULL, NULL, NULL).IsValid())
	{
		return ProcessId;
	}
	UE_LOGF(LogLiveCoding, Error, "Failed to start broker '%ls'.", *BrokerExe);
	return 0;
}

DWORD WINAPI HookThread(void* ModulePtr)
{
	FLiveCodingModule& Module = *(FLiveCodingModule*)ModulePtr;
	HANDLE HotKeyMutex = CreateMutexW(NULL, 0, L"Global\\LiveCodingHotKeyMutex");
	if (!HotKeyMutex)
	{
		UE_LOGF(LogLiveCoding, Error, "CreateMutexW failed when setting up listener for hot key");
		return 0;
	}
	ON_SCOPE_EXIT{ CloseHandle(HotKeyMutex); HotKeyMutex = NULL; };

	WNDCLASSA wc = { };
	wc.lpfnWndProc   = DefWindowProc;
	wc.hInstance     = hInstance;
	wc.lpszClassName = "LiveCodingHiddenWindow";
	RegisterClassA(&wc);
	ON_SCOPE_EXIT{ UnregisterClassA("LiveCodingHiddenWindow", hInstance); };

	HWND hwnd = CreateWindowExA(0, "LiveCodingHiddenWindow", "RawInputListener", 0, 0, 0, 0, 0, HWND_MESSAGE, NULL, NULL, NULL);
	if (!hwnd)
	{
		UE_LOGF(LogLiveCoding, Error, "CreateWindowEx failed when setting up listener for hot key");
		return 0;
	}
	ON_SCOPE_EXIT{ DestroyWindow(hwnd); };

	RAWINPUTDEVICE rid { 0x01, 0x06, RIDEV_INPUTSINK, hwnd };
	if (!RegisterRawInputDevices(&rid, 1, sizeof(rid)))
	{
		UE_LOGF(LogLiveCoding, Error, "RegisterRawInputDevices failed when setting up listener for hot key");
		return 1;
	}
	
	MSG msg;
	while (GetMessageW(&msg, NULL, 0, 0))
	{
		TranslateMessage(&msg);
		DispatchMessage(&msg);
		if (msg.message != WM_INPUT || !((GetAsyncKeyState(VK_CONTROL) & 0x8000) && (GetAsyncKeyState(VK_MENU) & 0x8000) && (GetAsyncKeyState(VK_F11) & 0x8000)))
		{
			continue;
		}

		DWORD waitResult = WaitForSingleObject(HotKeyMutex, INFINITE);
		if (waitResult == WAIT_OBJECT_0)
		{
			uint32 BrokerPid = StartBroker();
			ReleaseMutex(HotKeyMutex);
			Module.OnHotKeyPressed(BrokerPid);
		}
		else if (waitResult == WAIT_ABANDONED)
		{
			UE_LOGF(LogLiveCoding, Error, "HotKeyMutex was abandoned. Press hotkey again");
		}
    }
    return 0;
}

void FLiveCodingModule::OnHotKeyPressed(uint32 BrokerPid)
{
	LastSeenBrokerPid = BrokerPid;
	HotReloadRequested = true;

	if (!EndFrameDelegateHandle.IsValid())
	{
		EndFrameDelegateHandle = FCoreDelegates::OnEndFrame.AddRaw(this, &FLiveCodingModule::Tick);
	}
}

void FLiveCodingModule::StartupModule()
{
	LLM_SCOPE_BYTAG(LiveCoding);

	ListenKeyPressThread = CreateThread(NULL, 0, HookThread, this, 0, NULL);
}

void FLiveCodingModule::ShutdownModule()
{
	if (ListenKeyPressThread)
	{
		PostThreadMessage(GetThreadId(ListenKeyPressThread), WM_QUIT, 0, 0);
		WaitForSingleObject(ListenKeyPressThread, INFINITE);
		CloseHandle(ListenKeyPressThread);
	}

	FCoreDelegates::OnEndFrame.Remove(EndFrameDelegateHandle);
	EndFrameDelegateHandle = {};

	if (Agent)
	{
		lpp::LppDestroySynchronizedAgent(Agent);
		delete Agent;
		Agent = nullptr;
	}
}

void FLiveCodingModule::EnableByDefault(bool bEnable)
{
}

bool FLiveCodingModule::IsEnabledByDefault() const
{
	return true;
}

void FLiveCodingModule::EnableForSession(bool bEnable)
{
}

bool FLiveCodingModule::IsEnabledForSession() const
{
	return true;
}

const FText& FLiveCodingModule::GetEnableErrorText() const
{
	return EnableErrorText;
}

bool FLiveCodingModule::AutomaticallyCompileNewClasses() const
{
	return false;
}

bool FLiveCodingModule::CanEnableForSession() const
{
#if !IS_MONOLITHIC
	FModuleManager& ModuleManager = FModuleManager::Get();
	if(ModuleManager.HasAnyOverridenModuleFilename())
	{
		return false;
	}
#endif
	return true;
}

bool FLiveCodingModule::HasStarted() const
{
	return true;
}

void FLiveCodingModule::ShowConsole()
{
}

void FLiveCodingModule::Compile()
{
}

bool FLiveCodingModule::Compile(ELiveCodingCompileFlags CompileFlags, ELiveCodingCompileResult* Result)
{
	return false;
}

bool FLiveCodingModule::IsCompiling() const
{
	return false;
}

void FLiveCodingModule::Tick()
{
	if (!CreateAgent())
	{
		FCoreDelegates::OnEndFrame.Remove(EndFrameDelegateHandle);
		EndFrameDelegateHandle = {};
		return;
	}

	if (HotReloadRequested && Agent->IsReady())
	{
		HotReloadRequested = false;
		Agent->ScheduleReload();
	}

	// listen to hot-reload and hot-restart requests
	if (Agent->WantsReload(lpp::LPP_RELOAD_OPTION_SYNCHRONIZE_WITH_RELOAD))
	{
		// client code can do whatever it wants here, e.g. synchronize across several threads, the network, etc.
		// ...
		Agent->Reload(lpp::LPP_RELOAD_BEHAVIOUR_WAIT_UNTIL_CHANGES_ARE_APPLIED);
	}

	if (Agent->WantsRestart())
	{
		// client code can do whatever it wants here, e.g. finish logging, abandon threads, etc.
		// ...
		Agent->Restart(lpp::LPP_RESTART_BEHAVIOUR_INSTANT_TERMINATION, 0u, nullptr);
	}
}

ILiveCodingModule::FOnPatchCompleteDelegate& FLiveCodingModule::GetOnPatchCompleteDelegate()
{
	return OnPatchCompleteDelegate;
}

bool FLiveCodingModule::CreateAgent()
{
	if (Agent)
	{
		if (AgentBrokerPid == LastSeenBrokerPid)
		{
			return true;
		}

		delete Agent;
		Agent = nullptr;
	}

	AgentBrokerPid = LastSeenBrokerPid;

	FString LivePlusPlusPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Binaries/") / FPlatformProcess::GetBinariesSubdirectory() / TEXT("LivePlusPlus"));
	const TCHAR* VfsPaths = BuildSettings::GetVfsPathsWide();

	UE_LOGF(LogLiveCoding, Display, "Creating LiveCoding Agent.");

	using namespace lpp;
	LppLocalPreferences LocalPreferences = LppCreateDefaultLocalPreferences();

	LppProjectPreferences ProjectPreferences = LppCreateDefaultProjectPreferences();
	ProjectPreferences.hotReload.sourcePathFilters = ".gen.cpp";
	ProjectPreferences.compiler.removeShowIncludes = true;
	ProjectPreferences.compiler.removeSourceDependencies = true;
	ProjectPreferences.compiler.captureEnvironment = false; // We have no vcvarsXx in our toolchain dir
	ProjectPreferences.linker.captureEnvironment = false; // We have no vcvarsXx in our toolchain dir
	ProjectPreferences.exceptionHandler.isEnabled = false;
	ProjectPreferences.vfs.entries = VfsPaths;

	Agent = new LppSynchronizedAgent(LppCreateSynchronizedAgentWithPreferences(&LocalPreferences, *LivePlusPlusPath, &ProjectPreferences));

	if (!LppIsValidSynchronizedAgent(Agent))
	{
		delete Agent;
		Agent = nullptr;
		return false;
	}

#if IS_MONOLITHIC
	Agent->EnableModule(FPlatformProcess::ExecutablePath(), lpp::LPP_MODULES_OPTION_ALL_IMPORT_MODULES, nullptr, nullptr);
#else
	// Enable all prior modules.
	TArray<FModuleStatus> ModuleStatuses;
	TArray<const TCHAR*> ModuleNames;

	FModuleManager::Get().QueryModules(ModuleStatuses);
	ModuleNames.Reserve(ModuleStatuses.Num());

	for (const FModuleStatus& ModuleStatus : ModuleStatuses)
	{
		if (ModuleStatus.bIsLoaded)
		{
			ModuleNames.Add(*ModuleStatus.FilePath);
		}
	}
	
	Agent->EnableModules(ModuleNames.GetData(), ModuleNames.Num(), lpp::LPP_MODULES_OPTION_ALL_IMPORT_MODULES, nullptr, nullptr);
	Agent->EnableAutomaticHandlingOfDynamicallyLoadedModules(nullptr, nullptr);
#endif

	return true;
}

#undef LOCTEXT_NAMESPACE

#endif // LC_VERSION == 2
