// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if LC_VERSION == 2

#include "ILiveCodingModule.h"
#include "LiveCodingSettings.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/Text.h"
#include "Async/TaskGraphFwd.h"
#include <atomic>

namespace lpp { struct LppSynchronizedAgent; }


class FLiveCodingModule final : public ILiveCodingModule
{
public:
	FLiveCodingModule();
	~FLiveCodingModule();

	// IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// ILiveCodingModule implementation
	virtual void EnableByDefault(bool bInEnabled) override;
	virtual bool IsEnabledByDefault() const override;
	virtual void EnableForSession(bool bInEnabled) override;
	virtual bool IsEnabledForSession() const override;
	virtual const FText& GetEnableErrorText() const override;
	virtual bool AutomaticallyCompileNewClasses() const override;
	virtual bool CanEnableForSession() const override;
	virtual bool HasStarted() const override;
	virtual void ShowConsole() override;
	virtual void Compile() override;
	virtual bool Compile(ELiveCodingCompileFlags CompileFlags, ELiveCodingCompileResult* Result) override;
	virtual bool IsCompiling() const override;
	virtual void Tick() override;
	virtual FOnPatchCompleteDelegate& GetOnPatchCompleteDelegate() override;

	void OnHotKeyPressed(uint32 BrokerPid);
private:
	bool CreateAgent();

	FText EnableErrorText;
	FDelegateHandle EndFrameDelegateHandle;
	FOnPatchCompleteDelegate OnPatchCompleteDelegate;
	TAtomic<uint32> LastSeenBrokerPid = 0;
	TAtomic<bool> HotReloadRequested = false;

	uint32 AgentBrokerPid = 0;
	lpp::LppSynchronizedAgent* Agent = nullptr;

	void* ListenKeyPressThread = nullptr;
};

#endif // LC_VERSION == 2
