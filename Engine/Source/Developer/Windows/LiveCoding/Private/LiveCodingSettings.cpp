// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveCodingSettings.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveCodingSettings)

ULiveCodingSettings::ULiveCodingSettings(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	FProperty* EngineModulesProperty = StaticClass()->FindPropertyByName("bPreloadEngineModules");
	check(EngineModulesProperty != nullptr);

	FProperty* EnginePluginModulesProperty = StaticClass()->FindPropertyByName("bPreloadEnginePluginModules");
	check(EnginePluginModulesProperty != nullptr);

	if (FApp::IsEngineInstalled())
	{
		EngineModulesProperty->ClearPropertyFlags(CPF_Edit);
		EnginePluginModulesProperty->ClearPropertyFlags(CPF_Edit);
	}

	bPreloadProjectModules = true;
	bPreloadProjectPluginModules = true;
}

