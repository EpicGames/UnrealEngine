// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "LiveCodingSettings.generated.h"

UENUM()
enum class ELiveCodingStartupMode : uint8
{
	Automatic UMETA(DisplayName = "Start automatically and show console"),
	AutomaticButHidden UMETA(DisplayName = "Start automatically but hide console until summoned"),
	Manual UMETA(DisplayName = "Manual"),
};

UCLASS(config=EditorPerProjectUserSettings, meta=(DisplayName="Live Coding"))
class ULiveCodingSettings : public UObject
{
    GENERATED_BODY()

public:
    UPROPERTY(config, EditAnywhere, Category=General, Meta=(ConfigRestartRequired=false, DisplayName="Enable Live Coding"))
    bool bEnabled;

    UPROPERTY(config, EditAnywhere, Category=General, Meta=(ConfigRestartRequired=true, EditCondition="bEnabled"))
    ELiveCodingStartupMode Startup;

    UPROPERTY(config, EditAnywhere, Category=General, Meta=(EditCondition="bEnabled"))
	bool bEnableReinstancing;

	UPROPERTY(config, EditAnywhere, Category=General, Meta=(EditCondition="bEnabled", DisplayName="Automatically Compile Newly Added C++ Classes"))
	bool bAutomaticallyCompileNewClasses;

	UPROPERTY(config, EditAnywhere, Category=Modules, Meta=(ConfigRestartRequired=true, EditCondition="bEnabled"))
    bool bPreloadEngineModules;

    UPROPERTY(config, EditAnywhere, Category=Modules, Meta=(ConfigRestartRequired=true, EditCondition="bEnabled"))
    bool bPreloadEnginePluginModules;

    UPROPERTY(config, EditAnywhere, Category=Modules, Meta=(ConfigRestartRequired=true, EditCondition="bEnabled"))
    bool bPreloadProjectModules;

    UPROPERTY(config, EditAnywhere, Category=Modules, Meta=(ConfigRestartRequired=true, EditCondition="bEnabled"))
    bool bPreloadProjectPluginModules;

    UPROPERTY(config, EditAnywhere, Category=Modules, Meta=(ConfigRestartRequired=true, EditCondition="bEnabled", DisplayName="Names of modules, without target prefix, to always preload.  Wildcards are supported"))
    TArray<FName> PreloadNamedModules; 

	ULiveCodingSettings(const FObjectInitializer& Initializer);
};
