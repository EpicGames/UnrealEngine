// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"

#define LIVE_CODING_MODULE_NAME "LiveCoding"

class FText;

enum class ELiveCodingCompileFlags : uint8
{
	None = 0,
	WaitForCompletion = 1 << 0,
};
ENUM_CLASS_FLAGS(ELiveCodingCompileFlags)

enum class ELiveCodingCompileResult : uint8
{
	Success,			// Compile completed and there were changes 
	NoChanges,			// Compile completed and there were no changes
	InProgress,			// Compile started but wait for completion was not specified
	CompileStillActive,	// A prior compile request is still active
	NotStarted,			// Live coding monitor could not be started
	Failure,			// Complete completed but there was an error
	Cancelled,			// Compile was cancelled
};

class ILiveCodingModule : public IModuleInterface
{
public:

	// Enable by default tracks the value in the live coding settings.
	// If enabled by default and the startup mode is one of the automatic settings, then
	// live coding console will be started when the module is started.
	virtual void EnableByDefault(bool bEnabled) = 0;
	virtual bool IsEnabledByDefault() const = 0;

	// The enabled for session state and started state are NOT independent states.  
	// If a request is made to enabled live coding (automatically or manually) and live coding 
	// hasn't already been started, then it is started.  If and only if the console starts 
	// will both the started state and the enabled for session state will be true.
	// If enable for session is set to false, the console isn't stopped so the started state
	// will remain true.
	//
	// Started = false, Enabled = false => no request to enable has ever been made or it failed to start
	// Started = true, Enabled = true => a request to enable was made and it started properly
	// Started = true, Enabled = false => has been started but later disabled
	// Started = false, Enabled = true => impossible state.
	virtual bool HasStarted() const = 0;
	virtual void EnableForSession(bool bEnabled) = 0;
	virtual bool IsEnabledForSession() const = 0;
	virtual bool CanEnableForSession() const = 0;

	virtual const FText& GetEnableErrorText() const = 0;
	virtual bool AutomaticallyCompileNewClasses() const = 0;

	virtual void ShowConsole() = 0;
	virtual void Compile() = 0;
	virtual bool Compile(ELiveCodingCompileFlags CompileFlags, ELiveCodingCompileResult* Result) = 0;
	virtual bool IsCompiling() const = 0;
	virtual void Tick() = 0;

	DECLARE_MULTICAST_DELEGATE(FOnPatchCompleteDelegate);
	virtual FOnPatchCompleteDelegate& GetOnPatchCompleteDelegate() = 0;
};

