// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class LiveCodingServer : ModuleRules
{
	public LiveCodingServer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");
        PrivateDependencyModuleNames.Add("Json");
        PrivateDependencyModuleNames.Add("LiveCoding");
		PrivateDependencyModuleNames.Add("VisualStudioDTE");

        AddEngineThirdPartyPrivateStaticDependencies(Target, "Distorm");

        if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDefinitions.Add("LC_VERSION=1");

			PrivateIncludePaths.Add("Developer/Windows/LiveCoding/Private");
			PrivateIncludePaths.Add("Developer/Windows/LiveCoding/Private/External");

			PrivateDependencyModuleNames.Add("DiaSdk");
		}

		if (Target.Configuration == UnrealTargetConfiguration.Debug)
		{
			PrivateDefinitions.Add("LC_DEBUG=1");
		}
		else
		{
			PrivateDefinitions.Add("LC_DEBUG=0");
		}
		
		// Allow precompiling when generating project files so we can get intellisense
		if(!Target.bGenerateProjectFiles)
		{
			PrecompileForTargets = PrecompileTargetsType.None;
		}
	}
}
