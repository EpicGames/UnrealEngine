// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_AllocatorStats.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include <inttypes.h>
#include <intrin.h>

#include "Windows/WindowsHWrapper.h"
#include "Windows/AllowWindowsPlatformAtomics.h"
// END EPIC MOD

AllocatorStats::AllocatorStats(void)
	: m_allocationCount(0ull)
	, m_memorySize(0ull)
{
}


void AllocatorStats::RegisterAllocation(size_t size)
{
	::InterlockedIncrement(&m_allocationCount);
	::InterlockedAdd64(reinterpret_cast<volatile LONG64*>(&m_memorySize), static_cast<LONG64>(size));
}


void AllocatorStats::UnregisterAllocation(size_t size)
{
	::InterlockedDecrement(&m_allocationCount);
	::InterlockedAdd64(reinterpret_cast<volatile LONG64*>(&m_memorySize), -static_cast<LONG64>(size));
}


void AllocatorStats::Merge(const AllocatorStats& stats)
{
	::InterlockedAdd64(reinterpret_cast<volatile LONG64*>(&m_allocationCount), static_cast<LONG64>(stats.m_allocationCount));
	::InterlockedAdd64(reinterpret_cast<volatile LONG64*>(&m_memorySize), static_cast<LONG64>(stats.m_memorySize));
}


void AllocatorStats::Print(const char* name) const
{
	LC_LOG_TELEMETRY("Allocator \"%s\"", name);

	LC_LOG_INDENT_TELEMETRY;
	LC_LOG_TELEMETRY("Allocation count: %" PRId64, m_allocationCount);
	LC_LOG_TELEMETRY("Size: %" PRId64 " (%.3f KB, %.3f MB)", m_memorySize, m_memorySize / 1024.0f, m_memorySize / 1048576.0f);
}


uint64_t AllocatorStats::GetAllocationCount(void) const
{
	return m_allocationCount;
}


uint64_t AllocatorStats::GetMemorySize(void) const
{
	return m_memorySize;
}

// BEGIN EPIC MOD
#include "Windows/HideWindowsPlatformAtomics.h"
// END EPIC MOD


#endif