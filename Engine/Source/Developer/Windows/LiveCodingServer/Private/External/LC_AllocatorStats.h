// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include <stdint.h>
// END EPIC MOD

// thread-safe
class AllocatorStats
{
public:
	AllocatorStats(void);

	void RegisterAllocation(size_t size);
	void UnregisterAllocation(size_t size);

	void Merge(const AllocatorStats& stats);

	void Print(const char* name) const;

	uint64_t GetAllocationCount(void) const;
	uint64_t GetMemorySize(void) const;

private:
	uint64_t m_allocationCount;
	uint64_t m_memorySize;
};


#endif