// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Allocators.h"
#include "LC_Symbols.h"
#include "LC_DirectoryCache.h"

PoolAllocator<PoolAllocatorMultiThreadPolicy> g_symbolAllocator("Symbols", sizeof(symbols::Symbol), alignof(symbols::Symbol), 8192u);
MicroAllocator g_immutableStringAllocator("Immutable strings", 8u);
PoolAllocator<PoolAllocatorMultiThreadPolicy> g_contributionAllocator("Contributions", sizeof(symbols::Contribution), alignof(symbols::Contribution), 8192u);
PoolAllocator<PoolAllocatorMultiThreadPolicy> g_compilandAllocator("Compilands", sizeof(symbols::Compiland), alignof(symbols::Compiland), 8192u);
PoolAllocator<PoolAllocatorMultiThreadPolicy> g_amalgamatedCompilandAllocator("Amalgamated compilands", sizeof(symbols::AmalgamatedCompiland), alignof(symbols::AmalgamatedCompiland), 8192u);
PoolAllocator<PoolAllocatorMultiThreadPolicy> g_dependencyAllocator("Dependencies", sizeof(symbols::Dependency), alignof(symbols::Dependency), 8192u);
PoolAllocator<PoolAllocatorMultiThreadPolicy> g_directoryAllocator("Directory", sizeof(DirectoryCache::Directory), alignof(DirectoryCache::Directory), 8192u);
Mallocator g_objFileAllocator("OBJ files", 8u);
Mallocator g_rawCoffAllocator("Raw COFF", 8u);
Mallocator g_rawElfAllocator("Raw ELF files", 8u);
Mallocator g_relocatableFileAllocator("Relocatable files", 8u);
Mallocator g_executableFileAllocator("Executable files", 8u);
Mallocator g_tlogFileAllocator("Tlog files", 8u);


#endif