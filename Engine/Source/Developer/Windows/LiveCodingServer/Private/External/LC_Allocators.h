// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Mallocator.h"
#include "LC_PoolAllocator.h"
#include "LC_MicroAllocator.h"


namespace memory
{
	template <typename T>
	inline void Destruct(T* ptr)
	{
		if (ptr)
		{
			ptr->~T();
		}
	}

	template <typename Allocator, typename T>
	inline void Free(Allocator* allocator, T* ptr, size_t size)
	{
		allocator->Free(ptr, size);
	}

	template <typename Allocator, typename T>
	inline void Delete(Allocator* allocator, T* ptr, size_t size)
	{
		Destruct(ptr);
		Free(allocator, ptr, size);
	}
}


// macros in order to use custom allocators for allocations

// raw allocations
#define LC_ALLOC(_allocator, _size, _alignment)			(_allocator)->Allocate((_size), (_alignment))
#define LC_FREE(_allocator, _ptr, _size)				(_allocator)->Free((_ptr), (_size))

// new & delete replacements
#define LC_NEW_ALIGNED(_allocator, _type, _alignment)	new (LC_ALLOC(_allocator, sizeof(_type), _alignment)) _type
#define LC_NEW(_allocator, _type)						LC_NEW_ALIGNED(_allocator, _type, alignof(_type))
#define LC_DELETE(_allocator, _ptr, _size)				memory::Delete((_allocator), (_ptr), (_size))

// special allocators
extern PoolAllocator<PoolAllocatorMultiThreadPolicy> g_symbolAllocator;
extern MicroAllocator g_immutableStringAllocator;
extern PoolAllocator<PoolAllocatorMultiThreadPolicy> g_contributionAllocator;
extern PoolAllocator<PoolAllocatorMultiThreadPolicy> g_compilandAllocator;
extern PoolAllocator<PoolAllocatorMultiThreadPolicy> g_amalgamatedCompilandAllocator;
extern PoolAllocator<PoolAllocatorMultiThreadPolicy> g_dependencyAllocator;
extern PoolAllocator<PoolAllocatorMultiThreadPolicy> g_directoryAllocator;
extern Mallocator g_objFileAllocator;
extern Mallocator g_rawCoffAllocator;
extern Mallocator g_rawElfAllocator;
extern Mallocator g_relocatableFileAllocator;
extern Mallocator g_executableFileAllocator;
extern Mallocator g_tlogFileAllocator;


#endif