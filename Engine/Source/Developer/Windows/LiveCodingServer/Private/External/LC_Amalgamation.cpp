// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Amalgamation.h"
#include "LC_StringUtil.h"
#include "LC_Filesystem.h"
#include "LC_GrowingMemoryBlock.h"
#include "LC_FileAttributeCache.h"
// BEGIN EPIC MOD
#include "LC_Allocators.h"
#include "LC_AppSettings.h"
#include "LC_Logging.h"
// END EPIC MOD

namespace
{
	static const char* const LPP_AMALGAMATION_PART= ".lpp_part.";
	static const wchar_t* const LPP_AMALGAMATION_PART_WIDE = L".lpp_part.";

	struct Database
	{
		static const uint32_t MAGIC_NUMBER = 0x4C505020;	// "LPP "
		static const uint32_t VERSION = 8u;

		struct Dependency
		{
			std::string filename;
			uint64_t timestamp;
		};
	};


	// helper function to generate the database path for an .obj file
	static std::wstring GenerateDatabasePath(const symbols::ObjPath& objPath)
	{
		std::wstring path = string::ToWideString(objPath.c_str());
		path = Filesystem::RemoveExtension(path.c_str()).GetString();
		path += L".ldb";

		return path;
	}


	// helper function to generate a timestamp for a file
	static uint64_t GenerateTimestamp(const wchar_t* path)
	{
		const Filesystem::PathAttributes& attributes = Filesystem::GetAttributes(path);
		return Filesystem::GetLastModificationTime(attributes);
	}


	// helper function to generate a database dependency for a file
	static Database::Dependency GenerateDatabaseDependency(const ImmutableString& path)
	{
		return Database::Dependency { path.c_str(), GenerateTimestamp(string::ToWideString(path).c_str()) };
	}


	// helper function to generate a database dependency for a file, normalizing the path to the file
	static Database::Dependency GenerateNormalizedDatabaseDependency(const ImmutableString& path)
	{
		const std::wstring widePath = string::ToWideString(path);
		const std::wstring normalizedPath = Filesystem::NormalizePath(widePath.c_str()).GetString();
		return Database::Dependency { string::ToUtf8String(normalizedPath).c_str(), GenerateTimestamp(normalizedPath.c_str()) };
	}


	static bool IsMainCompilandCpp(const std::wstring& normalizedDependencySrcPath, const std::wstring& objPath)
	{
		// it should suffice to only check the source filename (without extension) against the object filename (without extension).
		// comparisons involving paths are tricky in this case, because certain build systems like FASTBuild automatically
		// generate unity files, and can do so in different directories, e.g:
		// OBJ: Z:\Intermediate\x64\Debug\Unity11.obj
		// SRC: Z:\Unity\Unity11.cpp
		const std::wstring srcFile = string::ToUpper(Filesystem::RemoveExtension(Filesystem::GetFilename(normalizedDependencySrcPath.c_str()).GetString()).GetString());
		const std::wstring objFile = string::ToUpper(Filesystem::RemoveExtension(Filesystem::GetFilename(objPath.c_str()).GetString()).GetString());
		return string::Contains(objFile.c_str(), srcFile.c_str());
	}

	static bool IsCppOrCFile(const std::wstring& normalizedLowercaseFilename)
	{
		const Filesystem::Path filenameExtension = Filesystem::GetExtension(normalizedLowercaseFilename.c_str());
		const types::vector<std::wstring>& amalgamatedCppFileExtensions = appSettings::GetAmalgamatedCppFileExtensions();

		for (size_t i = 0u; i < amalgamatedCppFileExtensions.size(); ++i)
		{
			if (string::Matches(filenameExtension.GetString(), amalgamatedCppFileExtensions[i].c_str()))
			{
				return true;
			}
		}

		return false;
	}

	static size_t FindNumberOfCppFiles(const types::vector<std::wstring>& includeFiles)
	{
		size_t count = 0u;

		const size_t fileCount = includeFiles.size();
		for (size_t i = 0u; i < fileCount; ++i)
		{
			const std::wstring wideFilename = includeFiles[i];
			const std::wstring lowercaseFilename = string::ToLower(wideFilename);
			const bool isCppOrCFile = IsCppOrCFile(lowercaseFilename);
			if (isCppOrCFile)
			{
				++count;
			}
		}

		return count;
	}

	static void AddFileDependency(symbols::CompilandDB* compilandDb, const ImmutableString& changedSrcFile, const ImmutableString& recompiledObjFile, uint64_t srcFileLastModificationTime, uint64_t moduleLastModificationTime)
	{
		// try updating dependencies for the given file and create a new dependency in case none exists yet
		const auto& insertPair = compilandDb->dependencies.emplace(changedSrcFile, nullptr);
		symbols::Dependency*& dependency = insertPair.first->second;

		if (insertPair.second)
		{
			// insertion was successful, create a new dependency
			dependency = LC_NEW(&g_dependencyAllocator, symbols::Dependency);

			// if the source file is newer than the module it belongs to, let the dependency store the modification time of the module instead of the source.
			// this ensures that changes to a source file which happened before Live++ loaded the module are automatically picked up when re-compiling.
			if (srcFileLastModificationTime > moduleLastModificationTime)
			{
				dependency->lastModification = moduleLastModificationTime;
				dependency->hadInitialChange = true;
			}
			else
			{
				dependency->lastModification = srcFileLastModificationTime;
				dependency->hadInitialChange = false;
			}
			dependency->parentDirectory = nullptr;
		}

		// update entry
		dependency->objPaths.push_back(recompiledObjFile);
	}
}


// serializes values and databases into an in-memory representation
namespace serializationToMemory
{
	bool Write(const void* buffer, size_t size, GrowingMemoryBlock* dbInMemory)
	{
		return dbInMemory->Insert(buffer, size);
	}

	template <typename T>
	bool Write(const T& value, GrowingMemoryBlock* dbInMemory)
	{
		return dbInMemory->Insert(&value, sizeof(T));
	}

	bool Write(const ImmutableString& str, GrowingMemoryBlock* dbInMemory)
	{
		// write length without null terminator and then the string
		const uint32_t lengthWithoutNull = str.GetLength();
		if (!Write(lengthWithoutNull, dbInMemory))
		{
			return false;
		}

		if (!Write(str.c_str(), lengthWithoutNull, dbInMemory))
		{
			return false;
		}

		return true;
	}

	bool Write(const std::string& str, GrowingMemoryBlock* dbInMemory)
	{
		// write length without null terminator and then the string
		const uint32_t lengthWithoutNull = static_cast<uint32_t>(str.length() * sizeof(char));
		if (!Write(lengthWithoutNull, dbInMemory))
		{
			return false;
		}

		if (!Write(str.c_str(), lengthWithoutNull, dbInMemory))
		{
			return false;
		}

		return true;
	}

	bool Write(const std::wstring& str, GrowingMemoryBlock* dbInMemory)
	{
		// write length without null terminator and then the string
		const uint32_t lengthWithoutNull = static_cast<uint32_t>(str.length() * sizeof(wchar_t));
		if (!Write(lengthWithoutNull, dbInMemory))
		{
			return false;
		}

		if (!Write(str.c_str(), lengthWithoutNull, dbInMemory))
		{
			return false;
		}

		return true;
	}

	bool Write(const Database::Dependency& dependency, GrowingMemoryBlock* dbInMemory)
	{
		if (!Write(dependency.filename, dbInMemory))
		{
			return false;
		}

		if (!Write(dependency.timestamp, dbInMemory))
		{
			return false;
		}

		return true;
	}
}


// serializes values and databases from disk
namespace serializationFromDisk
{
	struct ReadBuffer
	{
		const void* data;
		uint64_t leftToRead;
	};


	bool Read(void* memory, size_t size, ReadBuffer* buffer)
	{
		// is there enough data left to read?
		if (buffer->leftToRead < size)
		{
			return false;
		}

		memcpy(memory, buffer->data, size);
		buffer->data = static_cast<const char*>(buffer->data) + size;
		buffer->leftToRead -= size;

		return true;
	}

	template <typename T>
	bool Read(T& value, ReadBuffer* buffer)
	{
		return Read(&value, sizeof(T), buffer);
	}

	bool Read(std::string& str, ReadBuffer* buffer)
	{
		// read length first
		uint32_t length = 0u;
		if (!Read(length, buffer))
		{
			return false;
		}

		// read data
		str.resize(length + 1u, '\0');
		return Read(&str[0], length, buffer);
	}

	bool Read(Database::Dependency& dependency, ReadBuffer* buffer)
	{
		// read filename first
		if (!Read(dependency.filename, buffer))
		{
			return false;
		}

		// read timestamp
		return Read(dependency.timestamp, buffer);
	}


	bool Compare(const void* memory, size_t size, ReadBuffer* buffer)
	{
		// is there enough data left to read?
		if (buffer->leftToRead < size)
		{
			return false;
		}

		const bool identical = (memcmp(memory, buffer->data, size) == 0);
		if (!identical)
		{
			return false;
		}

		buffer->data = static_cast<const char*>(buffer->data) + size;
		buffer->leftToRead -= size;

		return true;
	}

	template <typename T>
	bool Compare(const T& value, ReadBuffer* buffer)
	{
		return Compare(&value, sizeof(T), buffer);
	}

	bool Compare(const ImmutableString& str, ReadBuffer* buffer)
	{
		// compare length first
		const uint32_t length = str.GetLength();
		if (!Compare(length, buffer))
		{
			return false;
		}

		// compare data
		return Compare(str.c_str(), length, buffer);
	}

	bool Compare(const std::string& str, ReadBuffer* buffer)
	{
		// compare length first
		const uint32_t length = static_cast<uint32_t>(str.length() * sizeof(char));
		if (!Compare(length, buffer))
		{
			return false;
		}

		// compare data
		return Compare(str.c_str(), length, buffer);
	}

	bool Compare(const std::wstring& str, ReadBuffer* buffer)
	{
		// compare length first
		const uint32_t length = static_cast<uint32_t>(str.length() * sizeof(wchar_t));
		if (!Compare(length, buffer))
		{
			return false;
		}

		// compare data
		return Compare(str.c_str(), length, buffer);
	}

	bool Compare(const Database::Dependency& dependency, ReadBuffer* buffer)
	{
		// compare filename first
		if (!Compare(dependency.filename, buffer))
		{
			return false;
		}

		// compare timestamp
		return Compare(dependency.timestamp, buffer);
	}
}


bool amalgamation::IsPartOfAmalgamation(const char* normalizedObjPath)
{
	return string::Contains(normalizedObjPath, LPP_AMALGAMATION_PART);
}


bool amalgamation::IsPartOfAmalgamation(const wchar_t* normalizedObjPath)
{
	return string::Contains(normalizedObjPath, LPP_AMALGAMATION_PART_WIDE);
}


std::wstring amalgamation::CreateObjPart(const std::wstring& normalizedFilename)
{
	std::wstring newObjPart(LPP_AMALGAMATION_PART_WIDE);
	newObjPart += Filesystem::RemoveExtension(Filesystem::GetFilename(normalizedFilename.c_str()).GetString()).GetString();
	newObjPart += L".obj";

	return newObjPart;
}


std::wstring amalgamation::CreateObjPath(const std::wstring& normalizedAmalgamatedObjPath, const std::wstring& objPart)
{
	std::wstring newObjPath(normalizedAmalgamatedObjPath);
	newObjPath = Filesystem::RemoveExtension(newObjPath.c_str()).GetString();
	newObjPath += objPart;

	return newObjPath;
}


void amalgamation::SplitAmalgamatedCompilands(
	unsigned int splitAmalgamatedFilesThreshold,
	symbols::CompilandDB* compilandDb,
	symbols::Compiland* compiland,
	const types::vector<std::wstring>& includeFiles,
	const wchar_t* diaCompilandPath,
	const std::wstring& compilandPath,
	FileAttributeCache& fileCache,
	bool generateLogs,
	symbols::TimeStamp moduleLastModificationTime)
{
	const std::wstring normalizedCompilandPath = Filesystem::NormalizePath(compilandPath.c_str()).GetString();
	const symbols::ObjPath objPath(string::ToUtf8String(normalizedCompilandPath));

	const size_t fileCount = includeFiles.size();

	const bool splitAmalgamatedFiles = (splitAmalgamatedFilesThreshold > 1u);
	const size_t cppFileCount = splitAmalgamatedFiles ? FindNumberOfCppFiles(includeFiles) : 0u;

	// make sure to treat single-part compilands as being non-amalgamated, i.e. we don't support
	// recursive amalgamation.
	// in case splitting of amalgamated files is turned off, this automatically takes care of
	// treating every compiland as single-file compiland.
	const bool isPartOfAmalgamation = amalgamation::IsPartOfAmalgamation(compilandPath.c_str());
	if ((!splitAmalgamatedFiles) || (isPartOfAmalgamation) || (cppFileCount < splitAmalgamatedFilesThreshold))
	{
		LC_LOG_DEV("Single .cpp file compiland %s", objPath.c_str());

		// only store source files when splitting amalgamated files in order to save memory
		// in the general case.
		if (splitAmalgamatedFiles)
		{
			// create array of source file indices for this compiland
			compiland->sourceFiles = new symbols::CompilandSourceFiles;
			compiland->sourceFiles->files.reserve(fileCount);
		}

		// this is not an amalgamated compiland
		for (size_t j = 0u; j < fileCount; ++j)
		{
			const std::wstring& normalizedFilename = includeFiles[j];

			if (Filesystem::GetDriveType(normalizedFilename.c_str()) == Filesystem::DriveType::OPTICAL)
			{
				LC_LOG_DEV("Ignoring file %S on optical drive", normalizedFilename.c_str());
			}
			else
			{
				const FileAttributeCache::Data& cacheData = fileCache.UpdateCacheData(normalizedFilename);
				if (cacheData.exists)
				{
					if (generateLogs)
					{
						LC_LOG_DEV("Dependency %S", normalizedFilename.c_str());
					}

					const ImmutableString& sourceFilePath = string::ToUtf8String(normalizedFilename);
					AddFileDependency(compilandDb, sourceFilePath, objPath, cacheData.lastModificationTime, moduleLastModificationTime);

					if (splitAmalgamatedFiles)
					{
						compiland->sourceFiles->files.push_back(sourceFilePath);
					}
				}
				else if (generateLogs)
				{
					LC_LOG_DEV("Missing dependency %S", normalizedFilename.c_str());
				}
			}
		}

		compilandDb->compilands.insert(std::make_pair(objPath, compiland));
		compilandDb->compilandNameToObjOnDisk.emplace(string::ToUtf8String(diaCompilandPath), objPath);
	}
	else
	{
		// this is an amalgamated compiland
		LC_LOG_DEV("Amalgamated .cpp file compiland %s", objPath.c_str());

		// always add a main compiland for the .obj file.
		// some amalgamated files don't store their main .cpp as dependency.
		{
			compiland->type = symbols::Compiland::Type::AMALGAMATION;
			compilandDb->compilands.insert(std::make_pair(objPath, compiland));
			compilandDb->compilandNameToObjOnDisk.insert(std::make_pair(string::ToUtf8String(diaCompilandPath), objPath));
		}

		for (size_t j = 0u; j < fileCount; ++j)
		{
			const std::wstring& normalizedFilename = includeFiles[j];
			const std::wstring lowercaseFilename = string::ToLower(normalizedFilename);

			const ImmutableString& sourceFilePath = string::ToUtf8String(normalizedFilename);

			if (IsCppOrCFile(normalizedFilename))
			{
				if (IsMainCompilandCpp(normalizedFilename, compilandPath))
				{
					LC_LOG_DEV("Main .cpp %S", normalizedFilename.c_str());

					if (Filesystem::GetDriveType(normalizedFilename.c_str()) == Filesystem::DriveType::OPTICAL)
					{
						if (generateLogs)
						{
							LC_LOG_DEV("Ignoring file %S on optical drive", normalizedFilename.c_str());
						}
					}
					else
					{
						const FileAttributeCache::Data& cacheData = fileCache.UpdateCacheData(normalizedFilename);
						if (cacheData.exists)
						{
							if (generateLogs)
							{
								LC_LOG_DEV("Dependency %S", normalizedFilename.c_str());
							}

							AddFileDependency(compilandDb, sourceFilePath, objPath, cacheData.lastModificationTime, moduleLastModificationTime);
						}
						else if (generateLogs)
						{
							LC_LOG_DEV("Missing dependency %S", normalizedFilename.c_str());
						}
					}
				}
				else
				{
					// this is a .cpp file included by the amalgamated file.
					// add a separate compiland and .obj for this file, and update dependencies so that changing
					// this source file will not trigger a build of the amalgamated file.
					LC_LOG_DEV("Included .cpp %S", normalizedFilename.c_str());

					if (Filesystem::GetDriveType(normalizedFilename.c_str()) == Filesystem::DriveType::OPTICAL)
					{
						if (generateLogs)
						{
							LC_LOG_DEV("Ignoring file %S on optical drive", normalizedFilename.c_str());
						}
					}
					else
					{
						const FileAttributeCache::Data& cacheData = fileCache.UpdateCacheData(normalizedFilename);
						if (cacheData.exists)
						{
							if (generateLogs)
							{
								LC_LOG_DEV("Dependency %S", normalizedFilename.c_str());
							}

							// create new .obj path by appending this file name to the real .obj, e.g.
							// Amalgamated.obj turns into Amalgamated.lpp_part.ASingleFile.obj.
							const std::wstring newObjPart = amalgamation::CreateObjPart(Filesystem::NormalizePath(normalizedFilename.c_str()).GetString());
							const std::wstring newObjPath = amalgamation::CreateObjPath(compilandPath, newObjPart);
							const std::wstring normalizedNewObjPath = Filesystem::NormalizePath(newObjPath.c_str()).GetString();

							AddFileDependency(compilandDb, sourceFilePath, string::ToUtf8String(normalizedNewObjPath), cacheData.lastModificationTime, moduleLastModificationTime);

							// create a new compiland matching this .obj.
							// we could use different PDBs for different files when no PCHs are being used, but with
							// our automatic multi-processor compilation this doesn't really gain anything performance-wise
							// and just complicates things.

							// adapt command line to accommodate new .obj path
							const std::wstring& newCommandLine = string::Replace(string::ToWideString(compiland->commandLine), L".obj", newObjPart);

							symbols::Compiland* newCompiland = LC_NEW(&g_compilandAllocator, symbols::Compiland)
							{
								string::ToUtf8String(newObjPath),
								string::ToUtf8String(normalizedFilename),
								compiland->pdbPath,
								compiland->compilerPath,
								string::ToUtf8String(newCommandLine),
								compiland->workingDirectory,
								objPath,												// .obj of the amalgamation
								nullptr,												// file indices

								// note that for the purpose of disambiguating symbols in COFF files,
								// we treat these files as being the amalgamated file.
								// symbols originally coming from amalgamated files need to have the same
								// name as symbols from individual files.
								compiland->uniqueId,

								symbols::Compiland::Type::PART_OF_AMALGAMATION,			// type of file
								compiland->isPartOfLibrary,								// isPartOfLibrary
								false													// wasRecompiled
							};

							compilandDb->compilands.insert(std::make_pair(string::ToUtf8String(normalizedNewObjPath), newCompiland));

							// try updating the amalgamated compiland for the given file and create a new one in case none exists yet
							{
								const auto& insertPair = compilandDb->amalgamatedCompilands.emplace(objPath, nullptr);
								symbols::AmalgamatedCompiland*& amalgamatedCompiland = insertPair.first->second;

								if (insertPair.second)
								{
									// insertion was successful, create a new amalgamated compiland
									amalgamatedCompiland = LC_NEW(&g_amalgamatedCompilandAllocator, symbols::AmalgamatedCompiland);
									amalgamatedCompiland->isSplit = false;
								}

								// update entry
								amalgamatedCompiland->singleParts.push_back(string::ToUtf8String(normalizedNewObjPath));
							}
						}
						else if (generateLogs)
						{
							LC_LOG_DEV("Missing dependency %S", normalizedFilename.c_str());
						}
					}
				}
			}
			else
			{
				// this is a header file. add it as regular dependency for the main amalgamated .obj file
				if (Filesystem::GetDriveType(normalizedFilename.c_str()) == Filesystem::DriveType::OPTICAL)
				{
					if (generateLogs)
					{
						LC_LOG_DEV("Ignoring file %S on optical drive", normalizedFilename.c_str());
					}
				}
				else
				{
					const FileAttributeCache::Data& cacheData = fileCache.UpdateCacheData(normalizedFilename);
					if (cacheData.exists)
					{
						if (generateLogs)
						{
							LC_LOG_DEV("Dependency %S", normalizedFilename.c_str());
						}

						AddFileDependency(compilandDb, sourceFilePath, objPath, cacheData.lastModificationTime, moduleLastModificationTime);
					}
					else if (generateLogs)
					{
						LC_LOG_DEV("Missing dependency %S", normalizedFilename.c_str());
					}
				}
			}
		}
	}
}


bool amalgamation::ReadAndCompareDatabase(const symbols::ObjPath& objPath, const std::wstring& compilerPath, const symbols::Compiland* compiland, const std::wstring& additionalCompilerOptions)
{
	// check if the .obj is there. if not, there is no need to check the database at all.
	{
		const Filesystem::PathAttributes& objAttributes = Filesystem::GetAttributes(string::ToWideString(objPath).c_str());
		if (!Filesystem::DoesExist(objAttributes))
		{
			return false;
		}
	}

	const std::wstring databasePath = GenerateDatabasePath(objPath);
	const Filesystem::PathAttributes& fileAttributes = Filesystem::GetAttributes(databasePath.c_str());
	if (!Filesystem::DoesExist(fileAttributes))
	{
		return false;
	}

	const uint64_t bytesLeftToRead = Filesystem::GetSize(fileAttributes);
	if (bytesLeftToRead == 0u)
	{
		LC_LOG_DEV("Failed to retrieve size of database file %S", databasePath.c_str());
		return false;
	}

	Filesystem::MemoryMappedFile* memoryFile = Filesystem::OpenMemoryMappedFile(databasePath.c_str(), Filesystem::OpenMode::READ);
	if (!memoryFile)
	{
		// database cannot be opened, treat as if a change was detected
		return false;
	}

	// start reading the database from disk, comparing against the compiland's database at the same time
	serializationFromDisk::ReadBuffer readBuffer { Filesystem::GetMemoryMappedFileData(memoryFile), bytesLeftToRead };
	if (!serializationFromDisk::Compare(Database::MAGIC_NUMBER, &readBuffer))
	{
		LC_LOG_DEV("Wrong magic number in database file %S", databasePath.c_str());

		Filesystem::CloseMemoryMappedFile(memoryFile);
		return false;
	}

	if (!serializationFromDisk::Compare(Database::VERSION, &readBuffer))
	{
		LC_LOG_DEV("Version has changed in database file %S", databasePath.c_str());

		Filesystem::CloseMemoryMappedFile(memoryFile);
		return false;
	}

	if (!serializationFromDisk::Compare(compilerPath, &readBuffer))
	{
		LC_LOG_DEV("Compiler path has changed in database file %S", databasePath.c_str());

		Filesystem::CloseMemoryMappedFile(memoryFile);
		return false;
	}

	if (!serializationFromDisk::Compare(GenerateTimestamp(compilerPath.c_str()), &readBuffer))
	{
		LC_LOG_DEV("Compiler timestamp has changed in database file %S", databasePath.c_str());

		Filesystem::CloseMemoryMappedFile(memoryFile);
		return false;
	}

	if (!serializationFromDisk::Compare(compiland->commandLine, &readBuffer))
	{
		LC_LOG_DEV("Compiland compiler options have changed in database file %S", databasePath.c_str());

		Filesystem::CloseMemoryMappedFile(memoryFile);
		return false;
	}

	if (!serializationFromDisk::Compare(additionalCompilerOptions, &readBuffer))
	{
		LC_LOG_DEV("Additional compiler options have changed in database file %S", databasePath.c_str());

		Filesystem::CloseMemoryMappedFile(memoryFile);
		return false;
	}

	if (!serializationFromDisk::Compare(GenerateNormalizedDatabaseDependency(compiland->srcPath), &readBuffer))
	{
		LC_LOG_DEV("Source file has changed in database file %S", databasePath.c_str());

		Filesystem::CloseMemoryMappedFile(memoryFile);
		return false;
	}

	// dependencies need to be treated differently, because the current list of files might differ from the one
	// stored in the database. this is not a problem however, because the database is always kept up-to-date as
	// soon as a file was compiled.
	// we need to read all files from the database and check their timestamp against the timestamp of the file
	// on disk.
	{
		uint32_t count = 0u;
		if (!serializationFromDisk::Read(count, &readBuffer))
		{
			LC_LOG_DEV("Failed to read dependency count in database file %S", databasePath.c_str());

			Filesystem::CloseMemoryMappedFile(memoryFile);
			return false;
		}

		for (uint32_t i = 0u; i < count; ++i)
		{
			Database::Dependency dependency = {};
			if (!serializationFromDisk::Read(dependency, &readBuffer))
			{
				LC_LOG_DEV("Failed to read dependency in database file %S", databasePath.c_str());

				Filesystem::CloseMemoryMappedFile(memoryFile);
				return false;
			}

			// check dependency timestamp
			const Filesystem::PathAttributes& attributes = Filesystem::GetAttributes(string::ToWideString(dependency.filename).c_str());
			if (Filesystem::GetLastModificationTime(attributes) != dependency.timestamp)
			{
				LC_LOG_DEV("Dependency has changed in database file %S", databasePath.c_str());

				Filesystem::CloseMemoryMappedFile(memoryFile);
				return false;
			}
		}
	}

	// no change detected
	Filesystem::CloseMemoryMappedFile(memoryFile);
	return true;
}


void amalgamation::WriteDatabase(const symbols::ObjPath& objPath, const std::wstring& compilerPath, const symbols::Compiland* compiland, const std::wstring& additionalCompilerOptions)
{
	// first serialize the database to memory and then write it to disk in one go.
	// note that we write the database to a temporary file first, and then move it to its final destination.
	// because moving is atomic, this ensures that databases are either fully written or not at all.
	GrowingMemoryBlock dbInMemory(1u * 1024u * 1024u);
	if (!serializationToMemory::Write(Database::MAGIC_NUMBER, &dbInMemory))
	{
		LC_LOG_DEV("Failed to serialize database for compiland %s", objPath.c_str());
		return;
	}

	if (!serializationToMemory::Write(Database::VERSION, &dbInMemory))
	{
		LC_LOG_DEV("Failed to serialize database for compiland %s", objPath.c_str());
		return;
	}

	if (!serializationToMemory::Write(compilerPath, &dbInMemory))
	{
		LC_LOG_DEV("Failed to serialize database for compiland %s", objPath.c_str());
		return;
	}

	if (!serializationToMemory::Write(GenerateTimestamp(compilerPath.c_str()), &dbInMemory))
	{
		LC_LOG_DEV("Failed to serialize database for compiland %s", objPath.c_str());
		return;
	}

	if (!serializationToMemory::Write(compiland->commandLine, &dbInMemory))
	{
		LC_LOG_DEV("Failed to serialize database for compiland %s", objPath.c_str());
		return;
	}

	if (!serializationToMemory::Write(additionalCompilerOptions, &dbInMemory))
	{
		LC_LOG_DEV("Failed to serialize database for compiland %s", objPath.c_str());
		return;
	}

	// the source file itself is treated as a dependency
	if (!serializationToMemory::Write(GenerateNormalizedDatabaseDependency(compiland->srcPath), &dbInMemory))
	{
		LC_LOG_DEV("Failed to serialize database for compiland %s", objPath.c_str());
		return;
	}

	// write all file dependencies
	{
		const bool hasDependencies = (compiland->sourceFiles != nullptr);
		const uint32_t count = hasDependencies
			? static_cast<uint32_t>(compiland->sourceFiles->files.size())
			: 0u;

		if (!serializationToMemory::Write(count, &dbInMemory))
		{
			LC_LOG_DEV("Failed to serialize database for compiland %s", objPath.c_str());
			return;
		}

		if (hasDependencies)
		{
			const types::vector<ImmutableString>& sourceFiles = compiland->sourceFiles->files;
			for (uint32_t i = 0u; i < count; ++i)
			{
				const ImmutableString& sourcePath = sourceFiles[i];
				if (!serializationToMemory::Write(GenerateDatabaseDependency(sourcePath), &dbInMemory))
				{
					LC_LOG_DEV("Failed to serialize database for compiland %s", objPath.c_str());
					return;
				}
			}
		}
	}

	const std::wstring databasePath = GenerateDatabasePath(objPath);
	std::wstring tempDatabasePath = databasePath;
	tempDatabasePath += L".tmp";

	if (!Filesystem::CreateFileWithData(tempDatabasePath.c_str(), dbInMemory.GetData(), dbInMemory.GetSize()))
	{
		LC_LOG_DEV("Failed to write database for compiland %s", objPath.c_str());
		return;
	}

	Filesystem::Move(tempDatabasePath.c_str(), databasePath.c_str());
}


void amalgamation::DeleteDatabase(const symbols::ObjPath& objPath)
{
	const std::wstring& databasePath = GenerateDatabasePath(objPath);
	Filesystem::DeleteIfExists(databasePath.c_str());
}


#endif