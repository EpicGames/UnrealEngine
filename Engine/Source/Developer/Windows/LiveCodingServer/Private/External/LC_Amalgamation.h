// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Symbols.h"

class FileAttributeCache;


namespace amalgamation
{
	bool IsPartOfAmalgamation(const char* normalizedObjPath);
	bool IsPartOfAmalgamation(const wchar_t* normalizedObjPath);

	// creates part of the .obj file for .cpp files that are part of an amalgamation, e.g.
	// turns C:\AbsoluteDir\SourceFile.cpp into .lpp_part.SourceFile.obj
	std::wstring CreateObjPart(const std::wstring& normalizedFilename);

	// creates a full .obj path for single .obj files that are part of an amalgamation, e.g.
	// turns C:\AbsoluteDir\Amalgamated.obj into C:\AbsoluteDir\Amalgamated.lpp_part.SourceFile.obj
	std::wstring CreateObjPath(const std::wstring& normalizedAmalgamatedObjPath, const std::wstring& objPart);


	void SplitAmalgamatedCompilands(
		unsigned int splitAmalgamatedFilesThreshold,
		symbols::CompilandDB* compilandDb,
		symbols::Compiland* compiland,
		const types::vector<std::wstring>& includeFiles,
		const wchar_t* diaCompilandPath,
		const std::wstring& compilandPath,
		FileAttributeCache& fileCache,
		bool generateLogs,
		symbols::TimeStamp moduleLastModificationTime);

	// dependency database handling
	
	// reads a database from disk and compares it against the compiland's data.
	// returns true if the database was read correctly and no change was detected.
	bool ReadAndCompareDatabase(const symbols::ObjPath& objPath, const std::wstring& compilerPath, const symbols::Compiland* compiland, const std::wstring& additionalCompilerOptions);

	// writes a compiland's dependency database to disk
	void WriteDatabase(const symbols::ObjPath& objPath, const std::wstring& compilerPath, const symbols::Compiland* compiland, const std::wstring& additionalCompilerOptions);

	// deletes a compiland's dependency database from disk
	void DeleteDatabase(const symbols::ObjPath& objPath);
}


#endif