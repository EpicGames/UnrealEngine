// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_AppSettings.h"
#include "LC_Process.h"
#include "LC_Filesystem.h"
#include "LC_StringUtil.h"
// BEGIN EPIC MOD
//#include "LC_App.h"
// END EPIC MOD
// BEGIN EPIC MOD
#include "LC_Logging.h"
// END EPIC MOD

// BEGIN EPIC MOD
#include "Windows/AllowWindowsPlatformTypes.h"
#include <ShlObj.h>
#include "Windows/HideWindowsPlatformTypes.h"
// END EPIC MOD

namespace
{
	// cache for compiler and linker path
#if LC_64_BIT
	static const wchar_t* const VS2017_COMPILER_PATH = L"bin\\hostx64\\x64\\cl.exe";
	static const wchar_t* const VS2015_AND_EARLIER_COMPILER_PATH = L"bin\\amd64\\cl.exe";
	static const wchar_t* const VS2017_LINKER_PATH = L"bin\\hostx64\\x64\\link.exe";
	static const wchar_t* const VS2015_AND_EARLIER_LINKER_PATH = L"bin\\amd64\\link.exe";
#else
	static const wchar_t* const VS2017_COMPILER_PATH = L"bin\\hostx86\\x86\\cl.exe";
	static const wchar_t* const VS2015_AND_EARLIER_COMPILER_PATH = L"bin\\cl.exe";
	static const wchar_t* const VS2017_LINKER_PATH = L"bin\\hostx86\\x86\\link.exe";
	static const wchar_t* const VS2015_AND_EARLIER_LINKER_PATH = L"bin\\link.exe";
#endif

	static const wchar_t* const COMPILER_EXE = L"cl.exe";
	static const wchar_t* const LINKER_EXE = L"link.exe";

	static std::wstring g_cachedCompilerPath;
	static std::wstring g_cachedLinkerPath;
	static types::vector<std::wstring> g_cachedAmalgamatedCppFileExtensions;

	static std::wstring DeterminePath(const SettingString* setting, const wchar_t* const type, const wchar_t* const vs2017Path, const wchar_t* const vs2015AndEarlierPath, const wchar_t* exeName)
	{
		// absolute paths can be used as they are, relative paths are supposed to be relative to the Live++ executable
		std::wstring path;
		const bool isRelativePath = Filesystem::IsRelativePath(setting->GetValue());
		if (isRelativePath)
		{
			path = Filesystem::GetDirectory(Process::Current::GetImagePath().GetString()).GetString();
			path += L"\\";
			path += setting->GetValue();
		}
		else
		{
			path = setting->GetValue();
		}

		const Filesystem::PathAttributes& attributes = Filesystem::GetAttributes(path.c_str());
		if (!Filesystem::DoesExist(attributes))
		{
			if (path.length() != 0u)
			{
				LC_ERROR_USER("Cannot determine %S at path %S", type, path.c_str());
			}
			return path;
		}

		if (!Filesystem::IsDirectory(attributes))
		{
			// this is not a directory, but a full path
// EPIC REMOVED			LC_SUCCESS_USER("Using %S at path %S", type, path.c_str());
			return path;
		}

		// try to find the compiler/linker in the given directory or any of its child directories
		const types::vector<Filesystem::Path>& files = Filesystem::EnumerateFiles(path.c_str());

		// walk over all files, grabbing only cl.exe and link.exe
		const size_t count = files.size();
		for (size_t i = 0u; i < count; ++i)
		{
			Filesystem::Path originalPath;
			originalPath = path.c_str();
			originalPath += L"\\";
			originalPath += files[i];

			const Filesystem::Path lowerCaseFilename = originalPath.ToLower();
			if (string::Contains(lowerCaseFilename.GetString(), vs2017Path))
			{
				// containing the proper sub-path is not enough, we also need to check the filename,
				// because Visual Studio has files named cl.exe.config and link.exe.config.
				const Filesystem::Path filenameOnly = Filesystem::GetFilename(lowerCaseFilename.GetString());
				if (string::Matches(filenameOnly.GetString(), exeName))
				{
					LC_SUCCESS_USER("Found %S at path %S", type, originalPath.GetString());
					return std::wstring(originalPath.GetString());
				}
			}
			else if (string::Contains(lowerCaseFilename.GetString(), vs2015AndEarlierPath))
			{
				// containing the proper sub-path is not enough, we also need to check the filename,
				// because Visual Studio has files named cl.exe.config and link.exe.config.
				const Filesystem::Path filenameOnly = Filesystem::GetFilename(lowerCaseFilename.GetString());
				if (string::Matches(filenameOnly.GetString(), exeName))
				{
					LC_SUCCESS_USER("Found %S at path %S", type, originalPath.GetString());
					return std::wstring(originalPath.GetString());
				}
			}
		}

		LC_ERROR_USER("Could not find %S while recursing directory %S", exeName, path.c_str());
		return path;
	}

	// helper function that tries to apply a new value to any setting
	template <typename SettingType, typename T>
	static SettingType* ApplySetting(SettingType* settings[], unsigned int settingsCount, const char* const settingName, T value)
	{
		const std::wstring wideSettingName(string::ToWideString(settingName));

		for (unsigned int i = 0u; i < settingsCount; ++i)
		{
			const wchar_t* name = settings[i]->GetName();
			if (string::Matches(name, wideSettingName.c_str()))
			{
				// found the correct setting, apply the new value
				settings[i]->SetValue(value);

				// BEGIN EPIC MOD - Removing UI
				// // tell the UI to update
				// g_theApp.GetMainFrame()->RefreshPropertyValue(settings[i]);
				// END EPIC MOD

				return settings[i];
			}
		}

		return nullptr;
	}
}


void appSettings::Startup(const wchar_t* group)
{
	// ensure that the directories exist
	::SHCreateDirectoryExW(NULL, GetLppDirectory().c_str(), NULL);
	::SHCreateDirectoryExW(NULL, GetSymbolsDirectory().c_str(), NULL);

	g_initialWindowMode = new SettingInt
	(
		group,
		L"initial_window_mode",
		L"Initial window mode",
		L"Specifies how Live++ is launched",
		SW_SHOWNORMAL
	);

	g_initialWindowModeProxy = new SettingIntProxy(g_initialWindowMode);
	(*g_initialWindowModeProxy)
		.AddMapping(L"Normal", SW_SHOWNORMAL)
		.AddMapping(L"Minimized", SW_SHOWMINIMIZED)
		.AddMapping(L"Maximized", SW_SHOWMAXIMIZED);

	g_showFullPathInTitle = new SettingBool
	(
		group,
		L"show_full_path_in_title",
		L"Show full path in title",
		L"Specifies whether the full path will be shown in the window title",
		false
	);

	g_showPathFirstInTitle = new SettingBool
	(
		group,
		L"show_path_first_in_title",
		L"Show path first in title",
		L"Specifies whether the path will be shown first in the window title",
		false
	);

	g_receiveFocusOnRecompile = new SettingInt
	(
		group,
		L"receive_focus_on_recompile",
		L"Receive focus on re-compile",
		L"Specifies when Live++ should receive focus",
		FocusOnRecompile::ON_SHORTCUT
	);

	g_receiveFocusOnRecompileProxy = new SettingIntProxy(g_receiveFocusOnRecompile);
	(*g_receiveFocusOnRecompileProxy)
		.AddMapping(L"On error", FocusOnRecompile::ON_ERROR)
		.AddMapping(L"On success", FocusOnRecompile::ON_SUCCESS)
		.AddMapping(L"On shortcut", FocusOnRecompile::ON_SHORTCUT)
		.AddMapping(L"Never", FocusOnRecompile::NEVER);

	g_showNotificationOnRecompile = new SettingBool
	(
		group,
		L"show_notification_on_recompile",
		L"Show notifications on re-compile",
		L"Specifies whether Live++ shows notifications when compiling",
		true
	);

	g_clearLogOnRecompile = new SettingBool
	(
		group,
		L"clear_log_on_recompile",
		L"Clear log on re-compile",
		L"Specifies whether Live++ clears the log when compiling",
		false
	);

	g_minimizeOnClose = new SettingBool
	(
		group,
		L"minimize_to_tray_on_close",
		L"Minimize to tray on close",
		L"Specifies whether Live++ should be minimized into the system tray when being closed",
		false
	);

	g_keepTrayIcon = new SettingBool
	(
		group,
		L"keep_system_tray_icon",
		L"Keep system tray icon",
		L"Specifies whether the Live++ icon should stay in the system tray",
		false
	);

	g_playSoundOnSuccess = new SettingString
	(
		group,
		L"sound_on_success",
		L"Play sound on success",
		L"Specifies a .WAV to play on successful re-compiles",
		L""
	);

	g_playSoundOnError = new SettingString
	(
		group,
		L"sound_on_error",
		L"Play sound on error",
		L"Specifies a .WAV to play on failed re-compiles",
		L""
	);

	g_compileShortcut = new SettingShortcut
	(
		group,
		L"compile_shortcut",
		L"Compile shortcut",
		L"Shortcut that triggers a re-compile",
		0x37A	// Ctrl+Alt+F11
	);

	g_prewarmTimeout = new SettingInt
	(
		group,
		L"prewarm_timeout",
		L"Prewarm timeout",
		L"Timeout in milliseconds when prewarming compiler/linker environments",
		10000
	);

	g_showUndecoratedNames = new SettingBool
	(
		group,
		L"show_undecorated_symbol_names",
		L"Show undecorated symbol names",
		L"Specifies whether output will show undecorated symbol names",
		false
	);

	g_showTimestamps = new SettingBool
	(
		group,
		L"show_timestamps",
		L"Show timestamps",
		L"Specifies whether output will show timestamps",
		false
	);

	g_wordWrapOutput = new SettingBool
	(
		group,
		L"enable_word_wrap_output",
		L"Enable word wrap for output",
		L"Specifies whether output will be word-wrapped",
		false
	);

	g_enableDevLog = new SettingBool
	(
		group,
		L"enable_dev_output",
		L"Enable Dev output",
		L"Specifies whether development logs will be generated",
		false
	);

	g_enableTelemetryLog = new SettingBool
	(
		group,
		L"enable_telemetry_output",
		L"Enable Telemetry output",
		L"Specifies whether telemetry logs will be generated",
		false
	);

	g_enableDevLogCompilands = new SettingBool
	(
		group,
		L"enable_dev_compiland_output",
		L"Enable Dev compiland output",
		L"Specifies whether dev logs for compiland info will be generated",
		false
	);

	g_compilerPath = new SettingString
	(
		group,
		L"override_compiler_path",
		L"Override compiler path",
		L"Overrides the compiler path found in the PDB",
		L""
	);

	g_useCompilerOverrideAsFallback = new SettingBool
	(
		group,
		L"override_compiler_path_as_fallback",
		L"Override compiler path only as fallback",
		L"Specifies whether Live++ uses the override compiler path only as fallback",
		false
	);

	g_useCompilerEnvironment = new SettingBool
	(
		group,
		L"use_compiler_environment",
		L"Use compiler environment",
		L"Specifies whether Live++ tries to find and use the compiler environment",
		true
	);

	g_compilerOptions = new SettingString
	(
		group,
		L"additional_compiler_options",
		L"Additional compiler options",
		L"Additional compiler options passed to the compiler when creating a patch",
		L""
	);

	g_compilerForcePchPdbs = new SettingBool
	(
		group,
		L"compiler_force_pch_pdbs",
		L"Force use of PCH PDBs",
		L"Forces Live++ to make each translation unit use the same PDB as the corresponding precompiled header when re-compiling",
		false
	);

	g_linkerPath = new SettingString
	(
		group,
		L"override_linker_path",
		L"Override linker path",
		L"Overrides the linker path found in the PDB",
		L""
	);

	g_useLinkerOverrideAsFallback = new SettingBool
	(
		group,
		L"override_linker_path_as_fallback",
		L"Override linker path only as fallback",
		L"Specifies whether Live++ uses the override linker path only as fallback",
		false
	);

	g_useLinkerEnvironment = new SettingBool
	(
		group,
		L"use_linker_environment",
		L"Use linker environment",
		L"Specifies whether Live++ tries to find and use the linker environment",
		true
	);

	g_linkerOptions = new SettingString
	(
		group,
		L"additional_linker_options",
		L"Additional linker options",
		L"Additional linker options passed to the linker when creating a patch",
		L""
	);	

	g_continuousCompilationEnabled = new SettingBool
	(
		group,
		L"continuous_compilation_enabled",
		L"Enable continuous compilation",
		L"Specifies whether continuous compilation is enabled",
		false
	);

	g_continuousCompilationPath = new SettingString
	(
		group,
		L"continuous_compilation_path",
		L"Directory to watch",
		L"Directory to watch for changes when using continuous compilation",
		L""
	);

	g_continuousCompilationTimeout = new SettingInt
	(
		group,
		L"continuous_compilation_timeout",
		L"Timeout (ms)",
		L"Timeout in milliseconds used when waiting for changes",
		100
	);

	g_virtualDriveLetter = new SettingString
	(
		group,
		L"virtual_drive_letter",
		L"Virtual drive letter",
		L"Drive letter of the virtual drive to use, e.g. Z:",
		L""
	);

	g_virtualDrivePath = new SettingString
	(
		group,
		L"virtual_drive_path",
		L"Virtual drive path",
		L"Path to map to the virtual drive, e.g. C:\\MyPath",
		L""
	);

	g_installCompiledPatchesMultiProcess = new SettingBool
	(
		group,
		L"install_compiled_patches_multi_process",
		L"Install compiled patches",
		L"Specifies whether compiled patches are installed into launched processes belonging to an existing process group",
		// BEGIN EPIC MOD - changing default for restart functionality
		true
		// END EPIC MOD
	);

	g_amalgamationSplitIntoSingleParts = new SettingBool
	(
		group,
		L"amalgamation_split_into_single_parts",
		L"Split into single parts",
		L"Specifies whether amalgamated/unity files are automatically split into single files",
		// BEGIN EPIC MOD
		true
		// END EPIC MOD
	);

	g_amalgamationSplitMinCppCount = new SettingInt
	(
		group,
		L"amalgamation_split_min_cpp_count",
		L"Split threshold",
		L"Minimum number of .cpp files that must be included in an amalgamated/unity file before it is split",
		3
	);	

	g_amalgamationCppFileExtensions = new SettingString
	(
		group,
		L"amalgamation_cpp_file_extensions",
		L"C/C++ file extensions",
		L"File extensions treated as C/C++ files when splitting amalgamated/unity files",
		L".cpp;.c;.cc;.c++;.cp;.cxx"
	);

	g_ue4EnableNatVisSupport = new SettingBool
	(
		group,
		L"ue4_enable_natvis_support",
		L"Enable NatVis support",
		L"Specifies whether NatVis visualizers are supported by linking with a special helper library",
		false
	);

	g_fastBuildDatabasePath = new SettingString
	(
		group,
		L"fastbuild_database_path",
		L"FASTBuild database path",
		L"Path to the FASTBuild database to gather dependencies from",
		L""
	);

	g_fastBuildDllName = new SettingString
	(
		group,
		L"fastbuild_dll_name",
		L"FASTBuild DLL name",
		L"Name of the DLL used to load FASTBuild databases",
		L"LPP_FASTBuild_1_01.dll"
	);
}


void appSettings::Shutdown(void)
{
	delete g_initialWindowMode;
	delete g_initialWindowModeProxy;
	delete g_showFullPathInTitle;
	delete g_showPathFirstInTitle;

	delete g_receiveFocusOnRecompile;
	delete g_receiveFocusOnRecompileProxy;
	delete g_showNotificationOnRecompile;
	delete g_clearLogOnRecompile;
	delete g_minimizeOnClose;
	delete g_keepTrayIcon;
	delete g_playSoundOnSuccess;
	delete g_playSoundOnError;
	delete g_compileShortcut;
	delete g_prewarmTimeout;

	delete g_showUndecoratedNames;
	delete g_showTimestamps;
	delete g_wordWrapOutput;
	delete g_enableDevLog;
	delete g_enableTelemetryLog;
	delete g_enableDevLogCompilands;

	delete g_compilerPath;
	delete g_useCompilerOverrideAsFallback;
	delete g_useCompilerEnvironment;
	delete g_compilerOptions;
	delete g_compilerForcePchPdbs;

	delete g_linkerPath;
	delete g_useLinkerOverrideAsFallback;
	delete g_useLinkerEnvironment;
	delete g_linkerOptions;

	delete g_continuousCompilationEnabled;
	delete g_continuousCompilationPath;
	delete g_continuousCompilationTimeout;

	delete g_virtualDriveLetter;
	delete g_virtualDrivePath;

	delete g_installCompiledPatchesMultiProcess;

	delete g_amalgamationSplitIntoSingleParts;
	delete g_amalgamationSplitMinCppCount;
	delete g_amalgamationCppFileExtensions;

	delete g_ue4EnableNatVisSupport;

	delete g_fastBuildDatabasePath;
	delete g_fastBuildDllName;
}


std::wstring appSettings::GetLppDirectory(void)
{
	wchar_t* knownPath = nullptr;
	::SHGetKnownFolderPath(FOLDERID_LocalAppData, 0u, NULL, &knownPath);

	std::wstring directory(knownPath);
	directory += L"\\Live++";

	::CoTaskMemFree(knownPath);

	return directory;
}


std::wstring appSettings::GetSymbolsDirectory(void)
{
	std::wstring directory(GetLppDirectory());
	directory += L"\\Symbols";

	return directory;
}


std::wstring appSettings::GetUserSettingsPath(void)
{
	// user settings are stored in the %localappdata%\Live++ directory
	std::wstring iniPath(GetLppDirectory());

#if LC_64_BIT
	iniPath += L"\\LPP_x64.ini";
#else
	iniPath += L"\\LPP_x86.ini";
#endif

	return Filesystem::NormalizePath(iniPath.c_str()).GetString();
}


std::wstring appSettings::GetProjectSettingsPath(void)
{
	// project settings are stored next to the Live++ executable
	const std::wstring& imagePath = Process::Current::GetImagePath().GetString();
	std::wstring iniPath(Filesystem::GetDirectory(imagePath.c_str()).GetString());

#if LC_64_BIT
	iniPath += L"\\LPP_x64.ini";
#else
	iniPath += L"\\LPP_x86.ini";
#endif

	return Filesystem::NormalizePath(iniPath.c_str()).GetString();
}


std::wstring appSettings::GetCompilerPath(void)
{
	return g_cachedCompilerPath;
}


std::wstring appSettings::GetLinkerPath(void)
{
	return g_cachedLinkerPath;
}


const types::vector<std::wstring>& appSettings::GetAmalgamatedCppFileExtensions(void)
{
	return g_cachedAmalgamatedCppFileExtensions;
}


void appSettings::UpdateCompilerPathCache(void)
{
	g_cachedCompilerPath = DeterminePath(g_compilerPath, L"compiler", VS2017_COMPILER_PATH, VS2015_AND_EARLIER_COMPILER_PATH, COMPILER_EXE);
}


void appSettings::UpdateLinkerPathCache(void)
{
	g_cachedLinkerPath = DeterminePath(g_linkerPath, L"linker", VS2017_LINKER_PATH, VS2015_AND_EARLIER_LINKER_PATH, LINKER_EXE);
}


void appSettings::UpdatePathCache(void)
{
	UpdateCompilerPathCache();
	UpdateLinkerPathCache();
}


void appSettings::UpdateAmalgamatedCppFileExtensions(void)
{
	g_cachedAmalgamatedCppFileExtensions.clear();
	g_cachedAmalgamatedCppFileExtensions.reserve(16u);

	const wchar_t DELIMITER = L';';
	const std::wstring extensions = g_amalgamationCppFileExtensions->GetValue();
	size_t start = extensions.find_first_not_of(DELIMITER);
	size_t end = start;

	while (start != std::string::npos)
	{
		// first find the next occurrence of the delimiter, then push the token into the vector,
		// then skip all occurrences of the delimiter until we find the start of the next token
		end = extensions.find(DELIMITER, start);
		g_cachedAmalgamatedCppFileExtensions.push_back(extensions.substr(start, end - start));
		start = extensions.find_first_not_of(DELIMITER, end);
	}
}


void appSettings::ApplySettingBool(const char* const settingName, bool value)
{
	const unsigned int COUNT = 21u;
	SettingBool* settings[COUNT] =
	{
		g_showFullPathInTitle,
		g_showPathFirstInTitle,
		g_showNotificationOnRecompile,
		g_clearLogOnRecompile,
		g_minimizeOnClose,
		g_keepTrayIcon,
		g_showUndecoratedNames,
		g_showTimestamps,
		g_wordWrapOutput,
		g_enableDevLog,
		g_enableTelemetryLog,
		g_enableDevLogCompilands,
		g_useCompilerOverrideAsFallback,
		g_useCompilerEnvironment,
		g_compilerForcePchPdbs,
		g_useLinkerOverrideAsFallback,
		g_useLinkerEnvironment,
		g_continuousCompilationEnabled,
		g_installCompiledPatchesMultiProcess,
		g_amalgamationSplitIntoSingleParts,
		g_ue4EnableNatVisSupport
	};

	const SettingBool* setting = ApplySetting(settings, COUNT, settingName, value);
	if (!setting)
	{
		LC_ERROR_USER("Cannot apply value for bool setting %s", settingName);
	}
}


void appSettings::ApplySettingInt(const char* const settingName, int value)
{
	// try int settings first
	{
		const unsigned int COUNT = 5u;
		SettingInt* settings[COUNT] =
		{
			g_initialWindowMode,
			g_receiveFocusOnRecompile,
			g_continuousCompilationTimeout,
			g_prewarmTimeout,
			g_amalgamationSplitMinCppCount
		};

		const SettingInt* setting = ApplySetting(settings, COUNT, settingName, value);
		if (setting)
		{
			return;
		}
	}

	// try shortcut setting second
	{
		const std::wstring wideSettingName(string::ToWideString(settingName));
		const wchar_t* name = g_compileShortcut->GetName();
		if (string::Matches(name, wideSettingName.c_str()))
		{
			// found the correct setting, apply the new value
			g_compileShortcut->SetValue(value);

			// BEGIN EPIC MOD - Removing UI
			// // tell the UI to update
			// g_theApp.GetMainFrame()->RefreshPropertyValue(g_compileShortcut);
			// END EPIC MOD
			return;
		}
	}

	LC_ERROR_USER("Cannot apply value for int setting %s", settingName);
}


void appSettings::ApplySettingString(const char* const settingName, const wchar_t* const value)
{
	// try string settings first
	{
		const unsigned int COUNT = 12u;
		SettingString* settings[COUNT] =
		{
			g_playSoundOnSuccess,
			g_playSoundOnError,
			g_compilerPath,
			g_compilerOptions,
			g_linkerPath,
			g_linkerOptions,
			g_continuousCompilationPath,
			g_virtualDriveLetter,
			g_virtualDrivePath,
			g_amalgamationCppFileExtensions,
			g_fastBuildDatabasePath,
			g_fastBuildDllName
		};

		// BEGIN EPIC MOD
		if (strcmp(settingName, "Vfs") == 0)
		{
			const wchar_t* semi = wcschr(value, ';');
			Filesystem::Path virtualPath(value, semi - value);
			Filesystem::AddVfsEntry(virtualPath.GetString(), semi + 1);
			return;
		}
		// END EPIC MOD

		const SettingString* setting = ApplySetting(settings, COUNT, settingName, value);
		if (setting)
		{
			// update cache when compiler path or linker path changes
			if (setting == g_compilerPath)
			{
				UpdateCompilerPathCache();
			}
			else if (setting == g_linkerPath)
			{
				UpdateLinkerPathCache();
			}

			// update cache when amalgamated C++ file extensions change
			else if (setting == g_amalgamationCppFileExtensions)
			{
				UpdateAmalgamatedCppFileExtensions();
			}

			return;
		}
	}

	// try proxies second
	{
		const unsigned int COUNT = 2u;
		SettingIntProxy* settings[COUNT] =
		{
			g_initialWindowModeProxy,
			g_receiveFocusOnRecompileProxy
		};

		for (unsigned int i = 0u; i < COUNT; ++i)
		{
			const int mappedValue = settings[i]->MapStringToInt(value);
			if (mappedValue != -1)
			{
				// found the correct setting
				settings[i]->GetSetting()->SetValue(mappedValue);

				// BEGIN EPIC MOD - Removing UI
				// // tell the UI to update
				// g_theApp.GetMainFrame()->RefreshPropertyValue(settings[i]);
				// END EPIC MOD
				return;
			}
		}
	}

	LC_ERROR_USER("Cannot apply value for string setting %s", settingName);
}


SettingInt* appSettings::g_initialWindowMode = nullptr;
SettingIntProxy* appSettings::g_initialWindowModeProxy = nullptr;
SettingBool* appSettings::g_showFullPathInTitle = nullptr;
SettingBool* appSettings::g_showPathFirstInTitle = nullptr;

SettingInt* appSettings::g_receiveFocusOnRecompile = nullptr;
SettingIntProxy* appSettings::g_receiveFocusOnRecompileProxy = nullptr;
SettingBool* appSettings::g_showNotificationOnRecompile = nullptr;
SettingBool* appSettings::g_clearLogOnRecompile = nullptr;
SettingBool* appSettings::g_minimizeOnClose = nullptr;
SettingBool* appSettings::g_keepTrayIcon = nullptr;

SettingString* appSettings::g_playSoundOnSuccess = nullptr;
SettingString* appSettings::g_playSoundOnError = nullptr;
SettingShortcut* appSettings::g_compileShortcut = nullptr;
SettingInt* appSettings::g_prewarmTimeout = nullptr;

SettingBool* appSettings::g_showUndecoratedNames = nullptr;
SettingBool* appSettings::g_showTimestamps = nullptr;
SettingBool* appSettings::g_wordWrapOutput = nullptr;
SettingBool* appSettings::g_enableDevLog = nullptr;
SettingBool* appSettings::g_enableTelemetryLog = nullptr;
SettingBool* appSettings::g_enableDevLogCompilands = nullptr;

SettingString* appSettings::g_compilerPath = nullptr;
SettingBool* appSettings::g_useCompilerOverrideAsFallback = nullptr;
SettingBool* appSettings::g_useCompilerEnvironment = nullptr;
SettingString* appSettings::g_compilerOptions = nullptr;
SettingBool* appSettings::g_compilerForcePchPdbs = nullptr;

SettingString* appSettings::g_linkerPath = nullptr;
SettingBool* appSettings::g_useLinkerOverrideAsFallback = nullptr;
SettingBool* appSettings::g_useLinkerEnvironment = nullptr;
SettingString* appSettings::g_linkerOptions = nullptr;

SettingBool* appSettings::g_continuousCompilationEnabled = nullptr;
SettingString* appSettings::g_continuousCompilationPath = nullptr;
SettingInt* appSettings::g_continuousCompilationTimeout = nullptr;

SettingString* appSettings::g_virtualDriveLetter = nullptr;
SettingString* appSettings::g_virtualDrivePath = nullptr;

SettingBool* appSettings::g_installCompiledPatchesMultiProcess = nullptr;

SettingBool* appSettings::g_amalgamationSplitIntoSingleParts = nullptr;
SettingInt* appSettings::g_amalgamationSplitMinCppCount = nullptr;
SettingString* appSettings::g_amalgamationCppFileExtensions = nullptr;

SettingBool* appSettings::g_ue4EnableNatVisSupport = nullptr;

SettingString* appSettings::g_fastBuildDatabasePath = nullptr;
SettingString* appSettings::g_fastBuildDllName = nullptr;


#endif