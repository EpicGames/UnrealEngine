// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Settings.h"


namespace appSettings
{
	struct FocusOnRecompile
	{
		enum Enum
		{
			ON_ERROR,
			ON_SUCCESS,
			ON_SHORTCUT,
			NEVER
		};
	};

	void Startup(const wchar_t* group);
	void Shutdown(void);

	// returns a directory where Live++-related files are saved, e.g. %LocalAppData%\Live++
	std::wstring GetLppDirectory(void);

	// returns a path where symbols can be stored
	std::wstring GetSymbolsDirectory(void);

	// returns a path where user settings can be stored
	std::wstring GetUserSettingsPath(void);

	// returns a path where project settings can be stored
	std::wstring GetProjectSettingsPath(void);

	// returns the overridden compiler path
	std::wstring GetCompilerPath(void);

	// returns the overridden linker path
	std::wstring GetLinkerPath(void);

	// returns an array of file extensions to treat as C++ files when splitting amalgamated files
	const types::vector<std::wstring>& GetAmalgamatedCppFileExtensions(void);

	void UpdateCompilerPathCache(void);
	void UpdateLinkerPathCache(void);
	void UpdatePathCache(void);

	void UpdateAmalgamatedCppFileExtensions(void);

	// apply a new value to any of the boolean settings
	void ApplySettingBool(const char* const settingName, bool value);

	// apply a new value to any of the int settings
	void ApplySettingInt(const char* const settingName, int value);

	// apply a new value to any of the string settings
	void ApplySettingString(const char* const settingName, const wchar_t* const value);


	// appearance
	extern SettingInt* g_initialWindowMode;
	extern SettingIntProxy* g_initialWindowModeProxy;
	extern SettingBool* g_showFullPathInTitle;
	extern SettingBool* g_showPathFirstInTitle;

	// behaviour
	extern SettingInt* g_receiveFocusOnRecompile;
	extern SettingIntProxy* g_receiveFocusOnRecompileProxy;
	extern SettingBool* g_showNotificationOnRecompile;
	extern SettingBool* g_clearLogOnRecompile;
	extern SettingBool* g_minimizeOnClose;
	extern SettingBool* g_keepTrayIcon;
	extern SettingString* g_playSoundOnSuccess;
	extern SettingString* g_playSoundOnError;
	extern SettingShortcut* g_compileShortcut;
	extern SettingInt* g_prewarmTimeout;

	// logging
	extern SettingBool* g_showUndecoratedNames;
	extern SettingBool* g_showTimestamps;
	extern SettingBool* g_wordWrapOutput;
	extern SettingBool* g_enableDevLog;
	extern SettingBool* g_enableTelemetryLog;
	extern SettingBool* g_enableDevLogCompilands;

	// compiler
	extern SettingString* g_compilerPath;						// DO NOT USE directly, use GetCompilerPath instead!
	extern SettingBool* g_useCompilerOverrideAsFallback;
	extern SettingBool* g_useCompilerEnvironment;
	extern SettingString* g_compilerOptions;
	extern SettingBool* g_compilerForcePchPdbs;

	// linker
	extern SettingString* g_linkerPath;							// DO NOT USE directly, use GetLinkerPath instead!
	extern SettingBool* g_useLinkerOverrideAsFallback;
	extern SettingBool* g_useLinkerEnvironment;
	extern SettingString* g_linkerOptions;

	// continuous compilation
	extern SettingBool* g_continuousCompilationEnabled;
	extern SettingString* g_continuousCompilationPath;
	extern SettingInt* g_continuousCompilationTimeout;

	// virtual drive
	extern SettingString* g_virtualDriveLetter;
	extern SettingString* g_virtualDrivePath;

	// multi-process editing
	extern SettingBool* g_installCompiledPatchesMultiProcess;

	// amalgamated/unity builds
	extern SettingBool* g_amalgamationSplitIntoSingleParts;
	extern SettingInt* g_amalgamationSplitMinCppCount;
	extern SettingString* g_amalgamationCppFileExtensions;		// DO NOT USE directly, use GetAmalgamatedCppFileExtensions instead!

	// UE4-specific
	extern SettingBool* g_ue4EnableNatVisSupport;

	// FASTBuild-specific
	extern SettingString* g_fastBuildDatabasePath;
	extern SettingString* g_fastBuildDllName;
}


#endif