// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Assembler.h"
// BEGIN EPIC MOD
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
#include "LC_Assert.h"
#include <cstdlib>
// END EPIC MOD


namespace Assembler
{
	namespace Opcode
	{
		// NOP - No Operation
		// multi-byte sequences of NOP instructions, recommended by Intel (see Intel Instruction Set Reference)
		const uint8_t NOP1[1u] = { 0x90 };												// nop
		const uint8_t NOP2[2u] = { 0x66, 0x90 };										// xchg ax,ax 
		const uint8_t NOP3[3u] = { 0x0F, 0x1F, 0x00 };									// nop DWORD PTR [rax]
		const uint8_t NOP4[4u] = { 0x0F, 0x1F, 0x40, 0x00 };							// nop DWORD PTR [rax+0x00]
		const uint8_t NOP5[5u] = { 0x0F, 0x1F, 0x44, 0x00, 0x00 };						// nop DWORD PTR [rax+rax*1+0x00]
		const uint8_t NOP6[6u] = { 0x66, 0x0F, 0x1F, 0x44, 0x00, 0x00 };				// nop WORD PTR [rax+rax*1+0x00]
		const uint8_t NOP7[7u] = { 0x0F, 0x1F, 0x80, 0x00, 0x00, 0x00, 0x00 };			// nop DWORD PTR [rax+0x0]
		const uint8_t NOP8[8u] = { 0x0F, 0x1F, 0x84, 0x00, 0x00, 0x00, 0x00, 0x00 };	// nop DWORD PTR [rax+rax*1+0x0]

		// longer NOPs are built from other multi-byte NOPs
		const uint8_t NOP9[9u] =	{ NOP8[0], NOP8[1], NOP8[2], NOP8[3], NOP8[4], NOP8[5], NOP8[6], NOP8[7], NOP1[0] };
		const uint8_t NOP10[10u] =	{ NOP8[0], NOP8[1], NOP8[2], NOP8[3], NOP8[4], NOP8[5], NOP8[6], NOP8[7], NOP2[0], NOP2[1] };
		const uint8_t NOP11[11u] =	{ NOP8[0], NOP8[1], NOP8[2], NOP8[3], NOP8[4], NOP8[5], NOP8[6], NOP8[7], NOP3[0], NOP3[1], NOP3[2] };
		const uint8_t NOP12[12u] =	{ NOP8[0], NOP8[1], NOP8[2], NOP8[3], NOP8[4], NOP8[5], NOP8[6], NOP8[7], NOP4[0], NOP4[1], NOP4[2], NOP4[3] };
		const uint8_t NOP13[13u] =	{ NOP8[0], NOP8[1], NOP8[2], NOP8[3], NOP8[4], NOP8[5], NOP8[6], NOP8[7], NOP5[0], NOP5[1], NOP5[2], NOP5[3], NOP5[4] };
		const uint8_t NOP14[14u] =	{ NOP8[0], NOP8[1], NOP8[2], NOP8[3], NOP8[4], NOP8[5], NOP8[6], NOP8[7], NOP6[0], NOP6[1], NOP6[2], NOP6[3], NOP6[4], NOP6[5] };
		const uint8_t NOP15[15u] =	{ NOP8[0], NOP8[1], NOP8[2], NOP8[3], NOP8[4], NOP8[5], NOP8[6], NOP8[7], NOP7[0], NOP7[1], NOP7[2], NOP7[3], NOP7[4], NOP7[5], NOP7[6] };

		// RET - Return
		const uint8_t RET[1u] = { 0xC3 };						// ret

		// JMP - Jump
		const uint8_t JMP_REL8[1u] = { 0xEB };					// JMP rel8; Jump short, RIP = RIP + 8-bit displacement sign extended to 64-bits
		const uint8_t JMP_REL32[1u] = { 0xE9 };					// JMP rel32; Jump near, relative, RIP = RIP + 32-bit displacement sign extended to 64-bits
		const uint8_t JMP_INDIRECT_RIP32[2u] = { 0xFF, 0x25 };	// JMP r/m64; Jump near, absolute indirect, RIP = 64-Bit offset from register or memory

		// CALL - Call Procedure
		const uint8_t CALL_INDIRECT_RIP32[2u] = { 0xFF, 0x15 };	// CALL r/m64; Call near, absolute indirect, address given in r/m64.

		// MOV - Move
		const uint8_t MOV_RAX_TCB[9u] = { 0x64, 0x48, 0x8B, 0x04, 0x25, 0x0, 0x0, 0x0, 0x0 };	// mov RAX, QWORD PTR FS:[0000000000000000h]
		
		// ADD - Addition
		const uint8_t ADD_RAX_INDIRECT_RIP32[3u] = { 0x48, 0x03, 0x05 };						// add RAX, QWORD PTR [RIP + OFFSET]
	}
}


namespace
{
	// small lookup table from size to corresponding NOP opcode
	static const uint8_t* const SIZE_TO_NOP_OPCODE[16u] =
	{
		nullptr,
		Assembler::Opcode::NOP1,
		Assembler::Opcode::NOP2,
		Assembler::Opcode::NOP3,
		Assembler::Opcode::NOP4,
		Assembler::Opcode::NOP5,
		Assembler::Opcode::NOP6,
		Assembler::Opcode::NOP7,
		Assembler::Opcode::NOP8,
		Assembler::Opcode::NOP9,
		Assembler::Opcode::NOP10,
		Assembler::Opcode::NOP11,
		Assembler::Opcode::NOP12,
		Assembler::Opcode::NOP13,
		Assembler::Opcode::NOP14,
		Assembler::Opcode::NOP15
	};

	// generic helper function to generate different instructions using differently-sized offsets
	template <typename Opcode, typename Offset, size_t INSTRUCTION_SIZE = sizeof(Opcode) + sizeof(Offset)>
	static Assembler::SizedInstruction<INSTRUCTION_SIZE> MakeSizedInstructionWithOffset(const Opcode& opcode, Offset offset)
	{
		Assembler::SizedInstruction<INSTRUCTION_SIZE> instruction;
		memcpy(instruction.machineCode, opcode, sizeof(Opcode));

		// the displacement/offset is relative to the next instruction
		const Offset offsetRelativeToNextInstruction = static_cast<Offset>(offset - INSTRUCTION_SIZE);
		memcpy(instruction.machineCode + sizeof(Opcode), &offsetRelativeToNextInstruction, sizeof(Offset));

		return instruction;
	}
}


Assembler::Instruction Assembler::MakeNOP(uint8_t size)
{
	LC_ASSERT(size != 0u, "Invalid NOP length.");
	LC_ASSERT(size <= Instruction::MAXIMUM_LENGTH, "Exceeded maximum length of instruction.");

	Instruction instruction;
	instruction.size = size;

	memcpy(instruction.machineCode, SIZE_TO_NOP_OPCODE[size], size);

	return instruction;
}


Assembler::SizedInstruction<Assembler::InstructionSize::RET> Assembler::MakeRET(void)
{
	Assembler::SizedInstruction<Assembler::InstructionSize::RET> instruction;
	memcpy(instruction.machineCode, Opcode::RET, sizeof(Opcode::RET));

	return instruction;
}


Assembler::SizedInstruction<Assembler::InstructionSize::JMP_REL8> Assembler::MakeJMP8Relative(int8_t displacement)
{
	return MakeSizedInstructionWithOffset(Opcode::JMP_REL8, displacement);
}


Assembler::SizedInstruction<Assembler::InstructionSize::JMP_REL32> Assembler::MakeJMP32Relative(int32_t displacement)
{
	return MakeSizedInstructionWithOffset(Opcode::JMP_REL32, displacement);
}


Assembler::SizedInstruction<Assembler::InstructionSize::JMP_INDIRECT_RIP32> Assembler::MakeJMP32IndirectRIPRelative(int32_t ripOffset)
{
	return MakeSizedInstructionWithOffset(Opcode::JMP_INDIRECT_RIP32, ripOffset);
}


Assembler::SizedInstruction<Assembler::InstructionSize::CALL_INDIRECT_RIP32> Assembler::MakeCALL32IndirectRIPRelative(int32_t ripOffset)
{
	return MakeSizedInstructionWithOffset(Opcode::CALL_INDIRECT_RIP32, ripOffset);
}


Assembler::SizedInstruction<Assembler::InstructionSize::MOV_RAX_TCB> Assembler::MakeMOVLoadTCB(void)
{
	Assembler::SizedInstruction<Assembler::InstructionSize::MOV_RAX_TCB> instruction;
	memcpy(instruction.machineCode, Opcode::MOV_RAX_TCB, sizeof(Opcode::MOV_RAX_TCB));

	return instruction;
}


Assembler::SizedInstruction<Assembler::InstructionSize::ADD_RAX_INDIRECT_RIP32> Assembler::MakeADD32IndirectRIPRelative(int32_t ripOffset)
{
	return MakeSizedInstructionWithOffset(Opcode::ADD_RAX_INDIRECT_RIP32, ripOffset);
}


#endif