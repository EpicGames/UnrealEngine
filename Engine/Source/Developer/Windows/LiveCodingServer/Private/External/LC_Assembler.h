// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include <cinttypes>
// END EPIC MOD

namespace Assembler
{
	namespace Opcode
	{
		// NOP - No Operation
		extern const uint8_t NOP1[1u];
		extern const uint8_t NOP2[2u];
		extern const uint8_t NOP3[3u];
		extern const uint8_t NOP4[4u];
		extern const uint8_t NOP5[5u];
		extern const uint8_t NOP6[6u];
		extern const uint8_t NOP7[7u];
		extern const uint8_t NOP8[8u];
		extern const uint8_t NOP9[9u];
		extern const uint8_t NOP10[10u];
		extern const uint8_t NOP11[11u];
		extern const uint8_t NOP12[12u];
		extern const uint8_t NOP13[13u];
		extern const uint8_t NOP14[14u];
		extern const uint8_t NOP15[15u];

		// RET - Return
		extern const uint8_t RET[1u];

		// JMP - Jump
		extern const uint8_t JMP_REL8[1u];
		extern const uint8_t JMP_REL32[1u];
		extern const uint8_t JMP_INDIRECT_RIP32[2u];

		// CALL - Call Procedure
		extern const uint8_t CALL_INDIRECT_RIP32[2u];

		// MOV - Move
		extern const uint8_t MOV_RAX_TCB[9u];

		// ADD - Addition
		extern const uint8_t ADD_RAX_INDIRECT_RIP32[3u];
	}

	namespace InstructionSize
	{
		enum Enum : uint8_t
		{
			NOP1 = sizeof(Opcode::NOP1),
			NOP2 = sizeof(Opcode::NOP2),
			NOP3 = sizeof(Opcode::NOP3),
			NOP4 = sizeof(Opcode::NOP4),
			NOP5 = sizeof(Opcode::NOP5),
			NOP6 = sizeof(Opcode::NOP6),
			NOP7 = sizeof(Opcode::NOP7),
			NOP8 = sizeof(Opcode::NOP8),
			NOP9 = sizeof(Opcode::NOP9),
			NOP10 = sizeof(Opcode::NOP10),
			NOP11 = sizeof(Opcode::NOP11),
			NOP12 = sizeof(Opcode::NOP12),
			NOP13 = sizeof(Opcode::NOP13),
			NOP14 = sizeof(Opcode::NOP14),
			NOP15 = sizeof(Opcode::NOP15),
			RET = sizeof(Opcode::RET),
			JMP_REL8 = sizeof(Opcode::JMP_REL8) + 1u,								// 1-byte offset
			JMP_REL32 = sizeof(Opcode::JMP_REL32) + 4u,								// 4-byte offset
			JMP_INDIRECT_RIP32 = sizeof(Opcode::JMP_INDIRECT_RIP32) + 4u,			// 4-byte offset
			CALL_INDIRECT_RIP32 = sizeof(Opcode::CALL_INDIRECT_RIP32) + 4u,			// 4-byte offset
			MOV_RAX_TCB = sizeof(Opcode::MOV_RAX_TCB),
			ADD_RAX_INDIRECT_RIP32 = sizeof(Opcode::ADD_RAX_INDIRECT_RIP32) + 4u	// 4-byte offset
		};
	}

	struct Instruction
	{
		// From the Intel® 64 and IA-32 Architectures Software Developer's Manual: The maximum length of an Intel 64 and IA-32 instruction remains 15 bytes.
		static const unsigned int MAXIMUM_LENGTH = 15u;

		uint8_t size;
		uint8_t machineCode[MAXIMUM_LENGTH];
	};

	template <uint8_t N>
	struct SizedInstruction
	{
		uint8_t machineCode[N];
	};

	// Generates a NOP instruction of a certain size, making use of multi-byte NOPs.
	Instruction MakeNOP(uint8_t size);

	// Generates a RET instruction.
	SizedInstruction<InstructionSize::RET> MakeRET(void);

	// Generates a short jump using an 8-bit displacement.
	SizedInstruction<InstructionSize::JMP_REL8> MakeJMP8Relative(int8_t displacement);

	// Generates a near jump using a 32-bit displacement.
	SizedInstruction<InstructionSize::JMP_REL32> MakeJMP32Relative(int32_t displacement);

	// Generates an indirect RIP-relative near jump using a 32-bit offset.
	SizedInstruction<InstructionSize::JMP_INDIRECT_RIP32> MakeJMP32IndirectRIPRelative(int32_t ripOffset);

	// Generates an indirect RIP-relative near call using a 32-bit offset.
	SizedInstruction<InstructionSize::CALL_INDIRECT_RIP32> MakeCALL32IndirectRIPRelative(int32_t ripOffset);

	// Generates a MOV that loads the address of the thread control block into RAX.
	SizedInstruction<InstructionSize::MOV_RAX_TCB> MakeMOVLoadTCB(void);

	// Generates an indirect RIP-relative ADD using a 32-bit offset.
	SizedInstruction<InstructionSize::ADD_RAX_INDIRECT_RIP32> MakeADD32IndirectRIPRelative(int32_t ripOffset);
}


#endif