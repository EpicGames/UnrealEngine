// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_COMThread.h"
#include "LC_Thread.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
#include "Windows/MinimalWindowsAPI.h"
#include <objidl.h>

#include "Windows/AllowWindowsPlatformAtomics.h"
// END EPIC MOD

namespace
{
	// helper class to deal with RPC_E_CALL_REJECTED errors when trying to use automation
	// https://web.archive.org/web/20131209031731/http://msdn.microsoft.com/en-us/library/ms228772(VS.80).aspx
	// https://infosys.beckhoff.com/english.php?content=../content/1033/tc3_automationinterface/54043195771173899.html&id=
	class COMMessageFilter : public IMessageFilter
	{
	public:
		virtual ~COMMessageFilter(void) {}

		virtual HRESULT STDMETHODCALLTYPE QueryInterface(
			/* [in] */ REFIID riid,
			/* [iid_is][out] */ _COM_Outptr_ void __RPC_FAR *__RPC_FAR *ppvObject)
		{
			// always set out parameter to NULL, validating it first
			if (!ppvObject)
			{
				return E_INVALIDARG;
			}

			*ppvObject = NULL;
			if (riid == IID_IUnknown || riid == IID_IMessageFilter)
			{
				// increment the reference count and return the pointer
				*ppvObject = this;
				AddRef();

				return NOERROR;
			}

			return E_NOINTERFACE;
		}

		virtual ULONG STDMETHODCALLTYPE AddRef(void)
		{
			::InterlockedIncrement(&m_refCount);
			return m_refCount;
		}

		virtual ULONG STDMETHODCALLTYPE Release(void)
		{
			// decrement the object's internal counter and delete the interface if zero
			ULONG refCount = ::InterlockedDecrement(&m_refCount);
			if (0 == m_refCount)
			{
				delete this;
			}

			return refCount;
		}

		// Handle incoming thread requests
		virtual DWORD STDMETHODCALLTYPE HandleInComingCall(
			/* [annotation][in] */
			_In_  DWORD dwCallType,
			/* [annotation][in] */
			_In_  HTASK htaskCaller,
			/* [annotation][in] */
			_In_  DWORD dwTickCount,
			/* [annotation][in] */
			_In_opt_  LPINTERFACEINFO lpInterfaceInfo)
		{
			LC_UNUSED(dwCallType);
			LC_UNUSED(htaskCaller);
			LC_UNUSED(dwTickCount);
			LC_UNUSED(lpInterfaceInfo);

			return SERVERCALL_ISHANDLED;
		}

		// Thread call was rejected, so try again
		virtual DWORD STDMETHODCALLTYPE RetryRejectedCall(
			/* [annotation][in] */
			_In_  HTASK htaskCallee,
			/* [annotation][in] */
			_In_  DWORD dwTickCount,
			/* [annotation][in] */
			_In_  DWORD dwRejectType)
		{
			LC_UNUSED(htaskCallee);
			LC_UNUSED(dwTickCount);

			if (dwRejectType == SERVERCALL_RETRYLATER)
			{
				// retry the thread call immediately if return in rage [0, 99]
				return 99;
			}

			// too busy, cancel call
			return static_cast<DWORD>(-1);
		}

		virtual DWORD STDMETHODCALLTYPE MessagePending(
			/* [annotation][in] */
			_In_  HTASK htaskCallee,
			/* [annotation][in] */
			_In_  DWORD dwTickCount,
			/* [annotation][in] */
			_In_  DWORD dwPendingType)
		{
			LC_UNUSED(htaskCallee);
			LC_UNUSED(dwTickCount);
			LC_UNUSED(dwPendingType);

			return PENDINGMSG_WAITDEFPROCESS;
		}

	private:
		volatile ULONG m_refCount = 0ul;
	};
}


COMThread::COMThread(void)
	: m_function()
	, m_functionAvailableEvent(nullptr, Event::Type::AUTO_RESET)
	, m_functionFinishedExecutingEvent(nullptr, Event::Type::AUTO_RESET)
	, m_leaveThreadEvent(nullptr, Event::Type::AUTO_RESET)
	, m_internalThread(nullptr)
{
	// launch the thread that takes care of calling COM functions
	m_internalThread = Thread::CreateFromMemberFunction("Live++ COM", 65536u, this, &COMThread::ThreadFunction);
}


COMThread::~COMThread(void)
{
	// make the internal thread break out of its loop and wait until it terminates
	m_leaveThreadEvent.Signal();
	m_functionAvailableEvent.Signal();

	Thread::Join(m_internalThread);
	Thread::Close(m_internalThread);
}


Thread::ReturnValue COMThread::ThreadFunction(void)
{
	const HRESULT result = ::CoInitialize(NULL);
	if (result != S_OK)
	{
		LC_ERROR_DEV("Could not initialize COM. Error: 0x%X", result);
	}

	for (;;)
	{
		// wait until function becomes available
		m_functionAvailableEvent.Wait();

		// gracefully exit the thread
		if (m_leaveThreadEvent.TryWait())
		{
			m_functionFinishedExecutingEvent.Signal();
			break;
		}

		// install a COM message filter and execute the function.
		// the message filter deals with RPC_E_CALL_REJECTED errors automatically without having to wrap
		// every call into error checks and retries.
		COMMessageFilter* messageFilter = new COMMessageFilter;
		IMessageFilter* oldFilter = nullptr;
		::CoRegisterMessageFilter(messageFilter, &oldFilter);
		{
			(*m_function.function)(m_function.context, m_function.returnValueAddr);
		}
		::CoRegisterMessageFilter(oldFilter, nullptr);

		// tell waiting thread that function finished executing
		m_functionFinishedExecutingEvent.Signal();
	}

	::CoUninitialize();

	return Thread::ReturnValue(0u);
}

// BEGIN EPIC MOD
#include "Windows/HideWindowsPlatformAtomics.h"
// END EPIC MOD

#endif