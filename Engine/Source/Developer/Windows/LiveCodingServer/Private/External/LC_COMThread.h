// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include <type_traits>
// END EPIC MOD
#include "LC_Event.h"
#include "LC_ThreadTypes.h"


// helper class that is the only one calling COM-related functions and methods.
// this is needed because most of the COM VS automation interfaces only can be called from a single thread.
class COMThread
{
	struct COMFunction
	{
		typedef void (FunctionType)(void*, void*);

		FunctionType* function;
		void* context;
		void* returnValueAddr;
	};

public:
	COMThread(void);
	~COMThread(void);

	// helper function that runs any given function in the internal COM thread and waits until execution has finished
	// BEGIN EPIC MOD
	template <typename F, typename... Args>
	inline std::invoke_result_t<F, Args...> CallInThread(F ptrToFunction, Args&&... args)
	// END EPIC MOD
	{
		// BEGIN EPIC MOD
		typedef std::invoke_result_t<F, Args...> ReturnValue;
		// END EPIC MOD

		// a helper lambda that calls the given functions with the provided arguments.
		// because this lambda needs to capture, it cannot be converted to a function pointer implicitly,
		// and therefore cannot be used as a COM function directly.
		// however, we can solve this by another indirection.
		auto captureLambda = [ptrToFunction, args...]()
		{
			return (*ptrToFunction)(args...);
		};

		// helper to make the following easier to read
		typedef decltype(captureLambda) CaptureLambdaType;

		// here's the trick: we generate another capture-less lambda that has the same signature as our COM function,
		// and internally cast the given object to its original type, calling the lambda with captures from within
		// this capture-less lambda.
		// it is OK to leave both the lambda as well as the return value on the stack. this function returns only
		// after the COM function has been called by the internal thread.
		ReturnValue result = {};
		m_function.function = [](void* captureLambda, void* returnValueAddr)
		{
			CaptureLambdaType* realCaptureLambda = static_cast<CaptureLambdaType*>(captureLambda);
			ReturnValue* realReturnValueAddr = static_cast<ReturnValue*>(returnValueAddr);
			*realReturnValueAddr = (*realCaptureLambda)();
		};

		m_function.context = &captureLambda;
		m_function.returnValueAddr = &result;

		// signal event that function in thread should now be called
		m_functionAvailableEvent.Signal();

		// wait for signal that the function has finished executing
		m_functionFinishedExecutingEvent.Wait();

		return result;
	}

private:
	Thread::ReturnValue ThreadFunction(void);

	COMFunction m_function;
	Event m_functionAvailableEvent;
	Event m_functionFinishedExecutingEvent;
	Event m_leaveThreadEvent;
	Thread::Handle m_internalThread;
};


#endif