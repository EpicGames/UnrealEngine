// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_ChangeNotification.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include "LC_Vfs.h"
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

ChangeNotification::ChangeNotification(void)
	: m_handle(INVALID_HANDLE_VALUE)
{
}


ChangeNotification::~ChangeNotification(void)
{
	Destroy();
}


void ChangeNotification::Create(const wchar_t* path)
{
	// BEGIN EPIC MOD
	Filesystem::Path temp;
	path = Filesystem::Devirtualize(path, temp);
	m_handle = ::FindFirstChangeNotificationW(path, Windows::TRUE, FILE_NOTIFY_CHANGE_LAST_WRITE);
	// END EPIC MOD
	if (m_handle == INVALID_HANDLE_VALUE)
	{
		const DWORD error = ::GetLastError();
		LC_ERROR_USER("Cannot create change notification for path %S. Error: 0x%X", path, error);
	}
}


void ChangeNotification::Destroy(void)
{
	if (m_handle != INVALID_HANDLE_VALUE)
	{
		::FindCloseChangeNotification(m_handle);
		m_handle = INVALID_HANDLE_VALUE;
	}
}


bool ChangeNotification::Check(unsigned int timeoutMs)
{
	if (m_handle == INVALID_HANDLE_VALUE)
	{
		// change notification is not active
		return false;
	}

	// check whether there was any change at all.
	// if there was, repeatedly wait for other notifications or until the timeout has been reached.
	bool hasChange = WaitForNotification(0u);
	const bool hadAtLeastOneChange = hasChange;

	while (hasChange)
	{
		::FindNextChangeNotification(m_handle);
		hasChange = WaitForNotification(timeoutMs);
	}

	return hadAtLeastOneChange;
}


bool ChangeNotification::CheckOnce(void)
{
	if (m_handle == INVALID_HANDLE_VALUE)
	{
		// change notification is not active
		return false;
	}

	// check whether there was any change at all.
	const bool hasChange = WaitForNotification(0u);
	return hasChange;
}


void ChangeNotification::CheckNext(unsigned int timeoutMs)
{
	bool hasChange = true;
	while (hasChange)
	{
		::FindNextChangeNotification(m_handle);
		hasChange = WaitForNotification(timeoutMs);
	}
}


bool ChangeNotification::WaitForNotification(unsigned int timeoutMs)
{
	const DWORD waitStatus = ::WaitForSingleObject(m_handle, timeoutMs);
	switch (waitStatus)
	{
		case WAIT_OBJECT_0:
			return true;

		default:
			return false;
	}
}


#endif