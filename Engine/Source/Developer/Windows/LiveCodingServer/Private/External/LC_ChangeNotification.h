// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "Windows/MinimalWindowsAPI.h"
// END EPIC MOD

class ChangeNotification
{
public:
	ChangeNotification(void);
	~ChangeNotification(void);

	void Create(const wchar_t* path);
	void Destroy(void);

	bool Check(unsigned int timeoutMs);
	bool CheckOnce(void);
	void CheckNext(unsigned int timeoutMs);

private:
	bool WaitForNotification(unsigned int timeoutMs);

	// BEGIN EPIC MOD
	Windows::HANDLE m_handle;
	// END EPIC MOD
};


#endif