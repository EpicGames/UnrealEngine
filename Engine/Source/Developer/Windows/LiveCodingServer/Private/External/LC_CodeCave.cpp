// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_CodeCave.h"
#include "LC_Process.h"
#include "LC_Thread.h"


CodeCave::CodeCave(Process::Handle processHandle, Process::Id processId, Thread::Id commandThreadId, const void* jumpToSelf)
	: m_processHandle(processHandle)
	, m_processId(processId)
	, m_commandThreadId(commandThreadId)
	, m_jumpToSelf(jumpToSelf)
	, m_perThreadData()
{
	m_perThreadData.reserve(128u);
}


void CodeCave::Install(void)
{
	Process::Suspend(m_processHandle);

	// enumerate all threads of the process now that it's suspended
	const std::vector<Thread::Id>& threadIds = Process::EnumerateThreads(m_processId);
	
	const size_t threadCount = threadIds.size();
	m_perThreadData.resize(threadCount);

	// set all threads' instruction pointers into the code cave.
	// additionally, we set the threads' priority to IDLE so that they don't burn CPU cycles,
	// which could totally starve all CPUs and the OS, depending on how many threads
	// are currently running.
	for (size_t i = 0u; i < threadCount; ++i)
	{
		const Thread::Id id = threadIds[i];
		m_perThreadData[i].id = id;

		if (id == m_commandThreadId)
		{
			// this is the Live++ command thread, don't put it into the cave
			continue;
		}

		Thread::Handle threadHandle = Thread::Open(id);
		Thread::Context context = Thread::GetContext(threadHandle);
		m_perThreadData[i].priority = Thread::GetPriority(threadHandle);
		m_perThreadData[i].originalIp = Thread::ReadInstructionPointer(&context);
		Thread::SetPriority(threadHandle, THREAD_PRIORITY_IDLE);
		Thread::WriteInstructionPointer(&context, m_jumpToSelf);
		Thread::SetContext(threadHandle, &context);
		Thread::Close(threadHandle);
	}

	// let the process resume. all threads except the Live++ thread will be held in the code cave
	Process::Resume(m_processHandle);
}


void CodeCave::Uninstall(void)
{
	Process::Suspend(m_processHandle);

	// restore original thread instruction pointers
	const size_t threadCount = m_perThreadData.size();
	for (size_t i = 0u; i < threadCount; ++i)
	{
		const Thread::Id id = m_perThreadData[i].id;
		if (id == m_commandThreadId)
		{
			// this is the Live++ command thread
			continue;
		}

		Thread::Handle threadHandle = Thread::Open(id);
		Thread::Context context = Thread::GetContext(threadHandle);
		const void* currentIp = Thread::ReadInstructionPointer(&context);

		// only set the original instruction pointer if the thread is really being held in the cave.
		// in certain situations (e.g. after an exception), the debugger/OS already restored the context
		// of all threads, and it would be fatal to interfere with this.
		if (currentIp == m_jumpToSelf)
		{
			Thread::SetPriority(threadHandle, m_perThreadData[i].priority);
			Thread::WriteInstructionPointer(&context, m_perThreadData[i].originalIp);
			Thread::SetContext(threadHandle, &context);
		}
		Thread::Close(threadHandle);
	}

	Process::Resume(m_processHandle);
}


#endif