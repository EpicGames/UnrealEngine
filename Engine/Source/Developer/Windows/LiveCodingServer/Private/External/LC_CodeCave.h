// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ProcessTypes.h"
#include "LC_ThreadTypes.h"
// BEGIN EPIC MOD
#include "LC_Types.h"
// END EPIC MOD


// nifty helper to let all threads of a process except one make "progress" by being held inside a jump-to-self cave.
class CodeCave
{
public:
	// the jump-to-self code needs to be available in the address space of the given process
	CodeCave(Process::Handle processHandle, Process::Id processId, Thread::Id commandThreadId, const void* jumpToSelf);

	void Install(void);
	void Uninstall(void);

private:
	Process::Handle m_processHandle;
	Process::Id m_processId;
	Thread::Id m_commandThreadId;
	const void* m_jumpToSelf;

	struct PerThreadData
	{
		Thread::Id id;
		const void* originalIp;
		int priority;
	};

	types::vector<PerThreadData> m_perThreadData;
};


#endif