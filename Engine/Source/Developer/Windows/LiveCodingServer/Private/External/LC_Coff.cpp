// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Coff.h"
#include "LC_CoffDetail.h"
#include "LC_StringUtil.h"
#include "LC_PointerUtil.h"
#include "LC_Filesystem.h"
#include "LC_SymbolPatterns.h"
#include "LC_Symbols.h"
#include "LC_MemoryStream.h"
#include "LC_UniqueId.h"
#include "LC_MemoryMappedFile.h"
#include "LC_Assembler.h"
// BEGIN EPIC MOD
#include "LC_Allocators.h"
#include "LC_Assert.h"
#include "LC_Vfs.h"
#include <algorithm>
// END EPIC MOD

namespace
{
	static const uint32_t INVALID_CRT_SECTION = 0xFFFFFFFFu;

	// this is not used as a character in mangled names
	static const char COFF_SUFFIX = '%';
	static const wchar_t COFF_SUFFIX_WIDE = L'%';

	static const ImmutableString TLS_SECTION(".tls");
}


namespace coff
{
	ObjFile* OpenObj(const wchar_t* filename)
	{
		// BEGIN EPIC MOD
		Filesystem::Path temp;
		filename = Filesystem::Devirtualize(filename, temp);
		// END EPIC MOD

		ObjFile* objFile = LC_NEW(&g_objFileAllocator, ObjFile);
		objFile->filename = string::ToUtf8String(filename);
		objFile->memoryFile = Filesystem::OpenMemoryMappedFile(filename, Filesystem::OpenMode::READ, true);

		return objFile;
	}


	void CloseObj(ObjFile*& objFile)
	{
		Filesystem::CloseMemoryMappedFile(objFile->memoryFile);
		
		LC_DELETE(&g_objFileAllocator, objFile, sizeof(ObjFile));
		objFile = nullptr;
	}


	static size_t FindSymbolShortNameLength(const char* shortName)
	{
		size_t length = 0u;
		while ((shortName[length] != '\0') && (length < 8u))
		{
			// find (optional!) null-terminator
			++length;
		}

		return length;
	}


	static ImmutableString GetSymbolShortName(const BYTE* shortName)
	{
		const char* str = reinterpret_cast<const char*>(shortName);
		return ImmutableString(str, FindSymbolShortNameLength(str));
	}


	static ImmutableString DisambiguateStaticSymbolName(const char* name, size_t nameLength, uint32_t uniqueCoffId, uint16_t uniqueCounter)
	{
		// static symbols are not necessarily unique across different translation units, hence we need something
		// to disambiguate them. this is done by appending the unique ID of the .obj file in which this symbol is defined.
		// the new name has the form "name%ID" (plus a null terminator), e.g. "g_counter2%AF20"
		const size_t frontLength = nameLength;
		const size_t uniqueIdLength = 8u;

		const bool needsCounter = (uniqueCounter != 0u);
		const size_t length = needsCounter
			? frontLength + 1u + uniqueIdLength + 1u + 4u		// unlikely case, add a unique counter to the name of the symbol
			: frontLength + 1u + uniqueIdLength + 1u;			// more likely case, no unique counter needed

		char* temp = static_cast<char*>(_alloca(length));
		char* finalName = temp;

		memcpy(temp, name, frontLength);
		if (needsCounter)
		{
			temp[frontLength + 0u] = "0123456789ABCDEF"[(uniqueCounter >> 12u) & 0x0F];
			temp[frontLength + 1u] = "0123456789ABCDEF"[(uniqueCounter >> 8u) & 0x0F];
			temp[frontLength + 2u] = "0123456789ABCDEF"[(uniqueCounter >> 4u) & 0x0F];
			temp[frontLength + 3u] = "0123456789ABCDEF"[(uniqueCounter >> 0u) & 0x0F];
			temp += 4u;
		}

		temp[frontLength] = COFF_SUFFIX;

		// store unique ID in hex, no leading zeros
		bool wasValueWritten = false;
		temp += frontLength + 1u;
		for (unsigned int i = 0u; i < 8u; ++i)
		{
			const uint32_t shift = (7u - i) * 4u;		// 28, 24, 20, 16, ...
			const uint32_t index = (uniqueCoffId >> shift) & 0x0F;

			// if no value != zero has been written yet, don't advance to the next position.
			// this makes sure that no leading zeros are written and automatically handles the case of ID == 0,
			// where at least one zero has to be written.
			if (wasValueWritten)
			{
				++temp;
			}
			*temp = "0123456789ABCDEF"[index];

			if (index != 0u)
			{
				wasValueWritten = true;
			}
		}

		temp[1] = '\0';

		return ImmutableString(finalName);
	}


	template <typename T>
	static ImmutableString GetSymbolName(const char* stringTable, const T* symbol)
	{
		/*
		From the COFF spec:
			The ShortName field in a symbol table consists of 8 bytes that contain the name
			itself, if it is not more than 8 bytes long, or the ShortName field gives an offset into
			the string table. To determine whether the name itself or an offset is given, test the
			first 4 bytes for equality to zero.
		*/
		if (symbol->N.Name.Short != 0)
		{
			// short name
			return GetSymbolShortName(symbol->N.ShortName);
		}
		else
		{
			// long name, points into string table
			return ImmutableString(stringTable + symbol->N.Name.Long);
		}
	}


	template <typename T>
	static ImmutableString GetSymbolName(const char* stringTable, const T* symbol, uint32_t uniqueId, uint16_t uniqueCounter)
	{
		/*
		From the COFF spec:
			The ShortName field in a symbol table consists of 8 bytes that contain the name
			itself, if it is not more than 8 bytes long, or the ShortName field gives an offset into
			the string table. To determine whether the name itself or an offset is given, test the
			first 4 bytes for equality to zero.
		*/

		// static symbols must have their name disambiguated across several translation units.
		// this is true even for COMDAT symbols, because COMDAT folding done by the linker depends on linker settings
		// and (seemingly) the type/name of the symbol, e.g. folding is done for template functions, but not for
		// identical static inline functions in several translation units.
		const bool isStatic = (symbol->StorageClass == IMAGE_SYM_CLASS_STATIC);
		if (symbol->N.Name.Short != 0)
		{
			// short name
			if (isStatic)
			{
				const char* str = reinterpret_cast<const char*>(symbol->N.ShortName);
				return DisambiguateStaticSymbolName(str, FindSymbolShortNameLength(str), uniqueId, uniqueCounter);
			}
			else
			{
				return GetSymbolShortName(symbol->N.ShortName);
			}
		}
		else
		{
			// long name, points into string table
			const char* str = stringTable + symbol->N.Name.Long;
			return isStatic
				? DisambiguateStaticSymbolName(str, strlen(str), uniqueId, uniqueCounter)
				: ImmutableString(str);
		}
	}


	static ImmutableString GetSectionName(const char* stringTable, const IMAGE_SECTION_HEADER* section)
	{
		/*
		From the COFF spec:
			An 8-byte, null-padded UTF-8 encoded string. If the string is exactly 8 characters long, there is no
			terminating null. For longer names, this field contains a slash (/) that is followed by an ASCII
			representation of a decimal number that is an offset into the string table. Executable images do
			not use a string table and do not support section names longer than 8 characters. Long names in
			object files are truncated if they are emitted to an executable file.
		*/

		if (section->Name[0] == '/')
		{
			// potentially a long name, but could also be a section starting with '/'
			unsigned int offset = 0;
			const int convertedCount = sscanf_s(reinterpret_cast<const char*>(&section->Name[1]), "%u", &offset);
			if (convertedCount > 0)
			{
				return ImmutableString(stringTable + offset);
			}

			// could not convert decimal number, hence the section short name starts with '/'
		}

		// short name
		return GetSymbolShortName(section->Name);
	}


	static bool SortSymbolByAscendingRVA(const Symbol* lhs, const Symbol* rhs)
	{
		return lhs->rva < rhs->rva;
	}


	template <typename T>
	static coff::SymbolType::Enum DetermineDataSymbolType(const T* symbol)
	{
		LC_ASSERT(!coffDetail::IsFunctionSymbol(symbol), "Symbol must be a data symbol");

		if (symbol->StorageClass == IMAGE_SYM_CLASS_EXTERNAL)
		{
			return coff::SymbolType::EXTERNAL_DATA;
		}

		if ((symbol->StorageClass == IMAGE_SYM_CLASS_STATIC) || (symbol->Value == 0u))
		{
			return coff::SymbolType::STATIC_DATA;
		}

		return coff::SymbolType::UNKNOWN_DATA;
	}


	template <typename T>
	static coff::SymbolType::Enum DetermineFunctionSymbolType(const T* symbol)
	{
		LC_ASSERT(coffDetail::IsFunctionSymbol(symbol), "Symbol must be a function symbol");

		if (symbol->StorageClass == IMAGE_SYM_CLASS_EXTERNAL)
		{
			return coff::SymbolType::EXTERNAL_FUNCTION;
		}

		if (symbol->StorageClass == IMAGE_SYM_CLASS_STATIC)
		{
			return coff::SymbolType::STATIC_FUNCTION;
		}

		return coff::SymbolType::UNKNOWN_FUNCTION;
	}


	template <typename T>
	static coff::SymbolType::Enum DetermineSymbolType(const T* symbol)
	{
		return coffDetail::IsFunctionSymbol(symbol) ? DetermineFunctionSymbolType(symbol) : DetermineDataSymbolType(symbol);
	}


	template <typename RawCoffType, coffDetail::CoffType::Enum Type>
	static RawCoffType* ReadRaw(ObjFile* file, uint32_t uniqueId)
	{
		using CoffHeader = typename coffDetail::CoffType::TypeMap<Type>::HeaderType;
		using CoffSymbol = typename coffDetail::CoffType::TypeMap<Type>::SymbolType;
		using CoffAuxSymbol = typename coffDetail::CoffType::TypeMap<Type>::AuxSymbolType;

		RawCoffType* rawCoff = new RawCoffType;
		rawCoff->size = Filesystem::GetMemoryMappedSize(file->memoryFile);
		rawCoff->type = RawCoffType::TYPE;

		const void* objFileBase = Filesystem::GetMemoryMappedFileData(file->memoryFile);

		// section header is first
		const CoffHeader* rawCoffHeader = pointer::As<const CoffHeader*>(objFileBase);
		rawCoff->header = *rawCoffHeader;

		// section headers follow after that
		const IMAGE_SECTION_HEADER* rawSectionHeaders = coffDetail::GetSectionHeader<CoffHeader>(objFileBase);
		const uint32_t sectionCount = coffDetail::GetNumberOfSections<CoffHeader>(objFileBase);
		rawCoff->sections.reserve(sectionCount);
		for (uint32_t i = 0u; i < sectionCount; ++i)
		{
			const IMAGE_SECTION_HEADER* rawSection = rawSectionHeaders + i;

			RawSection section;

			// header
			{
				section.header = *rawSection;
			}

			// raw data
			if (rawSection->PointerToRawData != 0u)
			{
				// some sections like .bss don't store (uninitialized) data
				const DWORD rawDataSize = rawSection->SizeOfRawData;
				section.data = LC_ALLOC(&g_rawCoffAllocator, rawDataSize, 8u);
				memcpy(section.data, pointer::Offset<const void*>(objFileBase, rawSection->PointerToRawData), rawDataSize);
			}

			// relocations
			{
				const IMAGE_RELOCATION* rawRelocations = pointer::Offset<const IMAGE_RELOCATION*>(objFileBase, rawSection->PointerToRelocations);
				const size_t count = coffDetail::GetRelocationCount(objFileBase, rawSection);

				// if relocation count in section has overflown, ignore the first relocation
				const size_t startRelocation = (count > 0xFFFFu) ? 1u : 0u;
				section.relocations.reserve(count - startRelocation);
				for (size_t j = 0u; j < count - startRelocation; ++j)
				{
					section.relocations.push_back(rawRelocations[j + startRelocation]);
				}
			}

			// line numbers
			{
				const IMAGE_LINENUMBER* rawLineNumbers = pointer::Offset<const IMAGE_LINENUMBER*>(objFileBase, rawSection->PointerToLinenumbers);
				const size_t count = rawSection->NumberOfLinenumbers;
				section.lineNumbers.reserve(count);
				for (size_t j = 0u; j < count; ++j)
				{
					section.lineNumbers.push_back(rawLineNumbers[j]);
				}
			}

			rawCoff->sections.emplace_back(std::move(section));
		}

		// symbol table follows after section headers, but offset is stored in COFF header directly
		const void* rawSymbolTable = coffDetail::GetSymbolTable<CoffHeader>(objFileBase);
		const uint32_t symbolCount = coffDetail::GetNumberOfSymbols<CoffHeader>(objFileBase);
		rawCoff->symbols.reserve(symbolCount);
		for (uint32_t i = 0u; i < symbolCount; ++i)
		{
			const CoffSymbol* rawSymbol = coffDetail::GetSymbol<CoffSymbol>(rawSymbolTable, i);
			rawCoff->symbols.push_back(*rawSymbol);
		}

		// string table follows after symbol table
		const char* rawStringTable = coffDetail::GetStringTable<CoffSymbol>(rawSymbolTable, symbolCount);

		// first 4 bytes contain the total size of the string table (including these 4 bytes)
		rawCoff->rawStringTable.size = *pointer::As<const uint32_t*>(rawStringTable);
		rawCoff->rawStringTable.data = static_cast<char*>(LC_ALLOC(&g_rawCoffAllocator, rawCoff->rawStringTable.size, 8u));
		memcpy(rawCoff->rawStringTable.data, rawStringTable, rawCoff->rawStringTable.size);

		// construct string table
		{
			rawCoff->stringTable.resize(symbolCount);

			types::StringMap<uint16_t> uniqueStaticDataSymbols;
			uniqueStaticDataSymbols.reserve(16u);

			for (uint32_t i = 0u; i < symbolCount; ++i)
			{
				const CoffSymbol* symbol = coffDetail::GetSymbol<CoffSymbol>(rawSymbolTable, i);

				if (coffDetail::IsAbsoluteSymbol(symbol))
				{
					continue;
				}
				else if (coffDetail::IsDebugSymbol(symbol))
				{
					continue;
				}
				else if (coffDetail::IsSectionSymbol(symbol))
				{
					continue;
				}

				const SymbolType::Enum type = DetermineSymbolType(symbol);
				rawCoff->stringTable[i] = GetSymbolName(rawStringTable, symbol, uniqueId, 0u);
				const ImmutableString& name = rawCoff->stringTable[i];

				if (type == SymbolType::STATIC_DATA)
				{
					// make sure this symbol is unique
					auto insertIterator = uniqueStaticDataSymbols.emplace(name, static_cast<uint16_t>(0u));
					if (insertIterator.second == false)
					{
						// the name of this symbol is not unique.
						// as a workaround, try generating a unique name for it.
						// this does not fix potential issues in all cases, but works successfully in cases where
						// this compiland is never recompiled, or where the order of variables in the file doesn't change.

						// increase the counter associated with this name and generate a new name from it
						++insertIterator.first->second;
						rawCoff->stringTable[i] = GetSymbolName(rawStringTable, symbol, uniqueId, insertIterator.first->second);
					}
				}
			}
		}

		// construct look-up table of associated COMDAT sections
		{
			for (uint32_t i = 0u; i < symbolCount; ++i)
			{
				const CoffSymbol* symbol = coffDetail::GetSymbol<CoffSymbol>(rawSymbolTable, i);

				// ignore absolute, debug and undefined symbols
				if (coffDetail::IsAbsoluteSymbol(symbol))
				{
				}
				else if (coffDetail::IsDebugSymbol(symbol))
				{
				}
				else if (coffDetail::IsUndefinedSymbol(symbol))
				{
				}
				else if (coffDetail::IsSectionSymbol(symbol))
				{
					// if this is a COMDAT section, grab its selection number from the auxiliary record
					const uint32_t symbolSectionIndex = coffDetail::GetSectionIndex(symbol);
					RawSection& section = rawCoff->sections[symbolSectionIndex];
					if (coffDetail::IsComdatSection(&section.header))
					{
						// the auxiliary record holds information about the COMDAT section. according to the COFF spec 5.5.6,
						// a COMDAT section always has one auxiliary record which is "the COMDAT symbol".
						if (symbol->NumberOfAuxSymbols == 1u)
						{
							const CoffAuxSymbol* auxSymbol = coffDetail::GetSymbol<CoffAuxSymbol>(rawSymbolTable, i + 1u);
							const uint8_t selection = auxSymbol->Section.Selection;
							if (selection == IMAGE_COMDAT_SELECT_ASSOCIATIVE)
							{
								const uint32_t associatedSectionIndex = coffDetail::GetAssociatedComdatSectionIndex<CoffAuxSymbol>(auxSymbol);
								rawCoff->associatedComdatSections[associatedSectionIndex].push_back(symbolSectionIndex);
							}
							else if (selection == IMAGE_COMDAT_SELECT_ANY)
							{
								section.isSelectAnyComdat = true;
							}
						}
					}
				}
			}
		}

		return rawCoff;
	}


	RawCoff* ReadRaw(ObjFile* file, uint32_t uniqueId)
	{
		const coffDetail::CoffType::Enum type = coffDetail::GetCoffType(Filesystem::GetMemoryMappedFileData(file->memoryFile));
		if (type == coffDetail::CoffType::COFF)
		{
			return ReadRaw<RawCoffRegular, coffDetail::CoffType::COFF>(file, uniqueId);
		}
		else if (type == coffDetail::CoffType::BIGOBJ)
		{
			return ReadRaw<RawCoffBigObj, coffDetail::CoffType::BIGOBJ>(file, uniqueId);
		}

		return nullptr;
	}


	template <coffDetail::CoffType::Enum Type, typename RawCoffType>
	static void WriteRaw(const wchar_t* filename, const RawCoffType* rawCoff, SymbolRemovalStrategy::Enum removalStrategy)
	{
		using CoffHeader = typename coffDetail::CoffType::TypeMap<Type>::HeaderType;
		using CoffSymbol = typename coffDetail::CoffType::TypeMap<Type>::SymbolType;

		// the file structure is as follows:
		//	- COFF file header
		//	- all section headers
		//	- all symbols
		//	- string table
		//	- raw data for section 0
		//	- relocations for section 0
		//	- line numbers for section 0
		//	- ...
		//	- raw data for section N
		//	- relocations for section N
		//	- line numbers for section N

		// keeping the section data, relocations, and line numbers at the end of the file makes it easier to keep track
		// of file offsets that are stored in the section headers.
		memoryStream::Writer outputData(static_cast<size_t>(rawCoff->size));
		CoffHeader coffHeader(rawCoff->header);

		// careful: the number of sections is of type WORD, but DWORD in files compiled with /bigobj!
		typedef decltype(CoffHeader::NumberOfSections) SectionCountType;
		coffHeader.NumberOfSections = static_cast<SectionCountType>(rawCoff->sections.size());

		coffHeader.NumberOfSymbols = static_cast<DWORD>(rawCoff->symbols.size());
		coffHeader.PointerToSymbolTable = static_cast<DWORD>(sizeof(CoffHeader) + rawCoff->sections.size() * sizeof(IMAGE_SECTION_HEADER));

		outputData.Write(coffHeader);
		const size_t baseOffset =
			sizeof(CoffHeader) +
			rawCoff->sections.size() * sizeof(IMAGE_SECTION_HEADER) +
			rawCoff->symbols.size() * sizeof(CoffSymbol) +
			rawCoff->rawStringTable.size;

		size_t fileOffset = baseOffset;
		for (size_t i = 0u; i < rawCoff->sections.size(); ++i)
		{
			const RawSection& section(rawCoff->sections[i]);
			IMAGE_SECTION_HEADER sectionHeader(rawCoff->sections[i].header);
			if (sectionHeader.PointerToRawData != 0u)
			{
				// .bss and sections with uninitialized data have a size, but don't store any actual data
				sectionHeader.PointerToRawData = static_cast<DWORD>(fileOffset);
				fileOffset += sectionHeader.SizeOfRawData;
			}

			if (section.relocations.size() == 0u)
			{
				sectionHeader.PointerToRelocations = 0u;
				sectionHeader.NumberOfRelocations = 0u;
			}
			else
			{
				sectionHeader.PointerToRelocations = static_cast<DWORD>(fileOffset);

				size_t relocationCount = section.relocations.size();
				if (relocationCount >= 0xFFFFu)
				{
					// relocation count would overflow, so we need to add one extra relocation that stores
					// the actual number of relocations.
					sectionHeader.NumberOfRelocations = 0xFFFFu;
					sectionHeader.Characteristics |= IMAGE_SCN_LNK_NRELOC_OVFL;
					++relocationCount;
				}
				else
				{
					sectionHeader.NumberOfRelocations = static_cast<WORD>(relocationCount);
				}

				fileOffset += relocationCount * sizeof(IMAGE_RELOCATION);
			}

			if (section.lineNumbers.size() == 0u)
			{
				sectionHeader.PointerToLinenumbers = 0u;
			}
			else
			{
				sectionHeader.PointerToLinenumbers = static_cast<DWORD>(fileOffset);
				fileOffset += section.lineNumbers.size() * sizeof(IMAGE_LINENUMBER);
			}
			sectionHeader.NumberOfLinenumbers = static_cast<WORD>(section.lineNumbers.size());

			outputData.Write(sectionHeader);
		}

		for (size_t i = 0u; i < rawCoff->symbols.size(); ++i)
		{
			outputData.Write(rawCoff->symbols[i]);
		}

		outputData.Write(rawCoff->rawStringTable.data, rawCoff->rawStringTable.size);

		for (size_t i = 0u; i < rawCoff->sections.size(); ++i)
		{
			const RawSection& section = rawCoff->sections[i];
			if (section.header.PointerToRawData != 0u)
			{
				outputData.Write(section.data, section.header.SizeOfRawData);
			}

			const size_t relocationCount = section.relocations.size();
			if (relocationCount >= 0xFFFFu)
			{
				// relocation count would overflow, so we need to add one extra relocation that stores
				// the actual number of relocations.
				IMAGE_RELOCATION dummyRelocation = {};
				dummyRelocation.RelocCount = static_cast<DWORD>(relocationCount);
				outputData.Write(dummyRelocation);
			}

			if (removalStrategy == SymbolRemovalStrategy::Enum::MSVC_COMPATIBLE)
			{
				for (size_t j = 0u; j < relocationCount; ++j)
				{
					const IMAGE_RELOCATION& relocation = section.relocations[j];
					outputData.Write(relocation);
				}
			}
			else if (removalStrategy == SymbolRemovalStrategy::Enum::LLD_COMPATIBLE)
			{
				// LLD does not allow SECREL relocations to absolute symbols (which are our fake stripped symbols),
				// unless these relocations are contained in a debug section.
				// debug sections can be identified by checking if the section in question is marked as being discardable.
				for (size_t j = 0u; j < relocationCount; ++j)
				{
					IMAGE_RELOCATION relocation = section.relocations[j];
					if (relocation.Type == Relocation::Type::SECTION_RELATIVE)
					{
						// only "fix" section-relative relocations in non-debug sections
						if (!coffDetail::IsDiscardableSection(&section.header))
						{
							// only "fix" section-relative relocations pointing to a removed symbol
							if (IsRemovedSymbol(rawCoff, relocation.SymbolTableIndex, removalStrategy))
							{
								// fix by whatever means necessary in order to make LLD happy.
								// this is a stripped symbol, so its relocation will be patched anyway.
								relocation.Type = Relocation::Type::RVA_32;
							}
						}
					}

					outputData.Write(relocation);
				}
			}

			for (size_t j = 0u; j < section.lineNumbers.size(); ++j)
			{
				outputData.Write(section.lineNumbers[j]);
			}
		}

		Filesystem::CreateFileWithData(filename, outputData.GetData(), outputData.GetSize());
	}


	void WriteRaw(const wchar_t* filename, const RawCoff* rawCoff, SymbolRemovalStrategy::Enum removalStrategy)
	{
		if (rawCoff->type == RawCoffRegular::TYPE)
		{
			WriteRaw<coffDetail::CoffType::Enum::COFF>(filename, static_cast<const RawCoffRegular*>(rawCoff), removalStrategy);
		}
		else if (rawCoff->type == RawCoffBigObj::TYPE)
		{
			WriteRaw<coffDetail::CoffType::Enum::BIGOBJ>(filename, static_cast<const RawCoffBigObj*>(rawCoff), removalStrategy);
		}
	}


	template <typename RawCoffType>
	static void DestroyRaw(RawCoffType* rawCoff)
	{
		for (size_t i = 0u; i < rawCoff->sections.size(); ++i)
		{
			const RawSection& section = rawCoff->sections[i];
			LC_FREE(&g_rawCoffAllocator, section.data, section.header.SizeOfRawData);
		}

		LC_FREE(&g_rawCoffAllocator, rawCoff->rawStringTable.data, rawCoff->rawStringTable.size);

		delete rawCoff;
	}


	void DestroyRaw(RawCoff* rawCoff)
	{
		if (rawCoff->type == RawCoffRegular::TYPE)
		{
			DestroyRaw(static_cast<RawCoffRegular*>(rawCoff));
		}
		else if (rawCoff->type == RawCoffBigObj::TYPE)
		{
			DestroyRaw(static_cast<RawCoffBigObj*>(rawCoff));
		}
	}


	size_t GetSymbolCount(const RawCoff* rawCoff)
	{
		if (rawCoff->type == RawCoffRegular::TYPE)
		{
			return static_cast<const RawCoffRegular*>(rawCoff)->symbols.size();
		}
		else if (rawCoff->type == RawCoffBigObj::TYPE)
		{
			return static_cast<const RawCoffBigObj*>(rawCoff)->symbols.size();
		}

		return 0u;
	}


	size_t GetSectionCount(const RawCoff* rawCoff)
	{
		return rawCoff->sections.size();
	}


	size_t GetAuxSymbolCount(const RawCoff* rawCoff, size_t symbolIndex)
	{
		if (rawCoff->type == RawCoffRegular::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffRegular*>(rawCoff)->symbols[symbolIndex];
			return symbol.NumberOfAuxSymbols;
		}
		else if (rawCoff->type == RawCoffBigObj::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffBigObj*>(rawCoff)->symbols[symbolIndex];
			return symbol.NumberOfAuxSymbols;
		}

		return 0u;
	}


	SymbolType::Enum GetSymbolType(const RawCoff* rawCoff, size_t index)
	{
		if (rawCoff->type == RawCoffRegular::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffRegular*>(rawCoff)->symbols[index];
			return DetermineSymbolType(&symbol);
		}
		else if (rawCoff->type == RawCoffBigObj::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffBigObj*>(rawCoff)->symbols[index];
			return DetermineSymbolType(&symbol);
		}

		return SymbolType::UNKNOWN_FUNCTION;
	}


	const ImmutableString& GetSymbolName(const RawCoff* rawCoff, size_t index)
	{
		return rawCoff->stringTable[index];
	}


	ImmutableString GetSectionName(const RawCoff* rawCoff, size_t index)
	{
		const IMAGE_SECTION_HEADER* header = &rawCoff->sections[index].header;
		return GetSectionName(rawCoff->rawStringTable.data, header);
	}


	uint32_t GetSymbolSectionIndex(const RawCoff* rawCoff, size_t index)
	{
		if (rawCoff->type == RawCoffRegular::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffRegular*>(rawCoff)->symbols[index];
			return coffDetail::GetSectionIndex(&symbol);
		}
		else if (rawCoff->type == RawCoffBigObj::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffBigObj*>(rawCoff)->symbols[index];
			return coffDetail::GetSectionIndex(&symbol);
		}

		return 0u;
	}


	bool IsAbsoluteSymbol(const RawCoff* rawCoff, size_t index)
	{
		if (rawCoff->type == RawCoffRegular::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffRegular*>(rawCoff)->symbols[index];
			return coffDetail::IsAbsoluteSymbol(&symbol);
		}
		else if (rawCoff->type == RawCoffBigObj::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffBigObj*>(rawCoff)->symbols[index];
			return coffDetail::IsAbsoluteSymbol(&symbol);
		}

		return false;
	}


	bool IsDebugSymbol(const RawCoff* rawCoff, size_t index)
	{
		if (rawCoff->type == RawCoffRegular::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffRegular*>(rawCoff)->symbols[index];
			return coffDetail::IsDebugSymbol(&symbol);
		}
		else if (rawCoff->type == RawCoffBigObj::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffBigObj*>(rawCoff)->symbols[index];
			return coffDetail::IsDebugSymbol(&symbol);
		}

		return false;
	}


	bool IsSectionSymbol(const RawCoff* rawCoff, size_t index)
	{
		if (rawCoff->type == RawCoffRegular::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffRegular*>(rawCoff)->symbols[index];
			return coffDetail::IsSectionSymbol(&symbol);
		}
		else if (rawCoff->type == RawCoffBigObj::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffBigObj*>(rawCoff)->symbols[index];
			return coffDetail::IsSectionSymbol(&symbol);
		}

		return false;
	}


	bool IsUndefinedSymbol(const RawCoff* rawCoff, size_t index)
	{
		if (rawCoff->type == RawCoffRegular::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffRegular*>(rawCoff)->symbols[index];
			return coffDetail::IsUndefinedSymbol(&symbol);
		}
		else if (rawCoff->type == RawCoffBigObj::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffBigObj*>(rawCoff)->symbols[index];
			return coffDetail::IsUndefinedSymbol(&symbol);
		}

		return false;
	}


	bool IsRemovedSymbol(const RawCoff* rawCoff, size_t index, SymbolRemovalStrategy::Enum removalStrategy)
	{
		if (rawCoff->type == RawCoffRegular::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffRegular*>(rawCoff)->symbols[index];
			return ((symbol.Type == IMAGE_SYM_TYPE_NULL) && (symbol.StorageClass == IMAGE_SYM_CLASS_NULL) && (symbol.SectionNumber == removalStrategy));
		}
		else if (rawCoff->type == RawCoffBigObj::TYPE)
		{
			const auto& symbol = static_cast<const RawCoffBigObj*>(rawCoff)->symbols[index];
			return ((symbol.Type == IMAGE_SYM_TYPE_NULL) && (symbol.StorageClass == IMAGE_SYM_CLASS_NULL) && (symbol.SectionNumber == removalStrategy));
		}

		return false;
	}


	bool IsSelectAnyComdatSection(const RawCoff* rawCoff, size_t sectionIndex)
	{
		return (rawCoff->sections[sectionIndex].isSelectAnyComdat);
	}


	static types::vector<std::string> TokenizeLinkerDirectives(const char* sectionData, size_t size)
	{
		types::vector<std::string> directives;

		// gather all linker commands by tokenizing raw data.
		// individual commands are separated by spaces.
		const char* start = sectionData;
		for (size_t i = 0u; i < size; ++i)
		{
			// look for spaces
			if (sectionData[i] == ' ')
			{
				if (sectionData + i > start)
				{
					std::string directive(start, sectionData + i);
					directives.emplace_back(directive);
				}

				start = sectionData + i + 1u;
			}
		}

		return directives;
	}


	template <coffDetail::CoffType::Enum Type>
	static types::vector<std::string> ExtractLinkerDirectives(const ObjFile* objFile)
	{
		using CoffHeader = typename coffDetail::CoffType::TypeMap<Type>::HeaderType;

		const void* objFileBase = Filesystem::GetMemoryMappedFileData(objFile->memoryFile);

		const uint32_t sectionCount = coffDetail::GetNumberOfSections<CoffHeader>(objFileBase);
		const IMAGE_SECTION_HEADER* sectionHeader = coffDetail::GetSectionHeader<CoffHeader>(objFileBase);

		for (unsigned int i = 0u; i < sectionCount; ++i)
		{
			const IMAGE_SECTION_HEADER* section = sectionHeader + i;
			if (coffDetail::IsDirectiveSection(section))
			{			
				const char* rawData = pointer::Offset<const char*>(objFileBase, section->PointerToRawData);
				return TokenizeLinkerDirectives(rawData, section->SizeOfRawData);
			}
		}

		return types::vector<std::string>();
	}


	types::vector<std::string> ExtractLinkerDirectives(const ObjFile* file)
	{
		const coffDetail::CoffType::Enum type = coffDetail::GetCoffType(Filesystem::GetMemoryMappedFileData(file->memoryFile));
		if (type == coffDetail::CoffType::COFF)
		{
			return ExtractLinkerDirectives<coffDetail::CoffType::COFF>(file);
		}
		else if (type == coffDetail::CoffType::BIGOBJ)
		{
			return ExtractLinkerDirectives<coffDetail::CoffType::BIGOBJ>(file);
		}

		return types::vector<std::string>();
	}


	types::vector<std::string> ExtractLinkerDirectives(const RawCoff* rawCoff)
	{
		const size_t sectionCount = rawCoff->sections.size();
		for (size_t i = 0u; i < sectionCount; ++i)
		{
			const RawSection& section = rawCoff->sections[i];
			if (coffDetail::IsDirectiveSection(&section.header))
			{
				return TokenizeLinkerDirectives(static_cast<const char*>(section.data), section.header.SizeOfRawData);
			}
		}

		return types::vector<std::string>();
	}


	void ReplaceLinkerDirectives(RawCoff* rawCoff, const types::vector<std::string>& directives)
	{
		const size_t sectionCount = rawCoff->sections.size();
		for (size_t i = 0u; i < sectionCount; ++i)
		{
			RawSection& section = rawCoff->sections[i];
			if (coffDetail::IsDirectiveSection(&section.header))
			{				
				std::string newDirectives;
				newDirectives.reserve(1024u);

				// separate directives with spaces
				const size_t count = directives.size();
				for (size_t j = 0u; j < count; ++j)
				{
					newDirectives += directives[j];
					newDirectives += ' ';
				}

				LC_FREE(&g_rawCoffAllocator, section.data, section.header.SizeOfRawData);
				section.header.SizeOfRawData = static_cast<DWORD>(newDirectives.length());
				section.data = LC_ALLOC(&g_rawCoffAllocator, newDirectives.length(), 8u);
				memcpy(section.data, newDirectives.data(), newDirectives.length());

				// we assume that there's only one directive section in a COFF file
				return;
			}
		}
	}


	template <typename RawCoffType>
	static void PatchFunctionSymbol(RawCoffType* rawCoff, size_t symbolIndex, uint32_t sectionIndex)
	{
		LC_LOG_DEV("Patching function symbol %d (%s)", symbolIndex, rawCoff->stringTable[symbolIndex].c_str());

		coff::RawSection& section = rawCoff->sections[sectionIndex];
		uint8_t* code = static_cast<uint8_t*>(section.data);
		code[0] = Assembler::Opcode::RET[0];
	}


	void PatchFunctionSymbol(RawCoff* rawCoff, size_t symbolIndex, uint32_t sectionIndex)
	{
		if (rawCoff->type == RawCoffRegular::TYPE)
		{
			PatchFunctionSymbol(static_cast<RawCoffRegular*>(rawCoff), symbolIndex, sectionIndex);
		}
		else if (rawCoff->type == RawCoffBigObj::TYPE)
		{
			PatchFunctionSymbol(static_cast<RawCoffBigObj*>(rawCoff), symbolIndex, sectionIndex);
		}
	}


	template <typename RawCoffType>
	static void RemoveSymbol(RawCoffType* rawCoff, size_t symbolIndex, SymbolRemovalStrategy::Enum removalStrategy)
	{
		LC_LOG_DEV("Removing symbol %d (%s)", symbolIndex, rawCoff->stringTable[symbolIndex].c_str());

		auto& symbol = rawCoff->symbols[symbolIndex];
		symbol.Type = IMAGE_SYM_TYPE_NULL;
		symbol.StorageClass = IMAGE_SYM_CLASS_NULL;

		// rather than removing the symbol, we fake the removed symbol by putting it in an "appropriate" section.
		// we need two different strategies, depending on the linker used.
		// MSVC can be tricked by putting the symbol into the debug section, but LLD will report
		// "error: relocation against symbol in discarded section" in this case.
		// LLD can be tricked by putting the symbol into an absolute section, but MSVC will report
		// "error LNK2016: absolute symbol '&' used as target of REL32 relocation in section" in this case.
		symbol.SectionNumber = removalStrategy;
		symbol.Value = 0u;

		// replace the symbol name with the shortest possible (illegal in C++) identifier
		memcpy(symbol.N.ShortName, "&", 2u);
	}


	void RemoveSymbol(RawCoff* rawCoff, size_t symbolIndex, SymbolRemovalStrategy::Enum removalStrategy)
	{
		if (rawCoff->type == RawCoffRegular::TYPE)
		{
			RemoveSymbol(static_cast<RawCoffRegular*>(rawCoff), symbolIndex, removalStrategy);
		}
		else if (rawCoff->type == RawCoffBigObj::TYPE)
		{
			RemoveSymbol(static_cast<RawCoffBigObj*>(rawCoff), symbolIndex, removalStrategy);
		}
	}


	void RemoveRelocations(RawCoff* rawCoff, size_t symbolIndex)
	{
		LC_LOG_DEV("Removing relocations pointing to symbol %d (%s)", symbolIndex, rawCoff->stringTable[symbolIndex].c_str());

		for (size_t i = 0u; i < rawCoff->sections.size(); ++i)
		{
			RawSection& section = rawCoff->sections[i];
			for (auto it = section.relocations.begin(); it != section.relocations.end(); /*nothing*/)
			{
				const IMAGE_RELOCATION& relocation = *it;
				if (relocation.SymbolTableIndex == symbolIndex)
				{
					it = section.relocations.erase(it);
				}
				else
				{
					++it;
				}
			}
		}
	}


	void RemoveSection(RawCoff* rawCoff, size_t sectionIndex)
	{
		LC_LOG_DEV("Removing section %d (%s)", sectionIndex, GetSectionName(rawCoff->rawStringTable.data, &rawCoff->sections[sectionIndex].header).c_str());

		// rather than really removing the section, we make it zero-sized without raw data, relocations, or line numbers.
		// otherwise, we would have to update the section numbers of all symbols for each removed section, which could
		// get complicated if we want to do it efficiently.
		RawSection& section = rawCoff->sections[sectionIndex];
		LC_FREE(&g_rawCoffAllocator, section.data, section.header.SizeOfRawData);
		section.header.SizeOfRawData = 0u;
		section.header.PointerToRawData = 0u;
		section.data = nullptr;
		section.header.NumberOfRelocations = 0u;
		section.header.PointerToRelocations = 0u;
		section.relocations.clear();
		section.header.NumberOfLinenumbers = 0u;
		section.header.PointerToLinenumbers = 0u;
		section.lineNumbers.clear();

		// furthermore, we also rename the section and set our own flags that tell the linker that this section
		// can and should be discarded.
		memcpy(section.header.Name, ".remove", 8u);
		section.header.Characteristics = IMAGE_SCN_LNK_REMOVE | IMAGE_SCN_MEM_DISCARDABLE;

		section.wasRemoved = true;
	}


	void RemoveAssociatedComdatSections(RawCoff* rawCoff, size_t sectionIndex)
	{
		LC_LOG_DEV("Removing COMDAT sections associated with section %d", sectionIndex);
		LC_LOG_INDENT_DEV;

		const types::vector<uint32_t>& associatedSections = rawCoff->associatedComdatSections[static_cast<uint32_t>(sectionIndex)];
		const size_t count = associatedSections.size();
		for (size_t i = 0u; i < count; ++i)
		{
			RemoveSection(rawCoff, associatedSections[i]);
		}
	}


	template <coffDetail::CoffType::Enum Type>
	static CoffDB* GatherDatabase(ObjFile* objFile, uint32_t uniqueId)
	{
		using CoffHeader = typename coffDetail::CoffType::TypeMap<Type>::HeaderType;
		using CoffSymbol = typename coffDetail::CoffType::TypeMap<Type>::SymbolType;
		using CoffAuxSymbol = typename coffDetail::CoffType::TypeMap<Type>::AuxSymbolType;

		CoffDB* coffDb = new CoffDB;
		coffDb->symbolAllocator = new PoolAllocator<PoolAllocatorSingleThreadPolicy>("COFF symbols", sizeof(coff::Symbol), alignof(coff::Symbol), 8192u);
		coffDb->relocationAllocator = new PoolAllocator<PoolAllocatorSingleThreadPolicy>("Relocations", sizeof(coff::Relocation), alignof(coff::Relocation), 8192u);

		const void* objFileBase = Filesystem::GetMemoryMappedFileData(objFile->memoryFile);

		const uint32_t sectionCount = coffDetail::GetNumberOfSections<CoffHeader>(objFileBase);
		const uint32_t symbolCount = coffDetail::GetNumberOfSymbols<CoffHeader>(objFileBase);
		const IMAGE_SECTION_HEADER* sectionHeader = coffDetail::GetSectionHeader<CoffHeader>(objFileBase);
		const void* symbolTable = coffDetail::GetSymbolTable<CoffHeader>(objFileBase);
		const char* stringTable = coffDetail::GetStringTable<CoffSymbol>(symbolTable, symbolCount);

		// pre-allocate data structures
		coffDb->sections.resize(sectionCount);
		coffDb->stringTable.resize(symbolCount);
		coffDb->symbols.reserve(symbolCount);

		// temporarily hold symbols per section in order to make assigning relocations to symbols easier
		types::vector<types::vector<Symbol*>> symbolsForSection;
		symbolsForSection.resize(sectionCount);

		// grab all sections and store CRT sections in a separate data structure, because they are needed for
		// finding dynamic initializers. we additionally use a lookup-table to make assigning symbols to CRT
		// sections faster.
		types::vector<uint32_t> lutSectionIndexToCrtSection;
		lutSectionIndexToCrtSection.resize(sectionCount, INVALID_CRT_SECTION);

		for (unsigned int i = 0u; i < sectionCount; ++i)
		{
			const IMAGE_SECTION_HEADER* section = sectionHeader + i;

			Section& s = coffDb->sections[i];
			s.name = GetSectionName(stringTable, section);
			s.rawDataSize = section->SizeOfRawData;
			s.rawDataRva = section->PointerToRawData;
			s.characteristics = section->Characteristics;
			s.comdatSelection = 0u;

			// store CRT sections in a separate data structure, they are needed for finding dynamic initializers
			if (string::Contains(s.name.c_str(), ".CRT$"))
			{
				// add this section to the lookup-table
				lutSectionIndexToCrtSection[i] = static_cast<uint32_t>(coffDb->crtSections.size());

				const CrtSection crtSection { s.name, section->SizeOfRawData, section->PointerToRawData, {} };
				coffDb->crtSections.push_back(crtSection);
			}

			// we would like to reserve space for the symbols of each section to avoid allocations where possible.
			// because we don't know yet how many symbols a section holds, using the number of relocations is a
			// good approximation.
			const unsigned int relocationCount = coffDetail::GetRelocationCount(objFileBase, section);
			symbolsForSection[i].reserve(relocationCount);
		}

		// unfortunately, some compilers such as VS 2013 and earlier do *not* generate fully unique names in COFFs
		// for certain symbols.
		// the simplest example are static (internal) data symbols with the same name in different namespaces:
		// namespace a
		// {
		//		static int g_counter = 10;
		// }

		// namespace b
		// {
		//		static int g_counter = 20;
		// }
		
		// the corresponding COFF file will have two symbols that are both named "g_counter", so there is no way
		// to distinguish them. at least the compiler only does this for data symbols, never for function symbols.
		// this means that we need to keep track of non-unique data symbols and try to fix them accordingly.
		types::StringMap<uint16_t> uniqueStaticDataSymbols;
		uniqueStaticDataSymbols.reserve(16u);

		for (unsigned int i = 0u; i < symbolCount; ++i)
		{
			const CoffSymbol* symbol = coffDetail::GetSymbol<CoffSymbol>(symbolTable, i);

			// ignore absolute, debug and section symbols
			if (coffDetail::IsAbsoluteSymbol(symbol))
			{
			}
			else if (coffDetail::IsDebugSymbol(symbol))
			{
			}
			else if (coffDetail::IsUndefinedSymbol(symbol))
			{
				coffDb->stringTable[i] = GetSymbolName(stringTable, symbol, uniqueId, 0u);
			}
			else if (coffDetail::IsSectionSymbol(symbol))
			{
				// if this is a COMDAT section, grab its selection number from the auxiliary record
				const uint32_t sectionIndex = coffDetail::GetSectionIndex(symbol);
				const IMAGE_SECTION_HEADER* section = sectionHeader + sectionIndex;

				if (coffDetail::IsComdatSection(section))
				{
					// the auxiliary record holds information about the COMDAT section. according to the COFF spec 5.5.6,
					// a COMDAT section always has one auxiliary record which is "the COMDAT symbol".
					if (symbol->NumberOfAuxSymbols == 1u)
					{
						const CoffAuxSymbol* auxSymbol = coffDetail::GetSymbol<CoffAuxSymbol>(symbolTable, i + 1u);
						coffDb->sections[sectionIndex].comdatSelection = auxSymbol->Section.Selection;
					}
				}

				coffDb->stringTable[i] = GetSectionName(stringTable, section);
			}
			else
			{
				// this symbol is stored in the COFF.
				coffDb->stringTable[i] = GetSymbolName(stringTable, symbol, uniqueId, 0u);
				const ImmutableString& name = coffDb->stringTable[i];

				// we are not interested in certain types of symbols.
				// they never store any relocations and don't convey any kind of meaningful information regarding
				// relocations.
				if (!coffDetail::IsLabelSymbol(symbol) && IsInterestingSymbol(name))
				{
					const uint32_t sectionIndex = coffDetail::GetSectionIndex(symbol);
					const IMAGE_SECTION_HEADER* section = sectionHeader + sectionIndex;
					const uint32_t rva = section->PointerToRawData + symbol->Value;
					const SymbolType::Enum type = DetermineSymbolType(symbol);

					Symbol* newSymbol = LC_NEW(coffDb->symbolAllocator, Symbol) { i, rva, sectionIndex, type, {} };
					newSymbol->relocations.reserve(32u);
					coffDb->symbols.push_back(newSymbol);

					// add the symbol to the corresponding CRT section, if any
					const uint32_t crtSectionIndex = lutSectionIndexToCrtSection[sectionIndex];
					if (crtSectionIndex != INVALID_CRT_SECTION)
					{
						CrtSection& crtSection = coffDb->crtSections[crtSectionIndex];
						crtSection.symbols.push_back(newSymbol);
					}

					if (type == SymbolType::STATIC_DATA)
					{
						// make sure this symbol is unique
						auto insertIterator = uniqueStaticDataSymbols.emplace(name, static_cast<uint16_t>(0u));
						if (insertIterator.second == false)
						{
							// the name of this symbol is not unique, inform the user.
							// when compiling with control-flow guard (CFG), the compiler will generate
							// non-unique __guard_fids__ symbols - ignore those.
							const ImmutableString& symbolName = GetSymbolName(stringTable, symbol);
							if (!string::Matches(symbolName.c_str(), "__guard_fids__"))
							{
								// BEGIN EPIC MOD
								// VS2019 seems to generate duplicate unwind and pdata block for the "dynamic atexit destructor" method (__F) under some conditions that I can't determine.
								// Also, do not generate warnings for symbols in COMDATs
								if (!coffDetail::IsComdatSection(section))
								{
									if (!string::StartsWith(symbolName.c_str(), "$unwind$??__F") &&
										!string::StartsWith(symbolName.c_str(), "$pdata$??__F"))
									{
										LC_WARNING_USER("Non-unique symbol %s found in COFF file %s. Do not change the order of these variables while live coding, or consider upgrading to a newer compiler (VS 2015 or later)",
											symbolName.c_str(), objFile->filename.c_str());
									}
									else
									{
										LC_LOG_USER("Non-unique at-exit symbol %s found in COFF file %s. These sometimes appear in debug builds",
											symbolName.c_str(), objFile->filename.c_str());
									}
								}
								// END EPIC MOD
							}

							// as a workaround, try generating a unique name for it.
							// this does not fix potential issues in all cases, but works successfully in cases where
							// this compiland is never recompiled, or where the order of variables in the file doesn't change.

							// increase the counter associated with this name and generate a new name from it
							++insertIterator.first->second;
							coffDb->stringTable[i] = GetSymbolName(stringTable, symbol, uniqueId, insertIterator.first->second);
						}
					}

					symbolsForSection[sectionIndex].push_back(newSymbol);
				}
			}

			// skip auxiliary symbols
			const BYTE auxSymbolCount = symbol->NumberOfAuxSymbols;
			i += auxSymbolCount;
		}

		// walk through all relocations
		{
			for (unsigned int i = 0u; i < sectionCount; ++i)
			{
				const IMAGE_SECTION_HEADER* section = sectionHeader + i;

				// ignore relocations inside sections that will either not be part of the final image, or can be
				// discarded at will. those are mostly ".drectve" and ".debug" sections.
				if (coffDetail::IsDiscardableSection(section))
				{
					continue;
				}
				if (!coffDetail::IsPartOfImage(section))
				{
					continue;
				}

				const unsigned int relocationCount = coffDetail::GetRelocationCount(objFileBase, section);
				const IMAGE_RELOCATION* relocations = pointer::Offset<const IMAGE_RELOCATION*>(objFileBase, section->PointerToRelocations);

				const types::vector<Symbol*>& symbolsForCurrentSection = symbolsForSection[i];
				if (symbolsForCurrentSection.size() == 0u)
				{
					// this section does not hold any symbols to which we could assign relocations
					continue;
				}

				// sort symbols in this section by RVA in order to make associating relocations with symbols much easier
				std::sort(symbolsForSection[i].begin(), symbolsForSection[i].end(), &SortSymbolByAscendingRVA);
				unsigned int currentSymbolIndex = 0u;

				// if relocation count in section has overflown, ignore the first relocation
				const unsigned int startRelocation = (relocationCount > 0xFFFFu) ? 1u : 0u;
				for (unsigned int j = 0u; j < relocationCount - startRelocation; ++j)
				{
					const IMAGE_RELOCATION* relocation = relocations + j + startRelocation;

					// ignore debug relocations
					if (coffDetail::IsDebugRelocation(relocation))
					{
						continue;
					}

					const CoffSymbol* symbol = coffDetail::GetSymbol<CoffSymbol>(symbolTable, relocation->SymbolTableIndex);
					const ImmutableString& dstSymbolName = coffDb->stringTable[relocation->SymbolTableIndex];

					// ignore relocations to line numbers and string literals
					if (!IsInterestingSymbol(dstSymbolName))
					{
						continue;
					}
					else if (symbols::IsRttiObjectLocator(dstSymbolName))
					{
						// RTTI Complete Object Locators are a strange thing.
						// they are always located before the first entry in the vtable, but they cannot be found using the vtable
						// symbol because that symbol starts at "section + 4", "skipping" the object locator.

						// example:
						// symbol "const SFV_Base::`vftable'" (??_7SFV_Base@@6B@)" has relocations to:
						//	00000000	??_R4SFV_Base@@6B@ (const SFV_Base::`RTTI Complete Object Locator')
						//	00000004	??_ESFV_Base@@UAEPAXI@Z (public: virtual void * __thiscall SFV_Base::`vector deleting destructor'(unsigned int))
						//	00000008	?TestFunction@SFV_Base@@UAEXXZ (public: virtual void __thiscall SFV_Base::TestFunction(void))
						
						// according to the symbol table, the vtable symbol sits in section .rdata, #1E, but at offset 4:
						//	00000004	SECT1E		External	??_7SFV_Base@@6B@ (const SFV_Base::`vftable')

						// this means that we don't have a symbol for which we can store this relocation.
						// it would be vtable - 4, but we don't handle that. it doesn't matter because the object locators
						// are public symbols anyway.
						continue;
					}

					const DWORD relocationRva = section->PointerToRawData + relocation->VirtualAddress;

					// find the symbol that contains this relocation and determine the RVA relative to the start
					// of the data or function. note that walking through the symbols of this section like this only
					// works because both the relocations as well as the symbols are sorted by their RVA.
					while (currentSymbolIndex < symbolsForCurrentSection.size() - 1u)
					{
						const uint32_t nextIndex = currentSymbolIndex + 1u;
						const uint32_t nextSymbolRva = symbolsForCurrentSection[nextIndex]->rva;
						if (relocationRva < nextSymbolRva)
						{
							// found symbol that holds this relocation
							break;
						}

						// relocation does not belong to this symbol, but possibly to the next
						currentSymbolIndex = nextIndex;
					}

					Symbol* srcSymbol = symbolsForCurrentSection[currentSymbolIndex];
					if (relocationRva < srcSymbol->rva)
					{
						LC_ERROR_DEV("Cannot find symbol that contains relocation at 0x%X for destination symbol %s in file %s", relocationRva, dstSymbolName.c_str(), objFile->filename.c_str());
						continue;
					}

					// RVA relative to the RVA of the symbol that holds the relocation
					const uint32_t srcRva = relocationRva - srcSymbol->rva;

					const int32_t sectionIndex = symbol->SectionNumber - 1;
					switch (relocation->Type)
					{
						case Relocation::Type::SECTION_RELATIVE:
						case Relocation::Type::RELATIVE:
						case Relocation::Type::VA_32:
						case Relocation::Type::RVA_32:

#if LC_64_BIT
						case Relocation::Type::RELATIVE_OFFSET_1:
						case Relocation::Type::RELATIVE_OFFSET_2:
						case Relocation::Type::RELATIVE_OFFSET_3:
						case Relocation::Type::RELATIVE_OFFSET_4:
						case Relocation::Type::RELATIVE_OFFSET_5:
#endif
						{
							const uint32_t* relocationAddress = pointer::Offset<const uint32_t*>(objFileBase, relocationRva);
							const uint32_t destinationOffset = *relocationAddress;
							Relocation* newRelocation = LC_NEW(coffDb->relocationAllocator, Relocation)
							{
								relocation->SymbolTableIndex, 
								srcRva,
								destinationOffset,
								sectionIndex,
								static_cast<Relocation::Type::Enum>(relocation->Type),
								srcSymbol->type,
								DetermineSymbolType(symbol),
								coffDetail::IsSectionSymbol(symbol)
							};
							srcSymbol->relocations.emplace_back(newRelocation);
						}
						break;

#if LC_64_BIT
						case Relocation::Type::VA_64:
						{
							const uint64_t* relocationAddress = pointer::Offset<const uint64_t*>(objFileBase, relocationRva);

							// read the destination offset as 64-bit, but convert it into a 32-bit offset.
							// no symbol can ever be larger than 4 GB.
							const uint64_t destinationOffset = *relocationAddress;
							Relocation* newRelocation = LC_NEW(coffDb->relocationAllocator, Relocation)
							{
								relocation->SymbolTableIndex,
								srcRva,
								static_cast<uint32_t>(destinationOffset),
								sectionIndex,
								static_cast<Relocation::Type::Enum>(relocation->Type),
								srcSymbol->type,
								DetermineSymbolType(symbol),
								coffDetail::IsSectionSymbol(symbol)
							};
							srcSymbol->relocations.emplace_back(newRelocation);
						}
						break;
#endif

						default:
							LC_ERROR_DEV("Unknown relocation %d", relocation->Type);
							break;
					};
				}
			}
		}

		// minimize amount of memory needed and generate lookup table
		coffDb->symbols.shrink_to_fit();
		{
			coffDb->indexToSymbol.resize(symbolCount);
			
			const size_t count = coffDb->symbols.size();
			for (size_t i = 0u; i < count; ++i)
			{
				Symbol* symbol = coffDb->symbols[i];
				symbol->relocations.shrink_to_fit();

				coffDb->indexToSymbol[symbol->nameIndex] = symbol;
			}
		}

		// sort symbols in CRT sections by RVA
		{
			const size_t count = coffDb->crtSections.size();
			for (size_t i = 0u; i < count; ++i)
			{
				std::sort(coffDb->crtSections[i].symbols.begin(), coffDb->crtSections[i].symbols.end(), &SortSymbolByAscendingRVA);
			}
		}

		return coffDb;
	}


	CoffDB* GatherDatabase(ObjFile* file, uint32_t uniqueId)
	{
		const coffDetail::CoffType::Enum type = coffDetail::GetCoffType(Filesystem::GetMemoryMappedFileData(file->memoryFile));
		if (type == coffDetail::CoffType::COFF)
		{
			return GatherDatabase<coffDetail::CoffType::COFF>(file, uniqueId);
		}
		else if (type == coffDetail::CoffType::BIGOBJ)
		{
			return GatherDatabase<coffDetail::CoffType::BIGOBJ>(file, uniqueId);
		}

		return nullptr;
	}		


	void DestroyDatabase(CoffDB* db)
	{
		const size_t count = db->symbols.size();
		for (size_t i = 0u; i < count; ++i)
		{
			Symbol* symbol = db->symbols[i];
			const size_t relocationCount = symbol->relocations.size();
			for (size_t j = 0u; j < relocationCount; ++j)
			{
				Relocation* relocation = symbol->relocations[j];
				LC_DELETE(db->relocationAllocator, relocation, sizeof(Relocation));
			}

			LC_DELETE(db->symbolAllocator, symbol, sizeof(Symbol));
		}

		delete db->symbolAllocator;
		delete db->relocationAllocator;

		delete db;
	}


	template <coffDetail::CoffType::Enum Type>
	static UnresolvedSymbolDB* GatherUnresolvedSymbolDatabase(ObjFile* objFile, uint32_t uniqueId)
	{
		using CoffHeader = typename coffDetail::CoffType::TypeMap<Type>::HeaderType;
		using CoffSymbol = typename coffDetail::CoffType::TypeMap<Type>::SymbolType;

		UnresolvedSymbolDB* symbolDb = new UnresolvedSymbolDB;

		const void* objFileBase = Filesystem::GetMemoryMappedFileData(objFile->memoryFile);

		const uint32_t symbolCount = coffDetail::GetNumberOfSymbols<CoffHeader>(objFileBase);
		const void* symbolTable = coffDetail::GetSymbolTable<CoffHeader>(objFileBase);
		const char* stringTable = coffDetail::GetStringTable<CoffSymbol>(symbolTable, symbolCount);

		// pre-allocate data structures
		symbolDb->symbols.reserve(symbolCount);
		symbolDb->symbolIndex.reserve(symbolCount);

		for (uint32_t i = 0u; i < symbolCount; ++i)
		{
			const CoffSymbol* symbol = coffDetail::GetSymbol<CoffSymbol>(symbolTable, i);

			// ignore absolute, debug and section symbols
			if (coffDetail::IsAbsoluteSymbol(symbol))
			{
			}
			else if (coffDetail::IsDebugSymbol(symbol))
			{
			}
			else if (coffDetail::IsUndefinedSymbol(symbol))
			{
				ImmutableString name = GetSymbolName(stringTable, symbol, uniqueId, 0u);
				symbolDb->symbols.emplace_back(std::move(name));
				symbolDb->symbolIndex.emplace_back(i);
			}
			else if (coffDetail::IsSectionSymbol(symbol))
			{
			}

			// skip auxiliary symbols
			const BYTE auxSymbolCount = symbol->NumberOfAuxSymbols;
			i += auxSymbolCount;
		}

		return symbolDb;
	}


	UnresolvedSymbolDB* GatherUnresolvedSymbolDatabase(ObjFile* file, uint32_t uniqueId)
	{
		const coffDetail::CoffType::Enum type = coffDetail::GetCoffType(Filesystem::GetMemoryMappedFileData(file->memoryFile));
		if (type == coffDetail::CoffType::COFF)
		{
			return GatherUnresolvedSymbolDatabase<coffDetail::CoffType::COFF>(file, uniqueId);
		}
		else if (type == coffDetail::CoffType::BIGOBJ)
		{
			return GatherUnresolvedSymbolDatabase<coffDetail::CoffType::BIGOBJ>(file, uniqueId);
		}

		return nullptr;
	}


	void DestroyDatabase(UnresolvedSymbolDB* db)
	{
		delete db;
	}


	template <coffDetail::CoffType::Enum Type>
	static ExternalSymbolDB* GatherExternalSymbolDatabase(ObjFile* objFile, uint32_t uniqueId)
	{
		using CoffHeader = typename coffDetail::CoffType::TypeMap<Type>::HeaderType;
		using CoffSymbol = typename coffDetail::CoffType::TypeMap<Type>::SymbolType;

		ExternalSymbolDB* symbolDb = new ExternalSymbolDB;

		const void* objFileBase = Filesystem::GetMemoryMappedFileData(objFile->memoryFile);

		const uint32_t symbolCount = coffDetail::GetNumberOfSymbols<CoffHeader>(objFileBase);
		const void* symbolTable = coffDetail::GetSymbolTable<CoffHeader>(objFileBase);
		const char* stringTable = coffDetail::GetStringTable<CoffSymbol>(symbolTable, symbolCount);

		// pre-allocate data structures
		symbolDb->symbols.reserve(symbolCount);
		symbolDb->types.reserve(symbolCount);

		for (uint32_t i = 0u; i < symbolCount; ++i)
		{
			const CoffSymbol* symbol = coffDetail::GetSymbol<CoffSymbol>(symbolTable, i);

			// ignore absolute, debug, section and undefined symbols
			if (coffDetail::IsAbsoluteSymbol(symbol))
			{
			}
			else if (coffDetail::IsDebugSymbol(symbol))
			{
			}
			else if (coffDetail::IsUndefinedSymbol(symbol))
			{
			}
			else if (coffDetail::IsSectionSymbol(symbol))
			{
			}
			else
			{
				const SymbolType::Enum type = DetermineSymbolType(symbol);
				if ((type == SymbolType::EXTERNAL_DATA) || (type == SymbolType::EXTERNAL_FUNCTION))
				{
					// this is an external symbol stored in the COFF
					ImmutableString name = GetSymbolName(stringTable, symbol, uniqueId, 0u);
					symbolDb->symbols.emplace_back(std::move(name));
					symbolDb->types.emplace_back(type);
				}
			}

			// skip auxiliary symbols
			const BYTE auxSymbolCount = symbol->NumberOfAuxSymbols;
			i += auxSymbolCount;
		}

		return symbolDb;
	}


	ExternalSymbolDB* GatherExternalSymbolDatabase(ObjFile* file, uint32_t uniqueId)
	{
		const coffDetail::CoffType::Enum type = coffDetail::GetCoffType(Filesystem::GetMemoryMappedFileData(file->memoryFile));
		if (type == coffDetail::CoffType::COFF)
		{
			return GatherExternalSymbolDatabase<coffDetail::CoffType::COFF>(file, uniqueId);
		}
		else if (type == coffDetail::CoffType::BIGOBJ)
		{
			return GatherExternalSymbolDatabase<coffDetail::CoffType::BIGOBJ>(file, uniqueId);
		}

		return nullptr;
	}


	void DestroyDatabase(ExternalSymbolDB* db)
	{
		delete db;
	}


	size_t GetIndexCount(const CoffDB* coffDb)
	{
		return coffDb->indexToSymbol.size();
	}


	const Symbol* GetSymbolByIndex(const CoffDB* coffDb, size_t index)
	{
		return coffDb->indexToSymbol[index];
	}


	const ImmutableString& GetSymbolName(const CoffDB* coffDb, const Symbol* symbol)
	{
		return coffDb->stringTable[symbol->nameIndex];
	}


	const ImmutableString& GetRelocationDstSymbolName(const CoffDB* coffDb, const Relocation* relocation)
	{
		return coffDb->stringTable[relocation->dstSymbolNameIndex];
	}


	const ImmutableString& GetUnresolvedSymbolName(const CoffDB* coffDb, size_t unresolvedSymbolIndex)
	{
		return coffDb->stringTable[unresolvedSymbolIndex];
	}


	SymbolType::Enum GetRelocationSrcSymbolType(const Relocation* relocation)
	{
		return relocation->srcSymbolType;
	}


	SymbolType::Enum GetRelocationDstSymbolType(const Relocation* relocation)
	{
		return relocation->dstSymbolType;
	}


	const CrtSection* FindCrtSection(const CoffDB* coffDb, const ImmutableString& sectionName, uint32_t sectionSize)
	{
		const CrtSection* foundSection = nullptr;

		const size_t count = coffDb->crtSections.size();
		for (size_t i = 0u; i < count; ++i)
		{
			const CrtSection* section = &coffDb->crtSections[i];
			if (section->rawDataSize != sectionSize)
			{
				continue;
			}
			else if (section->name != sectionName)
			{
				continue;
			}
			else if (foundSection)
			{
				// matching section has been found more than once
				return nullptr;
			}

			foundSection = section;
		}

		return foundSection;
	}


	std::vector<const CrtSection*> FindMatchingCrtSections(const CoffDB* coffDb, const ImmutableString& sectionName, uint32_t sectionSize)
	{
		std::vector<const CrtSection*> result;
		result.reserve(4u);

		const size_t count = coffDb->crtSections.size();
		for (size_t i = 0u; i < count; ++i)
		{
			const CrtSection* section = &coffDb->crtSections[i];
			if (section->rawDataSize != sectionSize)
			{
				continue;
			}
			else if (section->name != sectionName)
			{
				continue;
			}

			result.push_back(section);
		}

		return result;
	}


	uint32_t FindCoffSuffix(const ImmutableString& symbolName)
	{
		return symbolName.Find(COFF_SUFFIX);
	}


	uint32_t GetRelocationDestinationSectionCharacteristics(const coff::CoffDB* coffDb, const coff::Relocation* relocation)
	{
		if (relocation->dstSectionIndex < 0)
		{
			return 0u;
		}

		const uint32_t index = static_cast<uint32_t>(relocation->dstSectionIndex);
		const coff::Section& section = coffDb->sections[index];
		return section.characteristics;
	}


	const void* GetBaseAddress(const ObjFile* file)
	{
		return Filesystem::GetMemoryMappedFileData(file->memoryFile);
	}


	bool IsFunctionSymbol(SymbolType::Enum type)
	{
		switch (type)
		{
			case SymbolType::EXTERNAL_DATA:
				return false;

			case SymbolType::EXTERNAL_FUNCTION:
				return true;

			case SymbolType::STATIC_DATA:
				return false;

			case SymbolType::STATIC_FUNCTION:
				return true;

			case SymbolType::UNKNOWN_DATA:
				return false;

			case SymbolType::UNKNOWN_FUNCTION:
				return true;

			default:
				LC_ERROR_DEV("Unexpected SymbolType value %d", type);
				return false;
		}
	}

	
	char GetCoffSuffix(void)
	{
		return COFF_SUFFIX;
	}


	wchar_t GetWideCoffSuffix(void)
	{
		return COFF_SUFFIX_WIDE;
	}


	const ImmutableString& GetTlsSectionName(void)
	{
		return TLS_SECTION;
	}


	bool IsInterestingSymbol(const ImmutableString& name)
	{
		if (symbols::IsStringLiteral(name))
		{
			return false;
		}
		else if (symbols::IsFloatingPointSseAvxConstant(name))
		{
			return false;
		}
		else if (symbols::IsLineNumber(name))
		{
			return false;
		}

		return true;
	}


	bool IsMSVCJustMyCodeSection(const char* sectionName)
	{
		return string::Matches(sectionName, ".msvcjmc");
	}
}


#endif