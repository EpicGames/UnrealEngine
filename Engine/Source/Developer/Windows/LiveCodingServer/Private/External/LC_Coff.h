// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ImmutableString.h"
#include "LC_MemoryMappedFile.h"
// BEGIN EPIC MOD
#include "LC_Types.h"
#include "LC_Logging.h"
#include "Windows/WindowsHWrapper.h"

#undef RELATIVE
#undef OPTIONAL
// END EPIC MOD

struct PoolAllocatorSingleThreadPolicy;
template <typename T> class PoolAllocator;


namespace coff
{
	struct ObjFile
	{
		ImmutableString filename;
		Filesystem::MemoryMappedFile* memoryFile;

		LC_DISABLE_ASSIGNMENT(ObjFile);
	};

	struct SymbolType
	{
		enum Enum : uint8_t
		{
			EXTERNAL_DATA,
			EXTERNAL_FUNCTION,
			STATIC_DATA,
			STATIC_FUNCTION,
			UNKNOWN_DATA,
			UNKNOWN_FUNCTION
		};
	};

	struct Relocation
	{
		struct Type
		{
			enum Enum : uint8_t
			{
#if LC_64_BIT
				/*
				From the COFF spec: "5.2.1. Type Indicators"
					The following relocation types are defined for x64 and compatible processors.
					Constant					Description
					IMAGE_REL_AMD64_ABSOLUTE	The relocation is ignored.
					IMAGE_REL_AMD64_ADDR64		The 64-bit VA of the relocation target.
					IMAGE_REL_AMD64_ADDR32		The 32-bit VA of the relocation target.
					IMAGE_REL_AMD64_ADDR32NB	The 32-bit address without an image base (RVA).
					IMAGE_REL_AMD64_REL32		The 32-bit relative address from the byte following the relocation.
					IMAGE_REL_AMD64_REL32_1		The 32-bit address relative to byte distance 1 from the relocation.
					IMAGE_REL_AMD64_REL32_2		The 32-bit address relative to byte distance 2 from the relocation.
					IMAGE_REL_AMD64_REL32_3		The 32-bit address relative to byte distance 3 from the relocation.
					IMAGE_REL_AMD64_REL32_4		The 32-bit address relative to byte distance 4 from the relocation.
					IMAGE_REL_AMD64_REL32_5		The 32-bit address relative to byte distance 5 from the relocation.
					IMAGE_REL_AMD64_SECTION		The 16-bit section index of the section that contains the target.
												This is used to support debugging information.
					IMAGE_REL_AMD64_SECREL		The 32-bit offset of the target from the beginning of its section.
												This is used to support debugging information and static thread local storage.
					IMAGE_REL_AMD64_SECREL7		A 7-bit unsigned offset from the base of the section that contains the target.
					IMAGE_REL_AMD64_TOKEN		CLR tokens.
					IMAGE_REL_AMD64_SREL32		A 32-bit signed span-dependent value emitted into the object.
					IMAGE_REL_AMD64_PAIR		A pair that must immediately follow every span-dependent value.
					IMAGE_REL_AMD64_SSPAN32		A 32-bit signed span-dependent value that is applied at link time.

					This means that only a handful of relocation types need to be supported.
				*/

				// also used in 32-bit
				RELATIVE = IMAGE_REL_AMD64_REL32,
				SECTION_RELATIVE = IMAGE_REL_AMD64_SECREL,
				VA_32 = IMAGE_REL_AMD64_ADDR32,
				RVA_32 = IMAGE_REL_AMD64_ADDR32NB,

				// 64-bit only
				RELATIVE_OFFSET_1 = IMAGE_REL_AMD64_REL32_1,
				RELATIVE_OFFSET_2 = IMAGE_REL_AMD64_REL32_2,
				RELATIVE_OFFSET_3 = IMAGE_REL_AMD64_REL32_3,
				RELATIVE_OFFSET_4 = IMAGE_REL_AMD64_REL32_4,
				RELATIVE_OFFSET_5 = IMAGE_REL_AMD64_REL32_5,
				VA_64 = IMAGE_REL_AMD64_ADDR64,
#else
				/*
				From the COFF spec: "5.2.1. Type Indicators"
					The following relocation type indicators are defined for Intel 386 and compatible
					processors.
					Constant					Description
					IMAGE_REL_I386_ABSOLUTE		The relocation is ignored.
					IMAGE_REL_I386_DIR16		Not supported.
					IMAGE_REL_I386_REL16		Not supported.
					IMAGE_REL_I386_DIR32		The target's 32-bit VA.
					IMAGE_REL_I386_DIR32NB		The target's 32-bit RVA.
					IMAGE_REL_I386_SEG12		Not supported.
					IMAGE_REL_I386_SECTION		The 16-bit section index of the section that contains the target.
												This is used to support debugging information.
					IMAGE_REL_I386_SECREL		The 32-bit offset of the target from the beginning of its section.
												This is used to support debugging information and static thread local storage.
					IMAGE_REL_I386_TOKEN		The CLR token.
					IMAGE_REL_I386_SECREL7		A 7-bit offset from the base of the section that contains the target.
					IMAGE_REL_I386_REL32		The 32-bit relative displacement to the target.
												This supports the x86 relative branch and call instructions.

					This means that only a handful of relocation types need to be supported.
				*/

				RELATIVE = IMAGE_REL_I386_REL32,
				SECTION_RELATIVE = IMAGE_REL_I386_SECREL,
				VA_32 = IMAGE_REL_I386_DIR32,
				RVA_32 = IMAGE_REL_I386_DIR32NB,
#endif
				UNKNOWN = 0xFFu
			};

			static const char* ToString(Enum value)
			{
				switch (value)
				{
					case Type::RELATIVE:			return "RELATIVE";
					case Type::SECTION_RELATIVE:	return "SECTION_RELATIVE";
					case Type::VA_32:				return "VA_32";
					case Type::RVA_32:				return "RVA_32";

#if LC_64_BIT
					case Type::RELATIVE_OFFSET_1:	return "RELATIVE_OFFSET_1";
					case Type::RELATIVE_OFFSET_2:	return "RELATIVE_OFFSET_2";
					case Type::RELATIVE_OFFSET_3:	return "RELATIVE_OFFSET_3";
					case Type::RELATIVE_OFFSET_4:	return "RELATIVE_OFFSET_4";
					case Type::RELATIVE_OFFSET_5:	return "RELATIVE_OFFSET_5";
					case Type::VA_64:				return "VA_64";
#endif

					case Type::UNKNOWN:				return "UNKNOWN";
				}

				return "Invalid type";
			}

			// returns the byte distance to the position of where the relocation should be applied
			static uint32_t GetByteDistance(Enum type)
			{
				switch (type)
				{
					case Type::RELATIVE:
						return 0u;

#if LC_64_BIT
					case Type::RELATIVE_OFFSET_1:
						return 1u;

					case Type::RELATIVE_OFFSET_2:
						return 2u;

					case Type::RELATIVE_OFFSET_3:
						return 3u;

					case Type::RELATIVE_OFFSET_4:
						return 4u;

					case Type::RELATIVE_OFFSET_5:
						return 5u;

					case Type::VA_64:
#endif
					case Type::SECTION_RELATIVE:
					case Type::VA_32:
					case Type::RVA_32:
					case Type::UNKNOWN:
						break;
				}

				LC_ERROR_DEV("Unexpected relocation type %s (%d)", ToString(type), type);
				return 0u;
			}
		};

		uint32_t dstSymbolNameIndex;			// name of the symbol that the relocation points to
		uint32_t srcRva;						// relative to start of source symbol
		uint32_t dstOffset;						// the offset to the destination symbol to which the relocation is applied
												// e.g. a write to a 64-bit integer has two relocations: both to the same symbol, but one at offset 0, the other at offset 4
		int32_t dstSectionIndex;				// index of the section the destination symbol belongs to
		Type::Enum type;						// type of the relocation
		SymbolType::Enum srcSymbolType;			// symbol type of the source symbol, cached (does not increase struct size)
		SymbolType::Enum dstSymbolType;			// symbol type of the destination symbol, cached (does not increase struct size)
		bool dstIsSection;						// true if this relocation points to a section
	};

	struct Symbol
	{
		uint32_t nameIndex;
		uint32_t rva;
		uint32_t sectionIndex;
		SymbolType::Enum type;
		types::vector<Relocation*> relocations;
	};

	struct Section
	{
		ImmutableString name;
		uint32_t rawDataSize;
		uint32_t rawDataRva;
		uint32_t characteristics;				// TODO: can save memory by putting characteristics and selection into bitfields
		uint8_t comdatSelection;				// COMDAT selection specification, if any (0 means this is not a COMDAT section)

		LC_DISABLE_ASSIGNMENT(Section);
	};

	struct CrtSection
	{
		ImmutableString name;
		uint32_t rawDataSize;
		uint32_t rawDataRva;
		types::vector<Symbol*> symbols;		// no symbol ownership

		LC_DISABLE_ASSIGNMENT(CrtSection);
	};

	struct CoffDB
	{
		// the string table
		types::vector<ImmutableString> stringTable;

		// an array of all sections
		types::vector<Section> sections;

		// an array of all symbols.
		// symbol ownership.
		types::vector<Symbol*> symbols;

		// lookup-table from name index to corresponding Symbol*.
		// no symbol ownership.
		types::vector<Symbol*> indexToSymbol;

		// C/C++ runtime sections
		types::vector<CrtSection> crtSections;

		// allocators
		PoolAllocator<PoolAllocatorSingleThreadPolicy>* symbolAllocator;
		PoolAllocator<PoolAllocatorSingleThreadPolicy>* relocationAllocator;
	};

	struct LibEntry
	{
		ImmutableString objPath;			// path of the .obj file stored in the archive
		uint64_t offset;					// offset into the file at which the COFF is stored

		LC_DISABLE_ASSIGNMENT(LibEntry);
	};

	struct UnresolvedSymbolDB
	{
		types::vector<ImmutableString> symbols;
		types::vector<uint32_t> symbolIndex;
	};

	struct ExternalSymbolDB
	{
		types::vector<ImmutableString> symbols;
		types::vector<SymbolType::Enum> types;
	};



	struct RawSection
	{
		IMAGE_SECTION_HEADER header;
		void* data;
		types::vector<IMAGE_RELOCATION> relocations;
		types::vector<IMAGE_LINENUMBER> lineNumbers;
		bool wasRemoved;
		bool isSelectAnyComdat;

		RawSection(void)
			: header()
			, data(nullptr)
			, relocations()
			, lineNumbers()
			, wasRemoved(false)
			, isSelectAnyComdat(false)
		{
		}

		// move constructor
		RawSection(RawSection&& other)
			: header(std::move(other.header))
			, data(std::move(other.data))
			, relocations(std::move(other.relocations))
			, lineNumbers(std::move(other.lineNumbers))
			, wasRemoved(std::move(other.wasRemoved))
			, isSelectAnyComdat(std::move(other.isSelectAnyComdat))
		{
			other.data = nullptr;
		}

		LC_DISABLE_COPY(RawSection);
		LC_DISABLE_ASSIGNMENT(RawSection);
		LC_DISABLE_MOVE_ASSIGNMENT(RawSection);
	};

	struct RawStringTable
	{
		char* data;
		uint32_t size;
	};

	struct RawCoff
	{
		types::vector<RawSection> sections;
		types::vector<ImmutableString> stringTable;

		// indexed by section index, gives all section indices of COMDAT sections which are associated with this section
		types::unordered_map<uint32_t, types::vector<uint32_t>> associatedComdatSections;

		RawStringTable rawStringTable;
		uint64_t size;
		uint32_t type;
	};

	struct RawCoffRegular : public RawCoff
	{
		static const uint32_t TYPE = 0u;

		IMAGE_FILE_HEADER header;
		types::vector<IMAGE_SYMBOL> symbols;
	};

	struct RawCoffBigObj : public RawCoff
	{
		static const uint32_t TYPE = 1u;

		ANON_OBJECT_HEADER_BIGOBJ header;
		types::vector<IMAGE_SYMBOL_EX> symbols;
	};
	

	struct SymbolRemovalStrategy
	{
		enum Enum : SHORT
		{
			MSVC_COMPATIBLE = IMAGE_SYM_DEBUG,
			LLD_COMPATIBLE = IMAGE_SYM_ABSOLUTE
		};
	};


	// LIFETIME
	ObjFile* OpenObj(const wchar_t* filename);
	void CloseObj(ObjFile*& file);


	// UPDATE
	RawCoff* ReadRaw(ObjFile* file, uint32_t uniqueId);
	void WriteRaw(const wchar_t* filename, const RawCoff* rawCoff, SymbolRemovalStrategy::Enum removalStrategy);
	void DestroyRaw(RawCoff* rawCoff);


	size_t GetSymbolCount(const RawCoff* rawCoff);
	size_t GetSectionCount(const RawCoff* rawCoff);

	size_t GetAuxSymbolCount(const RawCoff* rawCoff, size_t symbolIndex);
	SymbolType::Enum GetSymbolType(const RawCoff* rawCoff, size_t index);
	const ImmutableString& GetSymbolName(const RawCoff* rawCoff, size_t index);
	ImmutableString GetSectionName(const RawCoff* rawCoff, size_t index);
	uint32_t GetSymbolSectionIndex(const RawCoff* rawCoff, size_t index);

	bool IsAbsoluteSymbol(const RawCoff* rawCoff, size_t index);
	bool IsDebugSymbol(const RawCoff* rawCoff, size_t index);
	bool IsSectionSymbol(const RawCoff* rawCoff, size_t index);
	bool IsUndefinedSymbol(const RawCoff* rawCoff, size_t index);
	bool IsRemovedSymbol(const RawCoff* rawCoff, size_t index, SymbolRemovalStrategy::Enum removalStrategy);

	bool IsSelectAnyComdatSection(const RawCoff* rawCoff, size_t sectionIndex);


	// extracts directives from a COFF file
	types::vector<std::string> ExtractLinkerDirectives(const ObjFile* file);

	// extracts directives from a raw COFF file
	types::vector<std::string> ExtractLinkerDirectives(const RawCoff* rawCoff);

	// replaces linker directives in the raw COFF file
	void ReplaceLinkerDirectives(RawCoff* rawCoff, const types::vector<std::string>& directives);


	// patches a function symbol to return immediately
	void PatchFunctionSymbol(RawCoff* rawCoff, size_t symbolIndex, uint32_t sectionIndex);

	// removes a symbol
	void RemoveSymbol(RawCoff* rawCoff, size_t symbolIndex, SymbolRemovalStrategy::Enum removalStrategy);

	// remove all relocations to the symbol with the given index
	void RemoveRelocations(RawCoff* rawCoff, size_t symbolIndex);

	// removes a section with the given index
	void RemoveSection(RawCoff* rawCoff, size_t sectionIndex);

	// removes all COMDAT sections that have the given section index as their associated section.
	// (e.g. a COMDAT section with associative section 0x5 only needs to become part of the image if
	// section 5 is also part of the image).
	void RemoveAssociatedComdatSections(RawCoff* rawCoff, size_t sectionIndex);


	// unique ID must uniquely identify this ObjFile. each obj file with a unique name must have a unique ID
	UnresolvedSymbolDB* GatherUnresolvedSymbolDatabase(ObjFile* file, uint32_t uniqueId);
	void DestroyDatabase(UnresolvedSymbolDB* db);

	// unique ID must uniquely identify this ObjFile. each obj file with a unique name must have a unique ID
	ExternalSymbolDB* GatherExternalSymbolDatabase(ObjFile* file, uint32_t uniqueId);
	void DestroyDatabase(ExternalSymbolDB* db);


	// unique ID must uniquely identify this ObjFile. each obj file with a unique name must have a unique ID
	CoffDB* GatherDatabase(ObjFile* file, uint32_t uniqueId);
	void DestroyDatabase(CoffDB* db);


	size_t GetIndexCount(const CoffDB* coffDb);
	const Symbol* GetSymbolByIndex(const CoffDB* coffDb, size_t index);

	const ImmutableString& GetSymbolName(const CoffDB* coffDb, const Symbol* symbol);
	const ImmutableString& GetRelocationDstSymbolName(const CoffDB* coffDb, const Relocation* relocation);
	const ImmutableString& GetUnresolvedSymbolName(const CoffDB* coffDb, size_t unresolvedSymbolIndex);

	SymbolType::Enum GetRelocationSrcSymbolType(const Relocation* relocation);
	SymbolType::Enum GetRelocationDstSymbolType(const Relocation* relocation);


	// finds a CRT section with the given name and size. returns nullptr if not found or ambiguous
	const CrtSection* FindCrtSection(const CoffDB* coffDb, const ImmutableString& sectionName, uint32_t sectionSize);

	// finds all matching CRT sections
	std::vector<const CrtSection*> FindMatchingCrtSections(const CoffDB* coffDb, const ImmutableString& sectionName, uint32_t sectionSize);


	uint32_t FindCoffSuffix(const ImmutableString& symbolName);


	// returns 0 if the relocation destination section is invalid
	uint32_t GetRelocationDestinationSectionCharacteristics(const coff::CoffDB* coffDb, const coff::Relocation* relocation);


	// ACCESS
	const void* GetBaseAddress(const ObjFile* file);

	bool IsFunctionSymbol(SymbolType::Enum type);

	char GetCoffSuffix(void);
	wchar_t GetWideCoffSuffix(void);

	const ImmutableString& GetTlsSectionName(void);


	bool IsInterestingSymbol(const ImmutableString& name);
	bool IsMSVCJustMyCodeSection(const char* sectionName);
}


#endif