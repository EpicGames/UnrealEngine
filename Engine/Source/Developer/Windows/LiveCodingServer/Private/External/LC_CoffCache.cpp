// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_CoffCache.h"
#include "LC_Coff.h"


template <typename T>
void Destroy(T*&) {}

template <>
void Destroy(coff::CoffDB*& database)
{
	coff::DestroyDatabase(database);
}

template <>
void Destroy(coff::ExternalSymbolDB*& database)
{
	coff::DestroyDatabase(database);
}

template <typename T>
CoffCache<T>::CoffCache(void)
{
	m_cache.reserve(4096u);
}


template <typename T>
CoffCache<T>::~CoffCache(void)
{
	for (auto it = m_cache.begin(); it != m_cache.end(); ++it)
	{
		T* database = it->second;
		Destroy(database);
	}
}


template <typename T>
void CoffCache<T>::Update(const ImmutableString& coffIdentifier, T* database)
{
	CriticalSection::ScopedLock lock(&m_cs);

	// try to insert the element into the cache. if it exists, delete the old entry.
	// if it doesn't exist, store the new element.
	const std::pair<typename Cache::iterator, bool>& insertPair = m_cache.emplace(coffIdentifier, nullptr);
	T*& data = insertPair.first->second;

	if (!insertPair.second)
	{
		// value exists already, delete the old entry
		Destroy(data);
	}

	data = database;
}


template <typename T>
const T* CoffCache<T>::Lookup(const ImmutableString& coffIdentifier) const
{
	CriticalSection::ScopedLock lock(&m_cs);

	const auto it = m_cache.find(coffIdentifier);
	if (it != m_cache.end())
	{
		return it->second;
	}

	return nullptr;
}


template <typename T>
bool CoffCache<T>::Contains(const ImmutableString& coffIdentifier) const
{
	CriticalSection::ScopedLock lock(&m_cs);
	return (m_cache.find(coffIdentifier) != m_cache.end());
}


// explicit template instantiation
template class CoffCache<coff::CoffDB>;
template class CoffCache<coff::ExternalSymbolDB>;


#endif