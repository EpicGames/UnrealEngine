// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "LC_Coff.h"
// END EPIC MOD
#include "LC_CriticalSection.h"


template <typename T>
class CoffCache
{
public:
	CoffCache(void);
	~CoffCache(void);

	// updates an entry in the cache. if it already exists, the existing one will be deleted.
	// takes ownership over the database.
	// thread-safe.
	void Update(const ImmutableString& coffIdentifier, T* database);

	// returns a database associated with a certain COFF id.
	const T* Lookup(const ImmutableString& coffIdentifier) const;

	bool Contains(const ImmutableString& coffIdentifier) const;

private:
	mutable CriticalSection m_cs;

	typedef types::StringMap<T*> Cache;
	Cache m_cache;
};


#endif