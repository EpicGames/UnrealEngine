// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_CoffDetail.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
// END EPIC MOD

namespace
{
#if LC_64_BIT
	static const WORD EXPECTED_MACHINE = IMAGE_FILE_MACHINE_AMD64;
#else
	static const WORD EXPECTED_MACHINE = IMAGE_FILE_MACHINE_I386;
#endif

	static const uint8_t BIGOBJ_MAGIC_UUID[16] = { 0xC7, 0xA1, 0xBA, 0xD1, 0xEE, 0xBA, 0xA9, 0x4B, 0xAF, 0x20, 0xFA, 0xF6, 0x6A, 0xA4, 0xDC, 0xB8 };
}


namespace coffDetail
{
	CoffType::Enum GetCoffType(const void* imageBase)
	{
		// check object file header first
		// check for DOS header
		const IMAGE_DOS_HEADER* dosHeader = pointer::As<const IMAGE_DOS_HEADER*>(imageBase);
		if (dosHeader->e_magic == IMAGE_DOS_SIGNATURE)
		{
			LC_ERROR_DEV("Unhandled DOS image in COFF file");
			return CoffType::UNKNOWN;
		}

		// check for import headers, which are part of .lib import libraries that belong to a .dll.
		// there is no meaningful information to extract from them, so ignore them.
		// see COFF Spec "8. Import Library Format"
		const IMPORT_OBJECT_HEADER* importHeader = pointer::As<const IMPORT_OBJECT_HEADER*>(imageBase);
		if ((importHeader->Sig1 == IMAGE_FILE_MACHINE_UNKNOWN) && (importHeader->Sig2 == IMPORT_OBJECT_HDR_SIG2))
		{
			// note that COFF files compiled with /bigobj have the same signature as import headers, so we also
			// need to distinguish between bigobjs and import headers.
			const ANON_OBJECT_HEADER_BIGOBJ* bigobjHeader = pointer::As<const ANON_OBJECT_HEADER_BIGOBJ*>(imageBase);
			if ((bigobjHeader->Version >= 2u) && memcmp(&bigobjHeader->ClassID, BIGOBJ_MAGIC_UUID, sizeof(BIGOBJ_MAGIC_UUID)) == 0)
			{
				// avoid machine mismatches, e.g. loading .obj files from a wrong directory or similar
				if (bigobjHeader->Machine != EXPECTED_MACHINE)
				{
					LC_ERROR_DEV("Unknown machine in COFF file");
					return CoffType::UNKNOWN;
				}

				return CoffType::BIGOBJ;
			}
			else
			{
				// avoid machine mismatches, e.g. loading .obj files from a wrong directory or similar
				if (importHeader->Machine != EXPECTED_MACHINE)
				{
					LC_ERROR_DEV("Unknown machine in COFF file");
					return CoffType::UNKNOWN;
				}

				return CoffType::IMPORT_LIBRARY;
			}
		}

		// we should be dealing with an ordinary COFF file now, but check to make sure
		const IMAGE_FILE_HEADER* imageHeader = pointer::As<const IMAGE_FILE_HEADER*>(imageBase);
		if (imageHeader->SizeOfOptionalHeader != 0u)
		{
			LC_ERROR_DEV("Unknown COFF file format");
			return CoffType::UNKNOWN;
		}

		// avoid machine mismatches, e.g. loading .obj files from a wrong directory or similar
		if (imageHeader->Machine != EXPECTED_MACHINE)
		{
			LC_ERROR_DEV("Unknown machine in COFF file");
			return CoffType::UNKNOWN;
		}

		return CoffType::COFF;
	}


	unsigned int GetRelocationCount(const void* imageBase, const IMAGE_SECTION_HEADER* section)
	{
		const DWORD relocationCount = section->NumberOfRelocations;

		/*
		From the COFF spec:
			IMAGE_SCN_LNK_NRELOC_OVFL indicates that the count of relocations for the
			section exceeds the 16 bits that are reserved for it in the section header. If the bit is
			set and the NumberOfRelocations field in the section header is 0xffff, the actual
			relocation count is stored in the 32-bit VirtualAddress field of the first relocation. It is
			an error if IMAGE_SCN_LNK_NRELOC_OVFL is set and there are fewer than 0xffff
			relocations in the section.
		*/

		const bool hasOverflow = ((section->Characteristics & IMAGE_SCN_LNK_NRELOC_OVFL) != 0u);
		if ((relocationCount == 0xFFFF) && hasOverflow)
		{
			const IMAGE_RELOCATION* firstRelocation = pointer::Offset<const IMAGE_RELOCATION*>(imageBase, section->PointerToRelocations);
			return firstRelocation->RelocCount;
		}

		return relocationCount;
	}
}


#endif