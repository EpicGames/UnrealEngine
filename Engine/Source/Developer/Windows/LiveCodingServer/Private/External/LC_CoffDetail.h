// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_PointerUtil.h"
// BEGIN EPIC MOD
#include "LC_Platform.h"
#include <inttypes.h>
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

namespace coffDetail
{
	struct CoffType
	{
		enum Enum
		{
			COFF,					// regular COFF/OBJ files
			BIGOBJ,					// OBJ files compiled with /bigobj
			IMPORT_LIBRARY,			// import libraries (DLL stubs)
			UNKNOWN
		};

		// template helper to map from CoffType::Enum values to types
		template <Enum>
		struct TypeMap {};

		template <>
		struct TypeMap<Enum::COFF>
		{
			typedef IMAGE_FILE_HEADER HeaderType;
			typedef IMAGE_SYMBOL SymbolType;
			typedef IMAGE_AUX_SYMBOL AuxSymbolType;
		};

		template <>
		struct TypeMap<Enum::BIGOBJ>
		{
			typedef ANON_OBJECT_HEADER_BIGOBJ HeaderType;
			typedef IMAGE_SYMBOL_EX SymbolType;
			typedef IMAGE_AUX_SYMBOL_EX AuxSymbolType;
		};
	};

	CoffType::Enum GetCoffType(const void* imageBase);

	// returns the number of relocations
	unsigned int GetRelocationCount(const void* imageBase, const IMAGE_SECTION_HEADER* section);


	template <typename SymbolType>
	inline uint32_t GetSectionIndex(const SymbolType* symbol)
	{
		// according to COFF spec, section number is a one-based (because zero is taken by UNDEFINED symbols)
		// signed integer, and signed values like 0, -1 and -2 have special meaning. note that
		// in cases where the index is a negative value that does not correspond to any of
		// the special values, the section number *must* be treated as an unsigned value, which is not
		// stated in the spec.
		// e.g. some non-bigobj COFF files that have more than 32767 sections will have symbols with
		// section numbers like 0x8000, 0x8001, etc. which need to be treated as unsigned values.
		typedef typename std::make_unsigned<decltype(symbol->SectionNumber)>::type UnsignedSectionNumberType;

		const uint32_t sectionIndex = static_cast<uint32_t>(static_cast<UnsignedSectionNumberType>(symbol->SectionNumber) - 1u);
		return sectionIndex;
	}


	template <typename AuxSymbolType>
	// BEGIN EPIC MOD
	uint32_t GetAssociatedComdatSectionIndex(const AuxSymbolType* symbol);
	// END EPIC MOD

	template <>
	inline uint32_t GetAssociatedComdatSectionIndex(const IMAGE_AUX_SYMBOL* symbol)
	{
		typedef std::make_unsigned<decltype(symbol->Section.Number)>::type UnsignedSectionNumberType;
		const uint32_t sectionIndex = static_cast<uint32_t>(static_cast<UnsignedSectionNumberType>(symbol->Section.Number) - 1u);
		return sectionIndex;
	}

	template <>
	inline uint32_t GetAssociatedComdatSectionIndex(const IMAGE_AUX_SYMBOL_EX* symbol)
	{
		typedef std::make_unsigned<decltype(symbol->Section.Number)>::type UnsignedSectionNumberType;
		typedef std::make_unsigned<decltype(symbol->Section.HighNumber)>::type UnsignedHighSectionNumberType;

		const uint32_t sectionIndex = static_cast<uint32_t>(static_cast<UnsignedSectionNumberType>(symbol->Section.Number) - 1u);

		// note that the high number is zero-based instead of one-based
		const uint32_t highSectionIndex = static_cast<uint32_t>(static_cast<UnsignedHighSectionNumberType>(symbol->Section.HighNumber));

		return (highSectionIndex << 16u) | sectionIndex;
	}


	template <typename SymbolType>
	inline bool IsFunctionSymbol(const SymbolType* symbol)
	{
		return ISFCN(symbol->Type);
	}


	/*
	From the COFF spec: 5.4.2 Section Number Values
		Normally, the Section Value field in a symbol table entry is a one-based index into the section table.
		However, this field is a signed integer and can take negative values. The following values, less than one,
		have special meanings.

		IMAGE_SYM_UNDEFINED	The symbol record is not yet assigned a section. A value of zero indicates that a reference
		to an external symbol is defined elsewhere. A value of non-zero is a common symbol with a size that is specified
		by the value.

		IMAGE_SYM_ABSOLUTE	The symbol has an absolute (non-relocatable) value and is not an address.

		IMAGE_SYM_DEBUG		The symbol provides general type or debugging information but does not correspond to a section.
		Microsoft tools use this setting along with .file records (storage class FILE).
	*/

	template <typename SymbolType>
	inline bool IsUndefinedSymbol(const SymbolType* symbol)
	{
		return (symbol->SectionNumber == IMAGE_SYM_UNDEFINED);
	}

	template <typename SymbolType>
	inline bool IsAbsoluteSymbol(const SymbolType* symbol)
	{
		return (symbol->SectionNumber == IMAGE_SYM_ABSOLUTE);
	}

	template <typename SymbolType>
	inline bool IsDebugSymbol(const SymbolType* symbol)
	{
		return (symbol->SectionNumber == IMAGE_SYM_DEBUG);
	}


	template <typename SymbolType>
	inline bool IsSectionSymbol(const SymbolType* symbol)
	{
		/*
		From the COFF spec: 5.5.5 Auxiliary Format 5: Section Definitions
			Follows a symbol-table record that defines a section. The auxiliary record provides information
			about the section to which it refers.
		*/
		if (symbol->NumberOfAuxSymbols > 0u)
		{
			/*
			From the COFF spec: 5.4.4 Storage Class
				IMAGE_SYM_CLASS_STATIC: The offset of the symbol within the section. If the Value field is zero,
				then the symbol represents a section name.
			*/
			if ((symbol->StorageClass == IMAGE_SYM_CLASS_STATIC) && (symbol->Value == 0u))
			{
				return true;
			}
		}

		return false;
	}

	
	template <typename SymbolType>
	inline bool IsLabelSymbol(const SymbolType* symbol)
	{
		if (symbol->StorageClass == IMAGE_SYM_CLASS_LABEL)
		{
			return true;
		}
		else if (symbol->StorageClass == IMAGE_SYM_CLASS_UNDEFINED_LABEL)
		{
			return true;
		}

		return false;
	}


	/*
	From the COFF spec: 5.2.1 Type Indicators
	*/
	inline bool IsDebugRelocation(const IMAGE_RELOCATION* relocation)
	{
#if LC_64_BIT
		return (relocation->Type == IMAGE_REL_AMD64_SECTION);
#else
		return (relocation->Type == IMAGE_REL_I386_SECTION);
#endif
	}


	/*
	From the COFF spec: 4.1 Section Flags
	*/
	inline bool IsDiscardableSection(const IMAGE_SECTION_HEADER* section)
	{
		return (section->Characteristics & IMAGE_SCN_MEM_DISCARDABLE) != 0u;
	}

	inline bool IsPartOfImage(const IMAGE_SECTION_HEADER* section)
	{
		// if the LNK_REMOVE flag is not set, the section becomes part of the final image
		return (section->Characteristics & IMAGE_SCN_LNK_REMOVE) == 0u;
	}

	inline bool IsComdatSection(const IMAGE_SECTION_HEADER* section)
	{
		return (section->Characteristics & IMAGE_SCN_LNK_COMDAT) != 0u;
	}

	inline bool IsDirectiveSection(const IMAGE_SECTION_HEADER* section)
	{
		// sections containing info/comments/directives are mostly named ".drectve", but are more
		// easily identified using IMAGE_SCN_LNK_INFO.
		return (section->Characteristics & IMAGE_SCN_LNK_INFO) != 0u;
	}

	inline bool IsCodeSection(uint32_t characteristics)
	{
		return (characteristics & IMAGE_SCN_MEM_EXECUTE) != 0u;
	}

	inline bool IsReadSection(uint32_t characteristics)
	{
		return (characteristics & IMAGE_SCN_MEM_READ) != 0u;
	}

	inline bool IsWriteSection(uint32_t characteristics)
	{
		return (characteristics & IMAGE_SCN_MEM_WRITE) != 0u;
	}


	template <typename HeaderType>
	inline uint32_t GetNumberOfSections(const void* imageBase)
	{
		const HeaderType* header = pointer::As<const HeaderType*>(imageBase);
		return header->NumberOfSections;
	}


	template <typename HeaderType>
	inline uint32_t GetNumberOfSymbols(const void* imageBase)
	{
		const HeaderType* header = pointer::As<const HeaderType*>(imageBase);
		return header->NumberOfSymbols;
	}


	template <typename HeaderType>
	inline const IMAGE_SECTION_HEADER* GetSectionHeader(const void* imageBase)
	{
		return pointer::Offset<const IMAGE_SECTION_HEADER*>(imageBase, sizeof(HeaderType));
	}


	template <typename HeaderType>
	inline const void* GetSymbolTable(const void* imageBase)
	{
		const HeaderType* header = pointer::As<const HeaderType*>(imageBase);
		return pointer::Offset<const void*>(imageBase, header->PointerToSymbolTable);
	}


	template <typename SymbolType>
	inline const char* GetStringTable(const void* symbolTable, uint32_t symbolCount)
	{
		// string table comes after the symbol table
		return pointer::Offset<const char*>(symbolTable, sizeof(SymbolType)*symbolCount);
	}


	template <typename SymbolType>
	inline const SymbolType* GetSymbol(const void* symbolTable, size_t index)
	{
		return pointer::Offset<const SymbolType*>(symbolTable, index * sizeof(SymbolType));
	}


	// archives/libraries
	inline uint32_t GetArchiveMemberSize(const IMAGE_ARCHIVE_MEMBER_HEADER* header)
	{
		// COFF Spec, 7.2. Archive Member Headers:
		//	Field "Size" at offset 48: An ASCII decimal representation of the total size of the archive member, not including the size of the header.
		uint32_t size = 0u;
		sscanf_s(reinterpret_cast<const char*>(header->Size), "%" SCNu32, &size);

		return size;
	}


	inline uint32_t PadArchiveMemberSize(uint32_t size)
	{
		// COFF Spec, 7.2. Archive Member Headers:
		//	"Each member header starts on the first even address after the end of the previous archive member."
		return size + (size & 1u);
	}
}


#endif