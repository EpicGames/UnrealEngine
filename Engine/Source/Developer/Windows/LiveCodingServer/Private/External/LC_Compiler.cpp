// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Compiler.h"
#include "LC_StringUtil.h"
#include "LC_Filesystem.h"
#include "LC_Process.h"
#include "LC_Environment.h"
#include "LC_CriticalSection.h"
#include "LC_TimeStamp.h"
#include "LC_MemoryMappedFile.h"
#include "LC_StringUtil.h"
#include "LC_Thread.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include "LC_AppSettings.h"
// END EPIC MOD

namespace
{
	// simple key-value cache for storing environment blocks for certain compilers
	class CompilerEnvironmentCache
	{
	public:
		CompilerEnvironmentCache(void)
			: m_cache(16u)
		{
		}

		~CompilerEnvironmentCache(void)
		{
			for (auto it = m_cache.begin(); it != m_cache.end(); ++it)
			{
				Process::Environment env = it->second;
				Process::DestroyEnvironment(env);
			}
		}

		void Insert(const wchar_t* key, const Process::Environment& value)
		{
	        // BEGIN EPIC MOD - Allow passing environment block for linker
			auto it = m_cache.find(key);
			if (it != m_cache.end())
			{
				Process::DestroyEnvironment(it->second);
				it->second = value;
				return;
			}
			// END EPIC MOD

			m_cache[key] = value;
		}

		Process::Environment Fetch(const wchar_t* key)
		{
			const auto it = m_cache.find(key);
			if (it != m_cache.end())
			{
				return it->second;
			}

			return Process::Environment { 0u, nullptr };
		}

	private:
		types::unordered_map<std::wstring, Process::Environment> m_cache;
	};

	static CompilerEnvironmentCache g_compilerEnvironmentCache;

	static CriticalSection g_compilerCacheCS;


	static std::vector<const wchar_t*> DetermineRelativePathToVcvarsFile(const wchar_t* absolutePathToCompilerExe)
	{
		// COMPILER SPECIFIC: Visual Studio. other compilers and linkers don't need vcvars*.bat to be invoked.
		std::vector<const wchar_t*> paths;
		paths.reserve(5u);

		// find out which vcvars*.bat file we have to call, based on the path to the compiler used.
		// make sure to carry out the comparison with lowercase strings only.
		wchar_t lowercaseAbsolutePathToCompilerExe[MAX_PATH] = {};
		wcscpy_s(lowercaseAbsolutePathToCompilerExe, absolutePathToCompilerExe);
		_wcslwr_s(lowercaseAbsolutePathToCompilerExe);

		// Visual Studio 2017 and above
		if (string::Contains(lowercaseAbsolutePathToCompilerExe, L"bin\\hostx86\\x86"))
		{
			paths.push_back(L"\\..\\..\\..\\..\\..\\..\\Auxiliary\\Build\\vcvars32.bat");
		}
		else if (string::Contains(lowercaseAbsolutePathToCompilerExe, L"bin\\hostx86\\x64"))
		{
			paths.push_back(L"\\..\\..\\..\\..\\..\\..\\Auxiliary\\Build\\vcvarsx86_amd64.bat");
		}
		else if (string::Contains(lowercaseAbsolutePathToCompilerExe, L"bin\\hostx64\\x64"))
		{
			paths.push_back(L"\\..\\..\\..\\..\\..\\..\\Auxiliary\\Build\\vcvars64.bat");
		}
		else if (string::Contains(lowercaseAbsolutePathToCompilerExe, L"bin\\hostx64\\x86"))
		{
			paths.push_back(L"\\..\\..\\..\\..\\..\\..\\Auxiliary\\Build\\vcvarsamd64_x86.bat");
		}

		// Visual Studio 2015 and below
		else if (string::Contains(lowercaseAbsolutePathToCompilerExe, L"vc\\bin\\amd64_x86"))
		{
			paths.push_back(L"\\vcvarsamd64_x86.bat");
		}
		else if (string::Contains(lowercaseAbsolutePathToCompilerExe, L"vc\\bin\\x86_amd64"))
		{
			paths.push_back(L"\\vcvarsx86_amd64.bat");
		}
		else if (string::Contains(lowercaseAbsolutePathToCompilerExe, L"vc\\bin\\amd64"))
		{
			paths.push_back(L"\\vcvars64.bat");
		}
		else if (string::Contains(lowercaseAbsolutePathToCompilerExe, L"vc\\bin"))
		{
			paths.push_back(L"\\vcvars32.bat");
		}

		// fallback for toolchains which are not installed at the default location.
		// in this case, we assume the vcvars*.bat file is in the same directory and try all different flavours later.
		else
		{
			paths.push_back(L"\\vcvars64.bat");
			paths.push_back(L"\\vcvarsamd64_x86.bat");
			paths.push_back(L"\\vcvarsx86_amd64.bat");
			paths.push_back(L"\\vcvars32.bat");
		}

		return paths;
	}
}


Process::Environment compiler::CreateEnvironmentCacheEntry(const wchar_t* absolutePathToCompilerExe)
{
	LC_LOG_DEV("Creating environment cache entry for %S", absolutePathToCompilerExe);

	// COMPILER SPECIFIC: Visual Studio. other compilers and linkers don't need vcvars*.bat to be invoked.
	{
		// bail out early in case this is the LLVM/clang/lld toolchain
		const Filesystem::Path toolFilename = Filesystem::GetFilename(absolutePathToCompilerExe);

		// Clang
		if (string::Matches(toolFilename.GetString(), L"clang.exe"))
		{
			return Process::Environment { 0u, nullptr };
		}
		else if (string::Matches(toolFilename.GetString(), L"clang++.exe"))
		{
			return Process::Environment { 0u, nullptr };
		}
		else if (string::Matches(toolFilename.GetString(), L"clang-cl.exe"))
		{
			return Process::Environment { 0u, nullptr };
		}
		else if (string::Matches(toolFilename.GetString(), L"clang-cpp.exe"))
		{
			return Process::Environment { 0u, nullptr };
		}
		else if (string::Matches(toolFilename.GetString(), L"verse-clang-cl.exe"))
		{
			return Process::Environment { 0u, nullptr };
		}

		// LLD
		else if (string::Matches(toolFilename.GetString(), L"lld.exe"))
		{
			return Process::Environment { 0u, nullptr };
		}
		else if (string::Matches(toolFilename.GetString(), L"lld-link.exe"))
		{
			return Process::Environment { 0u, nullptr };
		}
		else if (string::Matches(toolFilename.GetString(), L"ld.lld.exe"))
		{
			return Process::Environment { 0u, nullptr };
		}
		else if (string::Matches(toolFilename.GetString(), L"ld64.lld.exe"))
		{
			return Process::Environment { 0u, nullptr };
		}
	}

	const double timeout = static_cast<double>(appSettings::g_prewarmTimeout->GetValue()) / 1000.0;
	const std::wstring path = Filesystem::GetDirectory(absolutePathToCompilerExe).GetString();

	// get all possible paths to vcvars*.bat files and check which one is available
	const std::vector<const wchar_t*>& relativePathsToVcvarsFile = DetermineRelativePathToVcvarsFile(absolutePathToCompilerExe);
	for (size_t i = 0u; i < relativePathsToVcvarsFile.size(); ++i)
	{
		std::wstring pathToVcvars(path);
		pathToVcvars += relativePathsToVcvarsFile[i];

		LC_LOG_DEV("Trying vcvars*.bat at %S", pathToVcvars.c_str());

		const Filesystem::PathAttributes& attributes = Filesystem::GetAttributes(pathToVcvars.c_str());
		if (Filesystem::DoesExist(attributes))
		{
			// this is the correct vcvars*.bat.
			// we need to invoke the command shell, run the .bat file, and extract the process' environment to cache it for later use.
			// this is slightly more complicated than it needs to be, because we cannot simply run a command in the shell and grab
			// the environment without knowing if the .bat has finished running. similarly, we cannot grab the environment once
			// the shell process has terminated already.

			// tell cmd.exe to execute commands, and quote all filenames involved.
			// the whole command needs to be quoted as well.
			const std::wstring& cmdPath = environment::GetVariable(L"COMSPEC", L"cmd");
			std::wstring commandLine(L"/c \"call \"");
			commandLine += pathToVcvars;

			// set an environment variable with the exit code from the batch file.
			// we can retrieve this from the environment later and check if there was an error.
			commandLine += L"\" & call set LPP_TOOLCHAIN_EXIT_CODE=%^ERRORLEVEL% & call pause \"";

			Process::Context* vcvarsProcess = Process::Spawn(cmdPath.c_str(), nullptr, commandLine.c_str(), nullptr, Process::SpawnFlags::NO_WINDOW);

			// wait until LPP_TOOLCHAIN_EXIT_CODE shows up in the environment of the process.
			// busy waiting like this is not very nice, but happens only once or twice during startup, and is called from a separate thread anyway.
			const uint64_t startTimestamp = timeStamp::Get();
			bool shownWarning = false;

			Process::Environment environment = {};
			const wchar_t* toolchainExitCodeStr = nullptr;
			for (;;)
			{
				// grab the environment from the process
				Process::Suspend(vcvarsProcess);
				environment = Process::CreateEnvironment(vcvarsProcess);
				Process::Resume(vcvarsProcess);

				if (environment.data)
				{
					toolchainExitCodeStr = string::Find(static_cast<const wchar_t*>(environment.data), environment.size / sizeof(wchar_t), L"LPP_TOOLCHAIN_EXIT_CODE", wcslen(L"LPP_TOOLCHAIN_EXIT_CODE"));
					if (toolchainExitCodeStr)
					{
						const wchar_t* exitCodeStr = toolchainExitCodeStr + wcslen(L"LPP_TOOLCHAIN_EXIT_CODE") + 1u;
						if (*exitCodeStr != '%')
						{
							// the environment variable is available and set, so the batch file has finished running
							break;
						}
					}
				}

				// the batch file hasn't finished running yet, wait a bit
				Process::DestroyEnvironment(environment);
				Thread::Current::SleepMilliSeconds(20u);

				// show a warning in case this takes longer than 5 seconds.
				// this can happen for some users:
				// https://developercommunity.visualstudio.com/content/problem/51179/vsdevcmdbat-or-vcvarsallbat-excecution-takes-a-ver.html
				const uint64_t delta = timeStamp::Get() - startTimestamp;
				if ((timeStamp::ToSeconds(delta) >= 5.0) && (!shownWarning))
				{
					LC_WARNING_USER("Prewarming compiler/linker environment for %S is taking suspiciously long.", pathToVcvars.c_str());
					shownWarning = true;
				}

				// safety net: bail out if this takes longer than the configured timeout
				if (timeStamp::ToSeconds(delta) >= timeout)
				{
					LC_WARNING_USER("Prewarming compiler/linker environment for %S took too long and was aborted.", pathToVcvars.c_str());
					return Process::Environment { 0u, nullptr };
				}
			}

			// insert the environment into the cache
			{
				CriticalSection::ScopedLock lock(&g_compilerCacheCS);
				g_compilerEnvironmentCache.Insert(absolutePathToCompilerExe, environment);
			}

			// test the exit code of the process
			{
				const unsigned int toolchainExitCode = string::StringToInt<unsigned int>(toolchainExitCodeStr + wcslen(L"LPP_TOOLCHAIN_EXIT_CODE") + 1u);
				if (toolchainExitCode != 0u)
				{
					LC_WARNING_USER("Prewarming environment cache for %S failed with exit code %u", pathToVcvars.c_str(), toolchainExitCode);
				}
			}

			Process::Terminate(vcvarsProcess);
			Process::Destroy(vcvarsProcess);

			return environment;
		}
		else
		{
			LC_LOG_DEV("%S does not exist", pathToVcvars.c_str());
		}
	}

	LC_WARNING_USER("Cannot determine vcvars*.bat environment for compiler/linker %S", absolutePathToCompilerExe);
	return Process::Environment { 0u, nullptr };
}


Process::Environment compiler::GetEnvironmentFromCache(const wchar_t* absolutePathToCompilerExe)
{
	CriticalSection::ScopedLock lock(&g_compilerCacheCS);
	return g_compilerEnvironmentCache.Fetch(absolutePathToCompilerExe);
}


Process::Environment compiler::UpdateEnvironmentCache(const wchar_t* absolutePathToCompilerExe)
{
	Process::Environment environment = GetEnvironmentFromCache(absolutePathToCompilerExe);
	if (environment.data)
	{
		return environment;
	}

	return CreateEnvironmentCacheEntry(absolutePathToCompilerExe);
}


compiler::CompilerType::Enum compiler::DetermineCompilerType(const wchar_t* compilerPath)
{
	// MSVC's cl.exe is much more common, so treat this as our default
	const std::wstring lowerCaseCompilerPath = string::ToLower(Filesystem::GetFilename(compilerPath).GetString());
	if (string::Contains(lowerCaseCompilerPath.c_str(), L"clang"))
	{
		return CompilerType::CLANG;
	}
	else if (string::Contains(lowerCaseCompilerPath.c_str(), L"clang++"))
	{
		return CompilerType::CLANG;
	}
	else if (string::Contains(lowerCaseCompilerPath.c_str(), L"clang-cl"))
	{
		return CompilerType::CLANG;
	}
	else if (string::Contains(lowerCaseCompilerPath.c_str(), L"clang-cpp"))
	{
		return CompilerType::CLANG;
	}

	return CompilerType::CL;
}


compiler::LinkerType::Enum compiler::DetermineLinkerType(const wchar_t* linkerPath)
{
	// MSVC's link.exe is much more common, so treat this as our default
	const std::wstring lowerCaseLinkerPath = string::ToLower(Filesystem::GetFilename(linkerPath).GetString());
	if (string::Contains(lowerCaseLinkerPath.c_str(), L"lld"))
	{
		return LinkerType::LLD;
	}
	else if (string::Contains(lowerCaseLinkerPath.c_str(), L"lld-link"))
	{
		return LinkerType::LLD;
	}
	else if (string::Contains(lowerCaseLinkerPath.c_str(), L"ld.lld"))
	{
		return LinkerType::LLD;
	}
	else if (string::Contains(lowerCaseLinkerPath.c_str(), L"ld64.lld"))
	{
		return LinkerType::LLD;
	}

	return LinkerType::LINK;
}

// BEGIN EPIC MOD - Allow passing environment block for linker
void compiler::AddEnvironmentToCache(const wchar_t* absolutePathToCompilerExe, Process::Environment* environment)
{
	CriticalSection::ScopedLock lock(&g_compilerCacheCS);
	g_compilerEnvironmentCache.Insert(absolutePathToCompilerExe, *environment);
}
// END EPIC MOD

#endif