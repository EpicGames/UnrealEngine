// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ProcessTypes.h"


// TODO: rename to Toolchain or similar
namespace compiler
{
	// creates a new entry in the cache for the given compiler .exe, and returns it
	Process::Environment CreateEnvironmentCacheEntry(const wchar_t* absolutePathToCompilerExe);

	// gets the environment for a given compiler .exe from the cache.
	// returns nullptr if the environment is not yet in the cache.
	Process::Environment GetEnvironmentFromCache(const wchar_t* absolutePathToCompilerExe);

	// helper function that either creates a new entry in the cache if none exists yet,
	// or returns the one found in the cache.
	Process::Environment UpdateEnvironmentCache(const wchar_t* absolutePathToCompilerExe);

	// BEGIN EPIC MOD - Allow overriding environment block for tools
	void AddEnvironmentToCache(const wchar_t* absolutePathToCompilerExe, Process::Environment* environment);
	// END EPIC MOD

	struct CompilerType
	{
		enum Enum
		{
			CL,			// MSVC
			CLANG		// LLVM/Clang
		};
	};

	struct LinkerType
	{
		enum Enum
		{
			LINK,		// MSVC
			LLD			// LLVM/LLD
		};
	};

	CompilerType::Enum DetermineCompilerType(const wchar_t* compilerPath);

	LinkerType::Enum DetermineLinkerType(const wchar_t* linkerPath);
}


#endif