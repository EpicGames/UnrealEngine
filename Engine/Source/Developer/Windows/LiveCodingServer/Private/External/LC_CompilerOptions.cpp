// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_CompilerOptions.h"
#include "LC_StringUtil.h"


bool compilerOptions::CreatesPrecompiledHeader(const char* options)
{
	return string::Contains(options, "-Yc");
}


bool compilerOptions::UsesPrecompiledHeader(const char* options)
{
	return string::Contains(options, "-Yu");
}


std::string compilerOptions::GetPrecompiledHeaderPath(const char* options)
{
	const char* position = string::Find(options, "-Fp");
	if (position)
	{
		// skip "-Fp"
		const char* pathBegin = position + 3u;
		if (pathBegin[0] == '"')
		{
			// this is a quoted path
			const char* pathEnd = string::Find(pathBegin + 1u, "\"");
			return std::string(pathBegin + 1u, pathEnd);
		}
		else
		{
			// this is an unquoted path
			const char* pathEnd = string::Find(pathBegin, " ");
			return std::string(pathBegin, pathEnd);
		}
	}

	return std::string();
}


bool compilerOptions::UsesC7DebugFormat(const char* options)
{
	return string::Contains(options, "-Z7");
}


bool compilerOptions::UsesMinimalRebuild(const char* options)
{
	if (string::Contains(options, "-Gm-"))
	{
		return false;
	}

	return string::Contains(options, "-Gm");
}


#endif