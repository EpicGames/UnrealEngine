// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include <string>
// END EPIC MOD

namespace compilerOptions
{
	bool CreatesPrecompiledHeader(const char* options);
	bool UsesPrecompiledHeader(const char* options);
	std::string GetPrecompiledHeaderPath(const char* options);

	bool UsesC7DebugFormat(const char* options);
	bool UsesMinimalRebuild(const char* options);
}


#endif