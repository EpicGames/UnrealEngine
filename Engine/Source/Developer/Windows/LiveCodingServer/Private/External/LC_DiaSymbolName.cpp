// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_DiaSymbolName.h"


dia::SymbolName::SymbolName(BSTR str)
	: m_name(str)
{
}


dia::SymbolName::~SymbolName(void)
{
	if (m_name)
	{
		::SysFreeString(m_name);
	}
}


dia::SymbolName::SymbolName(SymbolName&& other)
	: m_name(other.m_name)
{
	other.m_name = nullptr;
}


#endif