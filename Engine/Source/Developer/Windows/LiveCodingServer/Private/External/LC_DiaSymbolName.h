// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
#include "Windows/WindowsHWrapper.h"
#include <dia2.h>
// END EPIC MOD

namespace dia
{
	// simple wrapper, does not allocate, does not deep copy
	class SymbolName
	{
	public:
		explicit SymbolName(BSTR str);
		~SymbolName(void);

		// symbol names can only be moved, never be copied
		SymbolName(SymbolName&& other);

		inline const wchar_t* GetString(void) const
		{
			return m_name;
		}

	private:
		BSTR m_name;

		LC_DISABLE_COPY(SymbolName);
		LC_DISABLE_ASSIGNMENT(SymbolName);
		LC_DISABLE_MOVE_ASSIGNMENT(SymbolName);
	};
}


#endif