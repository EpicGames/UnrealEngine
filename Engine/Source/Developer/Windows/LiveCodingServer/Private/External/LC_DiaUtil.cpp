// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_DiaUtil.h"


namespace detail
{
	template <typename T>
	static T* FindEnumerator(IDiaSession* session)
	{
		IDiaEnumTables* enumTables = nullptr;
		if (session->getEnumTables(&enumTables) != S_OK)
		{
			return nullptr;
		}

		T* enumerator = nullptr;
		IDiaTable* table = nullptr;
		ULONG fetchedCount = 0ul;
		while ((enumTables->Next(1, &table, &fetchedCount) == S_OK) && (fetchedCount == 1ul))
		{
			// there is only one table that matches the given IID, grab it
			const HRESULT hr = table->QueryInterface(__uuidof(T), reinterpret_cast<void**>(&enumerator));
			table->Release();

			if (hr == S_OK)
			{
				// found the table
				break;
			}
		}

		enumTables->Release();

		return enumerator;
	}


	template <typename T, typename S>
	static void FetchFromEnumerator(T* enumerator, types::vector<S*>* symbols)
	{
		symbols->reserve(32u);

		ULONG symbolsFetched = 0u;
		S* symbol = nullptr;
		while ((enumerator->Next(1u, &symbol, &symbolsFetched) == S_OK) && (symbolsFetched == 1u))
		{
			symbols->push_back(symbol);
		}
	}
}


dia::SymbolName dia::GetSymbolName(IDiaSymbol* symbol)
{
	BSTR str = {};
	const HRESULT hr = symbol->get_name(&str);
	if (hr != S_OK)
	{
		str = ::SysAllocString(L"");
	}

	return SymbolName(str);
}


dia::SymbolName dia::GetSymbolUndecoratedName(IDiaSymbol* symbol)
{
	BSTR str = {};
	const HRESULT hr = symbol->get_undecoratedName(&str);
	if (hr != S_OK)
	{
		str = ::SysAllocString(L"");
	}

	return SymbolName(str);
}


dia::SymbolName dia::GetSymbolLibraryName(IDiaSymbol* symbol)
{
	BSTR str = {};
	const HRESULT hr = symbol->get_libraryName(&str);
	if (hr != S_OK)
	{
		str = ::SysAllocString(L"");
	}

	return SymbolName(str);
}


dia::SymbolName dia::GetSymbolFilename(IDiaSourceFile* symbol)
{
	BSTR str = {};
	const HRESULT hr = symbol->get_fileName(&str);
	if (hr != S_OK)
	{
		str = ::SysAllocString(L"");
	}

	return SymbolName(str);
}


dia::Variant dia::GetSymbolEnvironmentOption(IDiaSymbol* environment)
{
	return Variant(environment);
}


uint32_t dia::GetSymbolRVA(IDiaSymbol* symbol)
{
	DWORD rva = 0u;
	symbol->get_relativeVirtualAddress(&rva);

	return rva;
}


uint32_t dia::GetSymbolSize(IDiaSymbol* symbol)
{
	ULONGLONG result = 0ull;
	symbol->get_length(&result);

	return static_cast<uint32_t>(result);
}


uint32_t dia::GetSymbolOffset(IDiaSymbol* symbol)
{
	LONG result = 0;
	symbol->get_offset(&result);

	return static_cast<uint32_t>(result);
}


bool dia::IsFunction(IDiaSymbol* symbol)
{
	// BEGIN EPIC MOD
	BOOL isFunction = Windows::FALSE;
	// END EPIC MOD
	symbol->get_function(&isFunction);

	// BEGIN EPIC MOD
	return (isFunction != Windows::FALSE);
	// END EPIC MOD
}


IDiaSymbol* dia::GetTypeSymbol(IDiaSymbol* symbol)
{
	IDiaSymbol* typeSymbol = nullptr;
	symbol->get_type(&typeSymbol);

	return typeSymbol;
}


IDiaSymbol* dia::GetParent(IDiaSymbol* symbol)
{
	IDiaSymbol* parent = nullptr;
	symbol->get_lexicalParent(&parent);

	return parent;
}


IDiaSymbol* dia::GetSymbolById(IDiaSession* session, uint32_t id)
{
	IDiaSymbol* symbol = nullptr;
	session->symbolById(id, &symbol);

	return symbol;
}


bool dia::WasCompiledWithLTCG(IDiaSymbol* compilandDetail)
{
	// BEGIN EPIC MOD
	BOOL isLTCG = Windows::FALSE;
	// END EPIC MOD
	compilandDetail->get_isLTCG(&isLTCG);

	return (isLTCG != 0);
}


bool dia::WasCompiledWithHotpatch(IDiaSymbol* compilandDetail)
{
	// BEGIN EPIC MOD
	BOOL isHotpatch = Windows::FALSE;
	// END EPIC MOD
	compilandDetail->get_isHotpatchable(&isHotpatch);

	return (isHotpatch != 0);
}


types::vector<IDiaSymbol*> dia::GatherChildSymbols(IDiaSymbol* parent, enum SymTagEnum symTag)
{
	types::vector<IDiaSymbol*> symbols;

	IDiaEnumSymbols* enumSymbols = nullptr;
	if (parent->findChildren(symTag, NULL, nsNone, &enumSymbols) == S_OK)
	{
		detail::FetchFromEnumerator(enumSymbols, &symbols);
		enumSymbols->Release();
	}

	return symbols;
}


types::vector<IDiaSourceFile*> dia::GatherCompilandFiles(IDiaSession* session, IDiaSymbol* compiland)
{
	types::vector<IDiaSourceFile*> symbols;

	IDiaEnumSourceFiles* enumFiles = nullptr;
	if (session->findFile(compiland, NULL, nsNone, &enumFiles) == S_OK)
	{
		detail::FetchFromEnumerator(enumFiles, &symbols);
		enumFiles->Release();
	}

	return symbols;
}


IDiaEnumSectionContribs* dia::FindSectionContributionsEnumerator(IDiaSession* session)
{
	return detail::FindEnumerator<IDiaEnumSectionContribs>(session);
}


IDiaSymbol* dia::FindSymbolByRVA(IDiaSession* session, uint32_t rva)
{
	IDiaSymbol* diaSymbol = nullptr;
	
	long displacement = 0;
	const HRESULT hr = session->findSymbolByRVAEx(rva, SymTagNull, &diaSymbol, &displacement);

	if ((hr == S_OK) && (displacement == 0))
	{
		// found an exact match
		return diaSymbol;
	}

	if (diaSymbol)
	{
		diaSymbol->Release();
	}

	return nullptr;
}


IDiaSymbol* dia::FindFunctionByRva(IDiaSession* session, uint32_t rva)
{
	IDiaSymbol* diaSymbol = nullptr;

	long displacement = 0;

	// look for functions first, this includes private/static functions.
	// if no symbol can be found, try public symbols next. this is needed to find symbols in stripped PDBs such as KernelBase.dll.
	// as a last resort, try finding ANY symbol.
	const enum SymTagEnum tagsToTry[3] = { SymTagFunction, SymTagPublicSymbol, SymTagNull };
	for (unsigned int i = 0u; i < 3u; ++i)
	{
		const HRESULT hr = session->findSymbolByRVAEx(rva, tagsToTry[i], &diaSymbol, &displacement);
		if (hr == S_OK)
		{
			return diaSymbol;
		}
	}

	return nullptr;
}


IDiaSymbol* dia::FindLabelByRva(IDiaSession* session, uint32_t rva)
{
	IDiaSymbol* diaSymbol = nullptr;

	long displacement = 0;
	const HRESULT hr = session->findSymbolByRVAEx(rva, SymTagLabel, &diaSymbol, &displacement);

	if ((hr == S_OK) && (displacement == 0))
	{
		// found an exact match
		return diaSymbol;
	}

	if (diaSymbol)
	{
		diaSymbol->Release();
	}

	return nullptr;
}


uint32_t dia::FindLineNumberByRVA(IDiaSession* session, uint32_t rva)
{
	IDiaEnumLineNumbers* enumLineNumbers = nullptr;

	// the longest instruction is 16 bytes, so there is no need to fetch lines for more than 16 instruction bytes
	const HRESULT hr = session->findLinesByRVA(rva, 16u, &enumLineNumbers);
	if ((hr == S_OK) && (enumLineNumbers))
	{
		IDiaLineNumber* line = nullptr;
		DWORD fetchedCount = 0u;
		if (SUCCEEDED(enumLineNumbers->Next(1, &line, &fetchedCount)) && (fetchedCount == 1))
		{
			DWORD lineNumber = 0u;
			line->get_lineNumber(&lineNumber);

			line->Release();
			enumLineNumbers->Release();

			return lineNumber;
		}

		enumLineNumbers->Release();
	}

	if (enumLineNumbers)
	{
		enumLineNumbers->Release();
	}

	return 0u;
}


dia::SymbolName dia::FindSourceFileByRVA(IDiaSession* session, uint32_t rva)
{
	IDiaEnumLineNumbers* enumLineNumbers = nullptr;

	// the longest instruction is 16 bytes, so there is no need to fetch lines for more than 16 instruction bytes
	const HRESULT hr = session->findLinesByRVA(rva, 16u, &enumLineNumbers);
	if ((hr == S_OK) && (enumLineNumbers))
	{
		IDiaLineNumber* line = nullptr;
		DWORD fetchedCount = 0u;
		if (SUCCEEDED(enumLineNumbers->Next(1, &line, &fetchedCount)) && (fetchedCount == 1))
		{
			IDiaSourceFile* sourceFile = nullptr;
			line->get_sourceFile(&sourceFile);

			if (sourceFile)
			{
				dia::SymbolName filename = dia::GetSymbolFilename(sourceFile);

				sourceFile->Release();
				line->Release();
				enumLineNumbers->Release();

				return filename;
			}

			line->Release();
		}

		enumLineNumbers->Release();
	}

	if (enumLineNumbers)
	{
		enumLineNumbers->Release();
	}

	return SymbolName(nullptr);
}


#endif