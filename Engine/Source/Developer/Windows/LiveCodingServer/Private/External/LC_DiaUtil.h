// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_DiaSymbolName.h"
#include "LC_DiaVariant.h"
// BEGIN EPIC MOD
#include "LC_Types.h"
// END EPIC MOD


namespace dia
{
	SymbolName GetSymbolName(IDiaSymbol* symbol);
	SymbolName GetSymbolUndecoratedName(IDiaSymbol* symbol);
	SymbolName GetSymbolLibraryName(IDiaSymbol* symbol);
	SymbolName GetSymbolFilename(IDiaSourceFile* symbol);

	Variant GetSymbolEnvironmentOption(IDiaSymbol* environment);

	uint32_t GetSymbolRVA(IDiaSymbol* symbol);
	uint32_t GetSymbolSize(IDiaSymbol* symbol);
	uint32_t GetSymbolOffset(IDiaSymbol* symbol);

	bool IsFunction(IDiaSymbol* symbol);

	IDiaSymbol* GetTypeSymbol(IDiaSymbol* symbol);
	IDiaSymbol* GetParent(IDiaSymbol* symbol);
	IDiaSymbol* GetSymbolById(IDiaSession* session, uint32_t id);

	bool WasCompiledWithLTCG(IDiaSymbol* compilandDetail);
	bool WasCompiledWithHotpatch(IDiaSymbol* compilandDetail);

	types::vector<IDiaSymbol*> GatherChildSymbols(IDiaSymbol* parent, enum SymTagEnum symTag);
	types::vector<IDiaSourceFile*> GatherCompilandFiles(IDiaSession* session, IDiaSymbol* compiland);

	IDiaEnumSectionContribs* FindSectionContributionsEnumerator(IDiaSession* session);
	IDiaSymbol* FindSymbolByRVA(IDiaSession* session, uint32_t rva);
	IDiaSymbol* FindFunctionByRva(IDiaSession* session, uint32_t rva);
	IDiaSymbol* FindLabelByRva(IDiaSession* session, uint32_t rva);

	uint32_t FindLineNumberByRVA(IDiaSession* session, uint32_t rva);
	SymbolName FindSourceFileByRVA(IDiaSession* session, uint32_t rva);
}


#endif