// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_DiaVariant.h"
// BEGIN EPIC MOD
#include <utility>
// END EPIC MOD

dia::Variant::Variant(IDiaSymbol* symbol)
// BEGIN EPIC MOD
	: m_str(nullptr)
// END EPIC MOD
{
	// BEGIN EPIC MOD
	m_var.vt = VT_EMPTY;
	// END EPIC MOD
	if (symbol->get_value(&m_var) == S_OK)
	{
		// the information we're interested in is always stored as string
		if (m_var.vt == VT_BSTR)
		{
			m_str = m_var.bstrVal;
		}
	}
}


dia::Variant::~Variant(void)
{
	if (m_str)
	{
		::VariantClear(&m_var);
	}
}


dia::Variant::Variant(Variant&& other)
	: m_var(std::move(other.m_var))
	, m_str(other.m_str)
{
	other.m_str = nullptr;
}


#endif