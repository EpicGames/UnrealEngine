// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
#include "Windows/WindowsHWrapper.h"
#include <dia2.h>
// END EPIC MOD

namespace dia
{
	// simple wrapper, does not allocate, does not deep copy
	class Variant
	{
	public:
		explicit Variant(IDiaSymbol* symbol);
		~Variant(void);

		// variants can only be moved, never be copied
		Variant(Variant&& other);

		inline const wchar_t* GetString(void) const
		{
			return m_str;
		}

	private:
		VARIANT m_var;
		BSTR m_str;

		LC_DISABLE_COPY(Variant);
		LC_DISABLE_ASSIGNMENT(Variant);
		LC_DISABLE_MOVE_ASSIGNMENT(Variant);
	};
}


#endif