// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_DirectoryCache.h"
// BEGIN EPIC MOD
#include "LC_Allocators.h"
// END EPIC MOD

DirectoryCache::DirectoryCache(size_t expectedDirectoryCount)
{
	m_directories.reserve(expectedDirectoryCount);
}


DirectoryCache::~DirectoryCache(void)
{
	for (auto it : m_directories)
	{
		Directory* dir = it.second;
		dir->changeNotification.Destroy();

		LC_FREE(&g_directoryAllocator, dir, sizeof(Directory));
	}
}


DirectoryCache::Directory* DirectoryCache::AddDirectory(const std::wstring& directory)
{
	const auto& insertPair = m_directories.emplace(directory, nullptr);
	Directory*& changeDirectory = insertPair.first->second;
	if (insertPair.second)
	{
		// insertion was successful, create a new directory
		changeDirectory = LC_NEW(&g_directoryAllocator, Directory);
		changeDirectory->changeNotification.Create(directory.c_str());
		changeDirectory->hadChange = false;
	}

	return changeDirectory;
}


void DirectoryCache::PrimeNotifications(void)
{
	for (auto it : m_directories)
	{
		Directory* dir = it.second;

		// don't throw away any changes that might have been set already
		dir->hadChange |= dir->changeNotification.Check(0u);
	}
}


void DirectoryCache::RestartNotifications(void)
{
	for (auto it : m_directories)
	{
		Directory* dir = it.second;
		dir->hadChange = false;
	}
}


#endif