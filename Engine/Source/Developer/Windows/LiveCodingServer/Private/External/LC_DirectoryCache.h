// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "LC_Types.h"
// END EPIC MOD
#include "LC_ChangeNotification.h"


class DirectoryCache
{
public:
	struct Directory
	{
		ChangeNotification changeNotification;
		bool hadChange;
	};

	explicit DirectoryCache(size_t expectedDirectoryCount);
	~DirectoryCache(void);

	Directory* AddDirectory(const std::wstring& directory);
	void PrimeNotifications(void);
	void RestartNotifications(void);

private:
	types::unordered_map<std::wstring, Directory*> m_directories;
};


#endif