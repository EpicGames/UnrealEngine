// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Disassembler.h"
#include "LC_Process.h"
#include "LC_PointerUtil.h"
#include "distorm.h"
#include "mnemonics.h"
// BEGIN EPIC MOD
#include "LC_Platform.h"
#include "LC_Logging.h"
// END EPIC MOD

namespace
{
	static _DInst DisassembleInstruction(const uint8_t* instructions, uint64_t address)
	{
		_CodeInfo codeInfo = {};
		codeInfo.code = instructions;
		codeInfo.codeLen = Disassembler::Stream::Instruction::MAXIMUM_LENGTH;
		codeInfo.codeOffset = address;

#if LC_64_BIT
		codeInfo.dt = Decode64Bits;
#else
		codeInfo.dt = Decode32Bits;
#endif

		_DInst result = {};
		unsigned int instructionCount = 0u;
		distorm_decompose(&codeInfo, &result, 1u, &instructionCount);

		if (instructionCount == 0u)
		{
			// something went horribly wrong
			LC_ERROR_DEV("Could not disassemble instruction at 0x%p", address);
			return result;
		}

		if (result.flags == FLAG_NOT_DECODABLE)
		{
			// the opcode could not be decoded
			LC_ERROR_DEV("Could not decode instruction at 0x%p", address);
			return result;
		}

		return result;
	}
}


uint8_t Disassembler::FindInstructionSize(Process::Handle processHandle, const void* address)
{
	uint8_t code[Stream::Instruction::MAXIMUM_LENGTH] = {};
	Process::ReadProcessMemory(processHandle, address, code, Stream::Instruction::MAXIMUM_LENGTH);

	const _DInst& instruction = DisassembleInstruction(code, pointer::AsInteger<uint64_t>(address));
	return instruction.size;
}


bool Disassembler::IsMOV(uint16_t opcode)
{
	return (opcode == I_MOV);
}


bool Disassembler::IsLEA(uint16_t opcode)
{
	return (opcode == I_LEA);
}


#endif