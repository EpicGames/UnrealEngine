// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ProcessTypes.h"


namespace Disassembler
{
	// Returns the size of the first instruction found at the given address.
	uint8_t FindInstructionSize(Process::Handle processHandle, const void* address);

	// Returns whether the given instruction opcode is a MOV of any kind.
	bool IsMOV(uint16_t opcode);

	// Returns whether the given instruction opcode is a LEA of any kind.
	bool IsLEA(uint16_t opcode);


	class Stream
	{
	public:
		struct Instruction
		{
			// BEGIN EPIC MOD - removed copy non-ascii character
			// From the Intel 64 and IA-32 Architectures Software Developer's Manual: The maximum length of an Intel 64 and IA-32 instruction remains 15 bytes.
			// END EPIC MOD
			static const unsigned int MAXIMUM_LENGTH = 15u;

			uint8_t machineCode[MAXIMUM_LENGTH];
			uint8_t size;
			uint64_t address;
			uint16_t opcode;
		};
	};
}


#endif