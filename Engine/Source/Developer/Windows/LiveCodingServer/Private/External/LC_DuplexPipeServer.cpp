// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_DuplexPipeServer.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
// END EPIC MOD

namespace
{
	static const DWORD PIPE_BUFFER_SIZE = 8192u;
}


bool DuplexPipeServer::Create(const wchar_t* name)
{
	m_pipe = ::CreateNamedPipe(
		name,
		PIPE_ACCESS_DUPLEX,
		PIPE_TYPE_MESSAGE | PIPE_READMODE_MESSAGE | PIPE_WAIT,
		PIPE_UNLIMITED_INSTANCES,
		PIPE_BUFFER_SIZE,
		PIPE_BUFFER_SIZE,
		0u,
		NULL);

	if (m_pipe == INVALID_HANDLE_VALUE)
	{
		const DWORD error = ::GetLastError();
		LC_ERROR_USER("Error 0x%X while trying to create named pipe", error);
		return false;
	}

	return true;
}


bool DuplexPipeServer::WaitForClient(void)
{
	const BOOL connected = ::ConnectNamedPipe(m_pipe, NULL);
	if (!connected)
	{
		// other process could not connect
		const DWORD error = ::GetLastError();

		// a client could have connected between the call to CreateNamedPipe and ConnectNamedPipe
		if (error != ERROR_PIPE_CONNECTED)
		{
			LC_ERROR_USER("Error 0x%X while waiting for client to connect to named pipe", error);
			return false;
		}
	}

	return true;
}


void DuplexPipeServer::Disconnect(void)
{
	::FlushFileBuffers(m_pipe);
	::DisconnectNamedPipe(m_pipe);
}


#endif