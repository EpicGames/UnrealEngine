// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_ExecutablePatcher.h"
#include "LC_Executable.h"
#include "LC_PointerUtil.h"
#include "LC_Process.h"
// BEGIN EPIC MOD
#include "LC_Assert.h"
#include "LC_Logging.h"
// END EPIC MOD

namespace
{
	// the DLL entry point is a C-function with three 4-byte parameters, which means that
	// we need to pop 12 bytes off the stack upon returning from the function, at least for x86 calling convention.
	// this can be done with a "RET imm16" instruction, which is encoded as "C2 0C 00".
	// additionally, the entry point has a BOOL return value, which means we must return
	// a value in eax/rax, which can be done with a simple "MOV" instruction.
	// in order to keep the injected code as small as possible, however, it is sufficient
	// to move a value into the lowest 8-bit of AL only - the value only needs to be NOT zero.

#if LC_64_BIT
	// the code to inject on x64 is:
	//		B0 01		mov al, 1
	//		C3			ret				different calling convention than x86
	static const uint8_t PATCH[ExecutablePatcher::INJECTED_CODE_SIZE] = { 0xB0, 0x01, 0xC3 };
#else
	// the code to inject on x86 is:
	//		B0 01		mov al, 1
	//		C2 0C 00	ret 0Ch			different calling convention than x64
	static const uint8_t PATCH[ExecutablePatcher::INJECTED_CODE_SIZE] = { 0xB0, 0x01, 0xC2, 0x0C, 0x00 };
#endif
}


ExecutablePatcher::ExecutablePatcher(executable::Image* image, executable::ImageSectionDB* imageSections)
{
	LC_ASSERT(image, "Invalid image.");

	const uint32_t entryPointRva = executable::GetEntryPointRva(image);
	const uint32_t entryPointFileOffset = executable::RvaToFileOffset(imageSections, entryPointRva);

	executable::ReadFromFileOffset(image, entryPointFileOffset, m_originalCode, INJECTED_CODE_SIZE);
}


ExecutablePatcher::ExecutablePatcher(const uint8_t* entryPointCode)
{
	memcpy(m_originalCode, entryPointCode, INJECTED_CODE_SIZE);
}


uint32_t ExecutablePatcher::DisableEntryPointInImage(executable::Image* image, executable::ImageSectionDB* imageSections)
{
	LC_ASSERT(image, "Invalid image.");

	const uint32_t entryPointRva = executable::GetEntryPointRva(image);
	const uint32_t entryPointFileOffset = executable::RvaToFileOffset(imageSections, entryPointRva);

	executable::WriteToFileOffset(image, entryPointFileOffset, PATCH, INJECTED_CODE_SIZE);

	return entryPointRva;
}


void ExecutablePatcher::DisableEntryPoint(Process::Handle processHandle, void* moduleBase, uint32_t entryPointRva)
{
	for (size_t i = 0u; i < INJECTED_CODE_SIZE; ++i)
	{
		uint8_t* address = pointer::Offset<uint8_t*>(moduleBase, entryPointRva + i);
		Process::WriteProcessMemory(processHandle, address, PATCH[i]);
	}
}


void ExecutablePatcher::RestoreEntryPoint(Process::Handle processHandle, void* moduleBase, uint32_t entryPointRva)
{
	for (size_t i = 0u; i < INJECTED_CODE_SIZE; ++i)
	{
		uint8_t* address = pointer::Offset<uint8_t*>(moduleBase, entryPointRva + i);
		Process::WriteProcessMemory(processHandle, address, m_originalCode[i]);
	}
}


#endif