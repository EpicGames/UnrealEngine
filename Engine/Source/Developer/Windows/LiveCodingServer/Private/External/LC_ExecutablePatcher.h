// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Executable.h"
#include "LC_ProcessTypes.h"


class ExecutablePatcher
{
public:
#if LC_64_BIT
	static const size_t INJECTED_CODE_SIZE = 3u;
#else
	static const size_t INJECTED_CODE_SIZE = 5u;
#endif

	// reads the original entry point code from the executable
	ExecutablePatcher(executable::Image* image, executable::ImageSectionDB* imageSections);

	// uses the given entry point code 
	explicit ExecutablePatcher(const uint8_t* entryPointCode);


	// disables the entry point directly in the image file, and returns the RVA of the entry point
	uint32_t DisableEntryPointInImage(executable::Image* image, executable::ImageSectionDB* imageSections);

	// disables the entry point in memory
	void DisableEntryPoint(Process::Handle processHandle, void* moduleBase, uint32_t entryPointRva);

	// restores the entry point of a loaded image that previously had its entry point disabled
	void RestoreEntryPoint(Process::Handle processHandle, void* moduleBase, uint32_t entryPointRva);


	inline const uint8_t* GetEntryPointCode(void) const
	{
		return m_originalCode;
	}

private:
	uint8_t m_originalCode[INJECTED_CODE_SIZE] = {};
};


#endif