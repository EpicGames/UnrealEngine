// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_FileAttributeCache.h"
#include "LC_Filesystem.h"


FileAttributeCache::FileAttributeCache(void)
{
	// make space for 128k entries
	m_data.reserve(128u * 1024u);
}


FileAttributeCache::Data FileAttributeCache::UpdateCacheData(const std::wstring& path)
{
	// try to insert the element into the cache. if it exists, return the cached data.
	// if it doesn't exist, get the file attributes once and store them.
	const std::pair<Cache::iterator, bool>& optional = m_data.emplace(path, Data{});

	Data& data = optional.first->second;
	if (optional.second)
	{
		// value was inserted, update it with the correct data
		const Filesystem::PathAttributes& attributes = Filesystem::GetAttributes(path.c_str());
		data.exists = Filesystem::DoesExist(attributes);
		data.lastModificationTime = Filesystem::GetLastModificationTime(attributes);
	}

	return data;
}


size_t FileAttributeCache::GetEntryCount(void) const
{
	return m_data.size();
}


#endif