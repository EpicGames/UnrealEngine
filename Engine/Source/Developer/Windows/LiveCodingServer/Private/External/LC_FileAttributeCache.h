// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "LC_Types.h"
// END EPIC MOD
#include "LC_Hashing.h"


class FileAttributeCache
{
	struct Hasher
	{
		inline size_t operator()(const std::wstring& key) const
		{
			return Hashing::Hash32(key.c_str(), key.length() * sizeof(wchar_t), 0u);
		}
	};

public:
	struct Data
	{
		uint64_t lastModificationTime;
		bool exists;
	};

	FileAttributeCache(void);
	Data UpdateCacheData(const std::wstring& path);

	size_t GetEntryCount(void) const;

private:
	typedef types::unordered_map_with_hash<std::wstring, Data, Hasher> Cache;
	Cache m_data;
};


#endif