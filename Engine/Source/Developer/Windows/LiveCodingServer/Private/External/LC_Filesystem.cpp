// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Filesystem.h"
#include "LC_CriticalSection.h"
#include "LC_Hashing.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
// END EPIC MOD

// BEGIN EPIC MOD
#include "Windows/AllowWindowsPlatformTypes.h"
#include <shlwapi.h>
#include "Windows/HideWindowsPlatformTypes.h"
// If PCH files are disabled, this define will cause problems with UnrealString.h
#ifdef PathAppend
#undef PathAppend
#endif
// END EPIC MOD

namespace detail
{
	class NormalizedFilenameCache
	{
		struct Hasher
		{
			inline size_t operator()(const std::wstring& key) const
			{
				return Hashing::Hash32(key.c_str(), key.length() * sizeof(wchar_t), 0u);
			}
		};

	public:
		NormalizedFilenameCache(void)
			: m_data()
			, m_cs()
		{
			// make space for 128k entries
			m_data.reserve(128u * 1024u);
		}

		Filesystem::Path UpdateCacheData(const wchar_t* path)
		{
			CriticalSection::ScopedLock lock(&m_cs);

			// try to insert the element into the cache. if it exists, return the cached data.
			// if it doesn't exist, get the file name once and store it.
			const std::pair<typename Cache::iterator, bool>& optional = m_data.emplace(std::wstring(path), std::wstring());
			std::wstring& data = optional.first->second;

			if (optional.second)
			{
				// value was inserted, update it with the correct data
				// 
				// BEGIN EPIC MOD
				Filesystem::Path temp;
				path = Filesystem::Devirtualize(path, temp);
				// END EPIC MOD

				HANDLE file = ::CreateFileW(path, FILE_READ_ATTRIBUTES | STANDARD_RIGHTS_READ, FILE_SHARE_READ, 0, OPEN_EXISTING, FILE_FLAG_BACKUP_SEMANTICS, 0);
				if (file != INVALID_HANDLE_VALUE)
				{
					wchar_t buffer[Filesystem::Path::CAPACITY] = {};
					::GetFinalPathNameByHandleW(file, buffer, Filesystem::Path::CAPACITY, 0u);
					::CloseHandle(file);
					// TODO: Revirtualize?

					// the path returned by GetFinalPathNameByHandle starts with "\\?\", cut that off
					data.assign(buffer + 4u);
				}
				else
				{
					data.assign(path);
				}
			}

			return Filesystem::Path(data.c_str());
		}

	private:
		typedef types::unordered_map_with_hash<std::wstring, std::wstring, Hasher> Cache;
		Cache m_data;
		CriticalSection m_cs;
	};
}


namespace
{
	static detail::NormalizedFilenameCache* g_normalizedFilenameCache = nullptr;

	static Filesystem::DriveType::Enum g_driveTypeCache['z' - 'a' + 1u] = {};
}


void Filesystem::Startup(void)
{
	g_normalizedFilenameCache = new detail::NormalizedFilenameCache;

	// fill cache of drive types
	char root[4u] = { 'a', ':', '\\', '\0' };
	for (char drive = 'a'; drive <= 'z'; ++drive)
	{
		const int index = drive - 'a';
		
		root[0] = drive;
		const UINT driveType = ::GetDriveTypeA(root);

		switch (driveType)
		{
			case DRIVE_UNKNOWN:
				g_driveTypeCache[index] = DriveType::UNKNOWN;
				break;

			case DRIVE_NO_ROOT_DIR:
				g_driveTypeCache[index] = DriveType::UNKNOWN;
				break;

			case DRIVE_REMOVABLE:
				g_driveTypeCache[index] = DriveType::REMOVABLE;
				break;

			case DRIVE_FIXED:
				g_driveTypeCache[index] = DriveType::FIXED;
				break;

			case DRIVE_REMOTE:
				g_driveTypeCache[index] = DriveType::REMOTE;
				break;

			case DRIVE_CDROM:
				g_driveTypeCache[index] = DriveType::OPTICAL;
				break;

			case DRIVE_RAMDISK:
				g_driveTypeCache[index] = DriveType::RAMDISK;
				break;

			default:
				g_driveTypeCache[index] = DriveType::UNKNOWN;
				break;
		}
	}
}


void Filesystem::Shutdown(void)
{
	delete g_normalizedFilenameCache;
}


Filesystem::DriveType::Enum Filesystem::GetDriveType(const wchar_t* path)
{
	const wchar_t driveLetter = path[0];
	if ((driveLetter >= L'a') && (driveLetter <= L'z'))
	{
		return g_driveTypeCache[driveLetter - L'a'];
	}
	else if ((driveLetter >= L'A') && (driveLetter <= L'Z'))
	{
		return g_driveTypeCache[driveLetter - L'A'];
	}

	return DriveType::UNKNOWN;
}


Filesystem::PathAttributes Filesystem::GetAttributes(const wchar_t* path)
{
	// BEGIN EPIC MOD
	Path temp;
	path = Devirtualize(path, temp);
	// END EPIC MOD

	::WIN32_FILE_ATTRIBUTE_DATA attributes = {};
	attributes.dwFileAttributes = INVALID_FILE_ATTRIBUTES;

	::GetFileAttributesExW(path, GetFileExInfoStandard, &attributes);

	::ULARGE_INTEGER fileSize = {};
	fileSize.LowPart = attributes.nFileSizeLow;
	fileSize.HighPart = attributes.nFileSizeHigh;

	::ULARGE_INTEGER lastModificationTime = {};
	lastModificationTime.LowPart = attributes.ftLastWriteTime.dwLowDateTime;
	lastModificationTime.HighPart = attributes.ftLastWriteTime.dwHighDateTime;

	return PathAttributes { fileSize.QuadPart, lastModificationTime.QuadPart, attributes.dwFileAttributes };
}


uint64_t Filesystem::GetSize(const PathAttributes& attributes)
{
	return attributes.size;
}


uint64_t Filesystem::GetLastModificationTime(const PathAttributes& attributes)
{
	return attributes.lastModificationTime;
}


bool Filesystem::DoesExist(const PathAttributes& attributes)
{
	return (attributes.flags != INVALID_FILE_ATTRIBUTES);
}


bool Filesystem::IsDirectory(const PathAttributes& attributes)
{
	if (!DoesExist(attributes))
	{
		return false;
	}

	return (attributes.flags & FILE_ATTRIBUTE_DIRECTORY);
}


bool Filesystem::IsRelativePath(const wchar_t* path)
{
	// empty paths are not considered to be relative
	if (path[0] == L'\0')
	{
		return false;
	}

	// BEGIN EPIC MOD
	return (::PathIsRelativeW(path) != Windows::FALSE);
	// END EPIC MOD
}


void Filesystem::Copy(const wchar_t* srcPath, const wchar_t* destPath)
{
	// BEGIN EPIC MOD
	Path srcTemp;
	srcPath = Devirtualize(srcPath, srcTemp);
	Path destTemp;
	destPath = Devirtualize(destPath, destTemp);
	const BOOL success = ::CopyFileW(srcPath, destPath, Windows::FALSE);
	if (success == Windows::FALSE)
	// END EPIC MOD
	{
		LC_ERROR_USER("Failed to copy file from %S to %S. Error: 0x%X", srcPath, destPath, ::GetLastError());
	}
}


void Filesystem::Move(const wchar_t* srcPath, const wchar_t* destPath)
{
	// BEGIN EPIC MOD
	Path srcTemp;
	srcPath = Devirtualize(srcPath, srcTemp);
	Path destTemp;
	destPath = Devirtualize(destPath, destTemp);
	// END EPIC MOD

	const BOOL success = ::MoveFileExW(srcPath, destPath, MOVEFILE_COPY_ALLOWED | MOVEFILE_REPLACE_EXISTING | MOVEFILE_WRITE_THROUGH);
	// BEGIN EPIC MOD
	if (success == Windows::FALSE)
	// END EPIC MOD
	{
		LC_ERROR_USER("Failed to move file from %S to %S. Error: 0x%X", srcPath, destPath, ::GetLastError());
	}
}


void Filesystem::Delete(const wchar_t* path)
{
	// BEGIN EPIC MOD
	Path temp;
	path = Devirtualize(path, temp);
	// END EPIC MOD

	const BOOL success = ::DeleteFileW(path);
	// BEGIN EPIC MOD
	if (success == Windows::FALSE)
	// END EPIC MOD
	{
		LC_ERROR_USER("Failed to delete file %S. Error: 0x%X", path, ::GetLastError());
	}
}


bool Filesystem::DeleteIfExists(const wchar_t* path)
{
	// BEGIN EPIC MOD
	Path temp;
	path = Devirtualize(path, temp);
	// END EPIC MOD

	const BOOL success = ::DeleteFileW(path);
	// BEGIN EPIC MOD
	return (success != Windows::FALSE);
	// END EPIC MOD
}


Filesystem::Path Filesystem::GenerateTempFilename(void)
{
	wchar_t path[Filesystem::Path::CAPACITY] = {};
	::GetTempPathW(Filesystem::Path::CAPACITY, path);

	wchar_t filename[Filesystem::Path::CAPACITY] = {};
	wchar_t prefix[1] = { '\0' };
	::GetTempFileNameW(path, prefix, 0u, filename);

	return Path(filename);
}


Filesystem::Path Filesystem::GetDirectory(const wchar_t* path)
{
	unsigned int lastFoundIndex = 0u;
	unsigned int index = 0u;
	for (/*nothing*/; path[index] != '\0'; ++index)
	{
		if (path[index] == '\\')
		{
			lastFoundIndex = index;
		}
	}

	if (lastFoundIndex == 0u)
	{
		// no directory found, return original path
		return Path(path);
	}

	return Path(path, lastFoundIndex);
}


Filesystem::Path Filesystem::GetFilename(const wchar_t* path)
{
	unsigned int lastFoundIndex = 0u;
	unsigned int index = 0u;
	for (/*nothing*/; path[index] != '\0'; ++index)
	{
		if (path[index] == '\\')
		{
			lastFoundIndex = index;
		}
	}

	if (lastFoundIndex == 0u)
	{
		// no filename found, return original path
		return Path(path);
	}

	return Path(path + lastFoundIndex + 1u, index - lastFoundIndex - 1u);
}


Filesystem::Path Filesystem::GetExtension(const wchar_t* path)
{
	unsigned int lastFoundIndex = 0u;
	unsigned int index = 0u;
	for (/*nothing*/; path[index] != '\0'; ++index)
	{
		if (path[index] == '.')
		{
			lastFoundIndex = index;
		}
	}

	if (lastFoundIndex == 0u)
	{
		// no extension found, return empty path
		return Path();
	}

	return Path(path + lastFoundIndex, index - lastFoundIndex);
}


Filesystem::Path Filesystem::RemoveExtension(const wchar_t* path)
{
	unsigned int lastFoundIndex = 0u;
	unsigned int index = 0u;
	for (/*nothing*/; path[index] != '\0'; ++index)
	{
		if (path[index] == '.')
		{
			lastFoundIndex = index;
		}
	}

	if (lastFoundIndex == 0u)
	{
		// no extension found, return original path
		return Path(path);
	}

	return Path(path, lastFoundIndex);
}


Filesystem::Path Filesystem::NormalizePath(const wchar_t* path)
{
	// normalizing files is really costly on Windows, so we cache results
	return g_normalizedFilenameCache->UpdateCacheData(path);
}


Filesystem::Path Filesystem::NormalizePathWithoutResolvingLinks(const wchar_t* path)
{
	// use the old trick of converting to short and to long path names to get a path with correct casing
	wchar_t shortPath[Filesystem::Path::CAPACITY] = {};
	{
		const DWORD charsWritten = ::GetShortPathNameW(path, shortPath, Filesystem::Path::CAPACITY);
		if (charsWritten == 0u)
		{
			return Path(path);
		}
	}

	wchar_t longPath[Filesystem::Path::CAPACITY] = {};
	{
		const DWORD charsWritten = ::GetLongPathNameW(shortPath, longPath, Filesystem::Path::CAPACITY);
		if (charsWritten == 0u)
		{
			return Path(path);
		}
	}

	return Path(longPath);
}


bool Filesystem::CreateFileWithData(const wchar_t* path, const void* data, size_t size)
{
	// BEGIN EPIC MOD
	Path temp;
	path = Devirtualize(path, temp);
	// END EPIC MOD

	HANDLE file = ::CreateFileW(path, GENERIC_READ | GENERIC_WRITE, 0, NULL, CREATE_ALWAYS, FILE_ATTRIBUTE_NORMAL, NULL);
	if (file == INVALID_HANDLE_VALUE)
	{
		LC_ERROR_USER("Cannot open file %S for writing. Error: 0x%X", path, ::GetLastError());
		return false;
	}

	DWORD bytesWritten = 0u;
	::WriteFile(file, data, static_cast<DWORD>(size), &bytesWritten, NULL);
	::CloseHandle(file);

	return true;
}


types::vector<Filesystem::Path> Filesystem::EnumerateFiles(const wchar_t* directory)
{
	types::vector<Path> foundFiles;
	foundFiles.reserve(1024u);

	HANDLE fileHandle = INVALID_HANDLE_VALUE;
	WIN32_FIND_DATAW findData = {};

	types::vector<Path> directories;
	directories.reserve(1024u);
	directories.push_back(Path());

	Path findPath;
	Path foundPath;

	while (!directories.empty())
	{
		Path subDirectoryOnly = directories.back();
		directories.pop_back();

		findPath = directory;
		findPath += L"\\";
		findPath += subDirectoryOnly;
		findPath += L"*";

		fileHandle = ::FindFirstFileExW(findPath.GetString(), FindExInfoBasic, &findData, FindExSearchNameMatch, NULL, FIND_FIRST_EX_LARGE_FETCH);
		if (fileHandle == INVALID_HANDLE_VALUE)
		{
			return foundFiles;
		}

		do
		{
			// ignore "." and ".."
			const bool isDot = (findData.cFileName[0] == L'.') && (findData.cFileName[1] == L'\0');
			const bool isDotDot = (findData.cFileName[0] == L'.') && (findData.cFileName[1] == L'.') && (findData.cFileName[2] == L'\0');
			if ((!isDot) && (!isDotDot))
			{
				foundPath = subDirectoryOnly;
				foundPath += findData.cFileName;

				if (findData.dwFileAttributes & FILE_ATTRIBUTE_DIRECTORY)
				{
					foundPath += L"\\";

					directories.push_back(foundPath);
				}
				else
				{
					foundFiles.push_back(foundPath);
				}
			}
		}
		while (::FindNextFile(fileHandle, &findData) != 0);

		const DWORD lastError = GetLastError();
		if (lastError != ERROR_NO_MORE_FILES)
		{
			LC_ERROR_USER("Could not enumerate files in directory %S. Error: %d", directory, lastError);

			::FindClose(fileHandle);
			return foundFiles;
		}

		::FindClose(fileHandle);
	}

	return foundFiles;
}


#endif