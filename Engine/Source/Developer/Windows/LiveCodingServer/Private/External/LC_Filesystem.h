// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Vfs.h"
// BEGIN EPIC MOD
#include "LC_Types.h"
// END EPIC MOD

namespace Filesystem
{
	void Startup(void);
	void Shutdown(void);

	// Returns the type of a drive. Drive letters can be both lower-case or upper-case.
	DriveType::Enum GetDriveType(const wchar_t* path);

	// Retrieves a path's attributes. The given path can point to any file or directory in the file system.
	PathAttributes GetAttributes(const wchar_t* path);

	// Returns the size of the file or directory that corresponds to the given attributes.
	uint64_t GetSize(const PathAttributes& attributes);

	// Returns the last modification time for any path's attributes.
	uint64_t GetLastModificationTime(const PathAttributes& attributes);

	// Returns whether a path exists.
	bool DoesExist(const PathAttributes& attributes);

	// Returns whether given attributes correspond to a directory.
	bool IsDirectory(const PathAttributes& attributes);

	// Returns whether the given path is a relative path.
	bool IsRelativePath(const wchar_t* path);

	// Copies a file from source to destination.
	void Copy(const wchar_t* srcPath, const wchar_t* destPath);

	// Moves a file from source to destination.
	void Move(const wchar_t* currentPath, const wchar_t* movedToPath);

	// Deletes a file, reports an error if the file cannot be deleted.
	void Delete(const wchar_t* path);

	// Deletes a file only if it exists, without reporting an error.
	bool DeleteIfExists(const wchar_t* path);


	// Creates a unique, temporary absolute filename, e.g. C:\Users\JohnDoe\AppData\Local\Temp\ABCD.tmp
	Path GenerateTempFilename(void);

	// Returns the directory part of a given path, e.g. GetDirectory("C:\Directory\File.txt") returns "C:\Directory".
	Path GetDirectory(const wchar_t* path);

	// Returns the file part of a given path, e.g. GetFilename("C:\Directory\File.txt") returns "File.txt".
	Path GetFilename(const wchar_t* path);

	// Returns the extension part of a given path, e.g. GetExtension("C:\Directory\File.txt") returns ".txt".
	Path GetExtension(const wchar_t* path);

	// Returns the given path without any file extensions, e.g. RemoveExtension("C:\Directory\File.internal.txt") returns "C:\Directory\File".
	Path RemoveExtension(const wchar_t* path);

	// Canonicalizes/normalizes any given path.
	Path NormalizePath(const wchar_t* path);

	// Canonicalizes/normalizes any given path without resolving any symbolic links/virtual drives.
	// NOTE: This is not cached internally and should only be used for cosmetic purposes.
	Path NormalizePathWithoutResolvingLinks(const wchar_t* path);


	// Creates a file, storing the given data.
	bool CreateFileWithData(const wchar_t* path, const void* data, size_t size);

	// Recursively enumerates all files in a directory.
	// Returns an array of filenames relative to the given directory, e.g. file1.txt, file2.txt, dir\file.txt, dir\subDir\file.txt.
	types::vector<Path> EnumerateFiles(const wchar_t* directory);
}


#endif