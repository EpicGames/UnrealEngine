// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_FunctionPatcher.h"
#include "LC_Patch.h"
#include "LC_Disassembler.h"
#include "LC_NameMangling.h"
#include "LC_AppSettings.h"
#include "LC_Process.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
// END EPIC MOD


namespace
{
	struct PatchTechnique
	{
		enum Enum
		{
			DIRECT_RELATIVE_JUMP,
			HOTPATCH_INDIRECTION
		};
	};


	static const uint8_t INT3_PADDING[16u] = { 0xCC, 0xCC, 0xCC, 0xCC, 0xCC, 0xCC, 0xCC, 0xCC, 0xCC, 0xCC, 0xCC, 0xCC, 0xCC, 0xCC, 0xCC, 0xCC };


	// helper function that checks whether a certain number of bytes in memory are available for patching
	template <size_t N>
	static inline bool AreBytesAvailableForPatching(Process::Handle processHandle, const void* address)
	{
		// available bytes are either 0xCC (int 3) or 0x0 (page padding)
		uint8_t memory[N] = {};
		Process::ReadProcessMemory(processHandle, address, memory, N);

		for (size_t i = 0u; i < N; ++i)
		{
			if ((memory[i] != 0xCC) && (memory[i] != 0x0))
			{
				return false;
			}
		}

		return true;
	}
}


functions::Record functions::PatchFunction
(
	char* originalAddress,
	char* patchAddress,
	uint32_t functionRva,
	uint32_t patchFunctionRva,
	const symbols::ThunkDB* thunkDb,
	const symbols::Contribution* contribution,
	Process::Handle processHandle,
	void* moduleBase,
	uint16_t moduleIndex,
	types::unordered_set<const void*>& patchedAddresses,
	const types::vector<const void*>& threadIPs,

	// debug only
	Process::Id processId,
	const char* functionName
)
{
	Record record = { thunkDb, functionRva, patchFunctionRva, moduleIndex, 0u };

	// there are three ways to patch the old function so that it redirects to the new one:
	// 1) patch the incremental linking table to point to the new function. this is easiest because it
	//    doesn't need any disassembling of instructions or instruction pointer checks.
	// 2) install a relative jump to the new function at the start of the old function.
	//    this relative jump needs 5 bytes and can only be installed if no thread of this process is currently
	//    reading instructions from the location in question.
	// 3) install a relative jump to the new function 5 bytes in front of the old function.
	//    install a short jump to this relative jump at the start of the old function.
	//    this needs support from the compiler (/hotpatch) to ensure that the first instruction of a
	//    function is always at least 2 bytes long, and support from the linker (/FUNCTIONPADMIN) to ensure
	//    that there are 5 unused bytes in front of each function.
	unsigned int patchTechniqueUsed = PatchTechnique::HOTPATCH_INDIRECTION;
	size_t wholeInstructionSize = 0u;

	// first check whether we can find an incremental linking thunk for this function.
	bool installedPatchToILT = false;
	const types::vector<uint32_t>& thunkTableEntries = symbols::FindThunkTableEntriesByRVA(thunkDb, functionRva);
	if (thunkTableEntries.size() != 0u)
	{
		// patch the ILTs directly, but keep installing patches into the real function too.
		// this acts as a safety net, should any relocation or any function ever point to the real function
		// instead of the ILT.
		for (auto thunkRva : thunkTableEntries)
		{
			LC_LOG_DEV("Patching ILT 0x%X of function %s at 0x%p (0x%X) (PID: %d)", thunkRva, functionName, moduleBase, functionRva, +processId);

			void* incrementalLinkingThunk = pointer::Offset<char*>(moduleBase, thunkRva);
			patch::InstallRelativeNearJump(processHandle, incrementalLinkingThunk, patchAddress);
		}

		installedPatchToILT = true;
	}

	// second check whether the function is at least 5 bytes long to consider it for direct patching using
	// a single relative jump.
	if (contribution && contribution->size >= 5u)
	{
		// the function seems to be long enough.
		// disassemble the first instructions to see how many bytes we can patch.
		while (wholeInstructionSize < 5u)
		{
			const uint8_t instructionSize = Disassembler::FindInstructionSize(processHandle, originalAddress + wholeInstructionSize);
			if (instructionSize == 0u)
			{
				// dump raw code in case it could not be decoded
				LC_ERROR_DEV("Failed to disassemble code for function %s at 0x%p (0x%X) (PID: %d)", functionName, moduleBase, functionRva, +processId);
				Process::DumpMemory(processHandle, originalAddress, contribution->size);
				break;
			}

			wholeInstructionSize += instructionSize;
		}

		// did we disassemble at least 5 bytes worth of instructions?
		if (wholeInstructionSize >= 5u)
		{
			record.directJumpInstructionSize = static_cast<uint8_t>(wholeInstructionSize);

			// yes, now check if a thread is currently reading from memory at the location where we
			// want to install a relative jump.
			bool anyThreadInside = false;
			const size_t threadCount = threadIPs.size();
			for (size_t t = 0u; t < threadCount; ++t)
			{
				const void* instructionPointer = threadIPs[t];
				if ((instructionPointer >= originalAddress) && (instructionPointer < originalAddress + wholeInstructionSize))
				{
					anyThreadInside = true;
					break;
				}
			}

			if (!anyThreadInside)
			{
				// no thread currently reads from there, install a direct relative jump
				patchTechniqueUsed = PatchTechnique::DIRECT_RELATIVE_JUMP;
			}
		}
	}

	// now install a patch using the selected technique
	if (patchTechniqueUsed == PatchTechnique::DIRECT_RELATIVE_JUMP)
	{
		LC_LOG_DEV("Patching function %s directly at 0x%p (0x%X) (PID: %d)", functionName, moduleBase, functionRva, +processId);

		patch::InstallRelativeNearJump(processHandle, originalAddress, patchAddress);

		// are there remaining bytes straddling the last instruction?
		if (wholeInstructionSize > 5u)
		{
			// yes, overwrite those with int 3
			Process::WriteProcessMemory(processHandle, originalAddress + 5u, INT3_PADDING, wholeInstructionSize - 5u);
		}
	}
	else if (patchTechniqueUsed == PatchTechnique::HOTPATCH_INDIRECTION)
	{
		const uint8_t instructionSize = Disassembler::FindInstructionSize(processHandle, originalAddress);
		if (instructionSize >= 2u)
		{
			// we need to go via an indirection, and install the relative jump to the patch
			// address right before the original function. this means we need 5 bytes to be
			// available in front of the function.
			const bool isAvailable = AreBytesAvailableForPatching<5u>(processHandle, originalAddress - 5u);
			const bool installedPatch = (patchedAddresses.find(originalAddress - 5u) != patchedAddresses.end());
			if (isAvailable || installedPatch)
			{
				LC_LOG_DEV("Hot-patching function %s at 0x%p (0x%X) (PID: %d)", functionName, moduleBase, functionRva, +processId);

				// it is safe to install the relative jump right in front of the function
				patch::InstallRelativeNearJump(processHandle, originalAddress - 5u, patchAddress);

				// note that in very, very rare cases, the memory region in front of the function might not
				// be executable pages. this can only happen for the function right at the start of the
				// code segment, but it can happen.
				Process::MakePagesExecutable(processHandle, originalAddress - 5u, 5u);

				// jump to the relative jump we just installed using a short jump, using 2 bytes.
				// this memory region must always be executable already.
				patch::InstallRelativeShortJump(processHandle, originalAddress, originalAddress - 5u);

				patchedAddresses.insert(originalAddress - 5u);
			}
			else
			{
				// there is not enough space.
				// only emit a warning if the ILT also couldn't be patched.
				if (!installedPatchToILT)
				{
					if (appSettings::g_showUndecoratedNames->GetValue())
					{
						LC_WARNING_USER("Not enough space near function '%s' at 0x%X to install patch (PID: %d). Changes to this function will not be observable.", nameMangling::UndecorateSymbol(functionName, 0u).c_str(), originalAddress, +processId);
					}
					else
					{
						LC_WARNING_USER("Not enough space near function '%s' at 0x%X to install patch (PID: %d). Changes to this function will not be observable.", functionName, originalAddress, +processId);
					}
				}
			}
		}
		else
		{
			// the instruction is too short.
			// only emit a warning if the ILT also couldn't be patched.
			if (!installedPatchToILT)
			{
				if (appSettings::g_showUndecoratedNames->GetValue())
				{
					LC_WARNING_USER("Instruction in function '%s' at 0x%X is too short to install patch (PID: %d). Changes to this function will not be observable.", nameMangling::UndecorateSymbol(functionName, 0u).c_str(), originalAddress, +processId);
				}
				else
				{
					LC_WARNING_USER("Instruction in function '%s' at 0x%X is too short to install patch (PID: %d). Changes to this function will not be observable.", functionName, originalAddress, +processId);
				}
			}
		}
	}

	return record;
}


void functions::PatchFunction
(
	const Record& record,
	Process::Handle processHandle,
	void* processModuleBases[],
	void* newModuleBase,
	types::unordered_set<const void*>& patchedAddresses,
	const types::vector<const void*>& threadIPs
)
{
	void* originalModuleBase = processModuleBases[record.patchIndex];
	if (!originalModuleBase)
	{
		return;
	}

	char* originalAddress = pointer::Offset<char*>(originalModuleBase, record.functionRva);
	char* patchAddress = pointer::Offset<char*>(newModuleBase, record.patchFunctionRva);

	unsigned int patchTechniqueUsed = PatchTechnique::HOTPATCH_INDIRECTION;
	size_t wholeInstructionSize = 0u;

	// first check whether we can find an incremental linking thunk for this function.
	bool installedPatchToILT = false;
	const types::vector<uint32_t>& thunkTableEntries = symbols::FindThunkTableEntriesByRVA(record.thunkDb, record.functionRva);
	if (thunkTableEntries.size() != 0u)
	{
		// patch the ILTs directly, but keep installing patches into the real function too.
		// this acts as a safety net, should any relocation or any function ever point to the real function
		// instead of the ILT.
		for (auto thunkRva : thunkTableEntries)
		{
			void* incrementalLinkingThunk = pointer::Offset<char*>(originalModuleBase, thunkRva);
			patch::InstallRelativeNearJump(processHandle, incrementalLinkingThunk, patchAddress);
		}

		installedPatchToILT = true;
	}

	// second check whether the function is at least 5 bytes long to consider it for direct patching using
	// a single relative jump.
	if (record.directJumpInstructionSize >= 5u)
	{
		// check if a thread is currently reading from memory at the location where we
		// want to install a relative jump.
		bool anyThreadInside = false;
		const size_t threadCount = threadIPs.size();
		for (size_t t = 0u; t < threadCount; ++t)
		{
			const void* instructionPointer = threadIPs[t];
			if ((instructionPointer >= originalAddress) && (instructionPointer < originalAddress + wholeInstructionSize))
			{
				anyThreadInside = true;
				break;
			}
		}

		if (!anyThreadInside)
		{
			// no thread currently reads from there, install a direct relative jump
			patchTechniqueUsed = PatchTechnique::DIRECT_RELATIVE_JUMP;
		}
	}

	// now install a patch using the selected technique
	if (patchTechniqueUsed == PatchTechnique::DIRECT_RELATIVE_JUMP)
	{
		patch::InstallRelativeNearJump(processHandle, originalAddress, patchAddress);

		// are there remaining bytes straddling the last instruction?
		if (record.directJumpInstructionSize > 5u)
		{
			// yes, overwrite those with int 3
			Process::WriteProcessMemory(processHandle, originalAddress + 5u, INT3_PADDING, record.directJumpInstructionSize - 5u);
		}
	}
	else if (patchTechniqueUsed == PatchTechnique::HOTPATCH_INDIRECTION)
	{
		const uint8_t instructionSize = Disassembler::FindInstructionSize(processHandle, originalAddress);
		if (instructionSize >= 2u)
		{
			// we need to go via an indirection, and install the relative jump to the patch
			// address right before the original function. this means we need 5 bytes to be
			// available in front of the function.
			const bool isAvailable = AreBytesAvailableForPatching<5u>(processHandle, originalAddress - 5u);
			const bool installedPatch = (patchedAddresses.find(originalAddress - 5u) != patchedAddresses.end());
			if (isAvailable || installedPatch)
			{
				// it is safe to install the relative jump right in front of the function
				patch::InstallRelativeNearJump(processHandle, originalAddress - 5u, patchAddress);

				// note that in very, very rare cases, the memory region in front of the function might not
				// be executable pages. this can only happen for the function right at the start of the
				// code segment, but it can happen.
				Process::MakePagesExecutable(processHandle, originalAddress - 5u, 5u);

				// jump to the relative jump we just installed using a short jump, using 2 bytes.
				// this memory region must always be executable already.
				patch::InstallRelativeShortJump(processHandle, originalAddress, originalAddress - 5u);

				patchedAddresses.insert(originalAddress - 5u);
			}
		}
	}
}


functions::LibraryRecord functions::PatchLibraryFunction
(
	char* srcAddress,
	char* destAddress,
	uint32_t srcRva,
	uint32_t destRva,
	const symbols::Contribution* contribution,
	Process::Handle processHandle,
	uint16_t moduleIndex
)
{
	LibraryRecord record = { srcRva, destRva, moduleIndex, 0u };

	// patching of public functions that were pulled in from libraries is a bit different because those libraries
	// were probably not built with the /hotpatch and /FUNCTIONPADMIN switches.
	// therefore, we need to install a relative jump to the original function directly, without any indirection.
	// such a jump needs 5 bytes but is actually easier to install in this case due to the following constraints:
	//	- if the function is shorter than 5 bytes it cannot contain a jump or a relocation to another symbol, because
	//	both would need at least (1 + 4) bytes. the function therefore cannot access any data or other function, and
	//	hence is of no relevance to us.
	//	- the instruction pointer cannot be in any of these functions currently, because no code calling these functions
	//	could have possibly been run at this point (the process is still suspended).
	// therefore, we analyze the instructions in the function until we have found at least 5 bytes.
	// these 5 bytes are then patched with a relative jump to the original function, and the remaining bytes (if any)
	// are patched with int 3.
	size_t wholeInstructionSize = 0u;
	while (wholeInstructionSize < 5u)
	{
		const uint8_t instructionSize = Disassembler::FindInstructionSize(processHandle, srcAddress + wholeInstructionSize);
		if (instructionSize == 0u)
		{
			// dump raw code in case it could not be decoded
			Process::DumpMemory(processHandle, srcAddress, contribution->size);
			break;
		}

		wholeInstructionSize += instructionSize;
	}

	// did we disassemble at least 5 bytes worth of instructions?
	if (wholeInstructionSize >= 5u)
	{
		record.wholeInstructionSize = static_cast<uint16_t>(wholeInstructionSize);

		// install a relative jump to the destination right here
		patch::InstallRelativeNearJump(processHandle, srcAddress, destAddress);

		// are there remaining bytes straddling the last instruction?
		if (wholeInstructionSize > 5u)
		{
			// yes, overwrite those with int 3
			Process::WriteProcessMemory(processHandle, srcAddress + 5u, INT3_PADDING, wholeInstructionSize - 5u);
		}
	}

	return record;
}


void functions::PatchLibraryFunction
(
	const LibraryRecord& record,
	Process::Handle processHandle,
	void* processModuleBases[],
	void* newModuleBase
)
{
	void* originalModuleBase = processModuleBases[record.patchIndex];
	if (!originalModuleBase)
	{
		return;
	}

	char* srcAddress = pointer::Offset<char*>(newModuleBase, record.srcRva);
	char* destAddress = pointer::Offset<char*>(originalModuleBase, record.destRva);

	if (record.wholeInstructionSize >= 5u)
	{
		// install a relative jump to the destination right here
		patch::InstallRelativeNearJump(processHandle, srcAddress, destAddress);

		// are there remaining bytes straddling the last instruction?
		if (record.wholeInstructionSize > 5u)
		{
			// yes, overwrite those with int 3
			Process::WriteProcessMemory(processHandle, srcAddress + 5u, INT3_PADDING, record.wholeInstructionSize - 5u);
		}
	}
}


bool functions::IsValidRecord(const Record& record)
{
	return (record.thunkDb != nullptr);
}


bool functions::IsValidRecord(const LibraryRecord& record)
{
	return (record.wholeInstructionSize != 0u);
}


#endif