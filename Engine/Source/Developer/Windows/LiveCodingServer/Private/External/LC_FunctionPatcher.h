// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ProcessTypes.h"
#include "LC_Symbols.h"
#include "LC_ModuleCache.h"


namespace functions
{
	struct Record
	{
		const symbols::ThunkDB* thunkDb;
		uint32_t functionRva;
		uint32_t patchFunctionRva;
		uint16_t patchIndex;
		uint8_t directJumpInstructionSize;
	};

	struct LibraryRecord
	{
		uint32_t srcRva;
		uint32_t destRva;
		uint16_t patchIndex;
		uint16_t wholeInstructionSize;
	};

	Record PatchFunction
	(
		char* originalAddress,
		char* patchAddress,
		uint32_t functionRva,
		uint32_t patchFunctionRva,
		const symbols::ThunkDB* thunkDb,
		const symbols::Contribution* contribution,
		Process::Handle processHandle,
		void* moduleBase,
		uint16_t moduleIndex,
		types::unordered_set<const void*>& patchedAddresses,
		const types::vector<const void*>& threadIPs,

		// debug only
		Process::Id processId,
		const char* functionName
	);

	LibraryRecord PatchLibraryFunction
	(
		char* srcAddress,
		char* destAddress,
		uint32_t srcRva,
		uint32_t destRva,
		const symbols::Contribution* contribution,
		Process::Handle processHandle,
		uint16_t moduleIndex
	);

	void PatchFunction
	(
		const Record& record,
		Process::Handle processHandle,
		void* processModuleBases[],
		void* newModuleBase,
		types::unordered_set<const void*>& patchedAddresses,
		const types::vector<const void*>& threadIPs
	);

	void PatchLibraryFunction
	(
		const LibraryRecord& record,
		Process::Handle processHandle,
		void* processModuleBases[],
		void* newModuleBase
	);

	bool IsValidRecord(const Record& record);
	bool IsValidRecord(const LibraryRecord& record);
}


#endif