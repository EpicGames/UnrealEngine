// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_GrowingMemoryBlock.h"
// BEGIN EPIC MOD
#include "LC_Assert.h"
#include "LC_Logging.h"
// END EPIC MOD


GrowingMemoryBlock::GrowingMemoryBlock(size_t initialCapacity)
	: m_capacity(initialCapacity)
	, m_size(0u)
	, m_data(new uint8_t[initialCapacity])
{
	LC_ASSERT(initialCapacity != 0u, "Initial capacity cannot be zero.");
}


GrowingMemoryBlock::~GrowingMemoryBlock(void)
{
	delete[] m_data;
}


bool GrowingMemoryBlock::Insert(const void* data, size_t size)
{
	// ensure we can hold the data
	const size_t oldCapacity = m_capacity;
	size_t bytesAvailable = m_capacity - m_size;
	while (bytesAvailable < size)
	{
		// grow to 2x the capacity until the data fits
		m_capacity *= 2u;
		bytesAvailable = m_capacity - m_size;
	}

	if (oldCapacity != m_capacity)
	{
		// grow to accommodate the new size, copy over the existing data, and delete the old storage
		uint8_t* newData = new uint8_t[m_capacity];
		memcpy(newData, m_data, m_size);
		delete[] m_data;
		m_data = newData;
	}

	// copy the data into our storage, it must fit now
	memcpy(m_data + m_size, data, size);
	m_size += size;

	return true;
}


#endif