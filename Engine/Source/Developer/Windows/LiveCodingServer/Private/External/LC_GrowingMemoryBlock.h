// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
// END EPIC MOD

class GrowingMemoryBlock
{
public:
	explicit GrowingMemoryBlock(size_t initialCapacity);
	~GrowingMemoryBlock(void);

	bool Insert(const void* data, size_t size);

	inline const void* GetData(void) const
	{
		return m_data;
	}

	inline size_t GetSize(void) const
	{
		return m_size;
	}

private:
	size_t m_capacity;
	size_t m_size;
	uint8_t* m_data;

	LC_DISABLE_COPY(GrowingMemoryBlock);
	LC_DISABLE_MOVE(GrowingMemoryBlock);
	LC_DISABLE_ASSIGNMENT(GrowingMemoryBlock);
	LC_DISABLE_MOVE_ASSIGNMENT(GrowingMemoryBlock);
};


#endif