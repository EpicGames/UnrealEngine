// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Hook.h"
#include "LC_Symbols.h"


uint32_t hook::FindFirstInSection(const symbols::ImageSectionDB* imageSectionDb, const ImmutableString& sectionName)
{
	const symbols::ImageSection* imageSection = symbols::FindImageSectionByName(imageSectionDb, sectionName);
	if (imageSection)
	{
		return imageSection->rva;
	}

	return 0u;
}


uint32_t hook::FindLastInSection(const symbols::ImageSectionDB* imageSectionDb, const ImmutableString& sectionName)
{
	const symbols::ImageSection* imageSection = symbols::FindImageSectionByName(imageSectionDb, sectionName);
	if (imageSection)
	{
		return imageSection->rva + imageSection->size;
	}

	return 0u;
}


#endif