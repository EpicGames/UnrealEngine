// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include <stdint.h>
// END EPIC MOD

class ImmutableString;

namespace symbols
{
	struct ImageSectionDB;
}


namespace hook
{
	struct Type
	{
		enum Enum
		{
			PREPATCH = 0,
			POSTPATCH,
			COMPILE_START,
			COMPILE_SUCCESS,
			COMPILE_ERROR,
			COMPILE_ERROR_MESSAGE,
		};
	};

	typedef void (*PrepatchFunction)(void);
	typedef void (*PostpatchFunction)(void);

	typedef void (*CompileStartFunction)(void);
	typedef void (*CompileSuccessFunction)(void);
	typedef void (*CompileErrorFunction)(void);
	typedef void (*CompileErrorMessageFunction)(const wchar_t*);

	uint32_t FindFirstInSection(const symbols::ImageSectionDB* imageSectionDb, const ImmutableString& sectionName);
	uint32_t FindLastInSection(const symbols::ImageSectionDB* imageSectionDb, const ImmutableString& sectionName);


	// calls arbitrary hooks in a given range
	template <typename T, typename... Args>
	void CallHooksInRange(const void* rangeBegin, const void* rangeEnd, Args&&... args)
	{
		const T* firstHook = static_cast<const T*>(rangeBegin);
		const T* lastHook = static_cast<const T*>(rangeEnd);

		for (const T* hook = firstHook; hook < lastHook; ++hook)
		{
			// note that sections are often padded with zeroes, so skip everything that's zero
			T function = *hook;
			if (function)
			{
				function(args...);
			}
		}
	}
}


#endif