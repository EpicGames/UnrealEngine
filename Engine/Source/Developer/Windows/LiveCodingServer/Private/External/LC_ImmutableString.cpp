// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_ImmutableString.h"
#include "LC_StringUtil.h"
#include "LC_Hashing.h"
// BEGIN EPIC MOD
#include "LC_Allocators.h"
#include "LC_Assert.h"
#include "LC_Platform.h"
#include "LC_Logging.h"
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

namespace
{
	static inline uint32_t Hash(const char* key, size_t length)
	{
		return Hashing::Hash32(key, length * sizeof(char), 0u);
	}

	static inline const char* Clone(const char* str, size_t length)
	{
		// account for null-terminator
		const size_t sizeNeeded = (length + 1u) * sizeof(char);

		char* memory = static_cast<char*>(LC_ALLOC(&g_immutableStringAllocator, sizeNeeded, 8u));
		memcpy(memory, str, sizeof(char)*length);
		memory[length] = '\0';

		return memory;
	}
}


namespace detail
{
	std::string ToAnsiString(const wchar_t* str, size_t length)
	{
		const int sizeNeeded = ::WideCharToMultiByte(CP_ACP, 0, str, static_cast<int>(length), NULL, 0, NULL, NULL);

		char* strTo = static_cast<char*>(_alloca(sizeNeeded * sizeof(char)));

		::WideCharToMultiByte(CP_ACP, 0, str, static_cast<int>(length), strTo, sizeNeeded, NULL, NULL);
		return std::string(strTo, static_cast<size_t>(sizeNeeded));
	}


	ImmutableString ToUtf8String(const wchar_t* str, size_t length)
	{
		const int sizeNeeded = ::WideCharToMultiByte(CP_UTF8, 0, str, static_cast<int>(length), NULL, 0, NULL, NULL);

		char* strTo = static_cast<char*>(_alloca(sizeNeeded * sizeof(char)));

		::WideCharToMultiByte(CP_UTF8, 0, str, static_cast<int>(length), strTo, sizeNeeded, NULL, NULL);
		return ImmutableString(strTo, static_cast<size_t>(sizeNeeded));
	}
}


namespace string
{
	ImmutableString ToUtf8String(const wchar_t* str)
	{
		return detail::ToUtf8String(str, wcslen(str));
	}


	ImmutableString ToUtf8String(const wchar_t* str, size_t count)
	{
		size_t length = 0u;
		while ((str[length] != L'\0') && (length < count))
		{
			// find null-terminator
			++length;
		}

		return detail::ToUtf8String(str, length);
	}


	ImmutableString ToUtf8String(const std::wstring& str)
	{
		return detail::ToUtf8String(str.c_str(), str.length());
	}


	std::string ToAnsiString(const ImmutableString& utf8Str)
	{
		const std::wstring& wideStr = ToWideString(utf8Str);
		return detail::ToAnsiString(wideStr.c_str(), wideStr.length());
	}


	std::wstring ToWideString(const ImmutableString& utf8Str)
	{
		return string::ToWideString(utf8Str.c_str(), utf8Str.GetLength());
	}
}


ImmutableString::ImmutableString(void)
	: ImmutableString("", 0u)
{
}


ImmutableString::ImmutableString(const char* str)
	: ImmutableString(str, strlen(str))
{
}


ImmutableString::ImmutableString(const char* str, size_t length)
{
	if (FitsIntoShortString(length))
	{
		memcpy(m_data.shortString.str, str, length);
		m_data.shortString.str[length] = '\0';
		m_data.shortString.length = static_cast<uint8_t>(length);
		m_data.shortString.hash = Hash(str, length);

		m_data.shortString.length |= ShortString::MSB_SET_MASK;
	}
	else
	{
		m_data.longString.str = Clone(str, length);
		m_data.longString.length = static_cast<uint32_t>(length);
		m_data.longString.hash = Hash(str, length);
	}
}


ImmutableString::ImmutableString(const ImmutableString& other)
{
	if (other.IsShortString())
	{
		m_data.shortString = other.m_data.shortString;
	}
	else
	{
		m_data.longString.length = other.m_data.longString.length;
		m_data.longString.str = other.m_data.longString.str ? Clone(other.m_data.longString.str, other.GetLength()) : nullptr;
		m_data.longString.hash = other.m_data.longString.hash;
	}
}


ImmutableString::ImmutableString(ImmutableString&& other)
	: m_data(other.m_data)
{
	other.m_data = {};
}


ImmutableString& ImmutableString::operator=(ImmutableString&& other)
{
	LC_ASSERT(this != &other, "Impossible move assignment.");

	Free();

	m_data = other.m_data;
	other.m_data = {};

	return *this;
}


ImmutableString::~ImmutableString(void)
{
	Free();
}


uint32_t ImmutableString::Find(char character) const
{
	const uint32_t length = GetLength();
	const char* str = c_str();
	for (uint32_t i = 0u; i < length; ++i)
	{
		if (str[i] == character)
		{
			return i;
		}
	}

	return NOT_FOUND;
}


void ImmutableString::Free(void)
{
	if (!IsShortString())
	{
		LC_FREE(&g_immutableStringAllocator, const_cast<char*>(m_data.longString.str), (GetLength() + 1u) * sizeof(char));
	}
}


#endif