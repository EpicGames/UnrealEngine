// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Memory.h"
// BEGIN EPIC MOD
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
#include <string>
// END EPIC MOD

class ImmutableString;

// conversion utilities between ImmutableString and other string representations
namespace string
{
	ImmutableString ToUtf8String(const wchar_t* str);
	ImmutableString ToUtf8String(const wchar_t* str, size_t count);
	ImmutableString ToUtf8String(const std::wstring& str);

	std::string ToAnsiString(const ImmutableString& utf8Str);
	std::wstring ToWideString(const ImmutableString& utf8Str);
}


class ImmutableString
{
public:
	static const uint32_t NOT_FOUND = 0xFFFFFFFFu;

	struct Hasher
	{
		LC_ALWAYS_INLINE size_t operator()(const ImmutableString& key) const
		{
			return key.GetHash();
		}
	};

	struct Comparator
	{
		LC_ALWAYS_INLINE bool operator()(const ImmutableString& lhs, const ImmutableString& rhs) const
		{
			return (lhs == rhs);
		}
	};

	inline bool operator==(const ImmutableString& rhs) const
	{
		if (GetHash() != rhs.GetHash())
		{
			return false;
		}

		if (GetLength() != rhs.GetLength())
		{
			return false;
		}

		// both strings are the same length
		const char* lhsStr = c_str();
		const char* rhsStr = rhs.c_str();
		for (/* nothing */; *lhsStr != '\0'; ++lhsStr, ++rhsStr)
		{
			if (*lhsStr != *rhsStr)
			{
				return false;
			}
		}

		// reached the end of the string, must be the same
		return true;
	}

	inline bool operator!=(const ImmutableString& rhs) const
	{
		return !(*this == rhs);
	}

	// leaves the string empty, does not allocate anything
	ImmutableString(void);

	// copy and hash
	explicit ImmutableString(const char* str);

	// copy and hash
	ImmutableString(const char* str, size_t length);

	// copy
	ImmutableString(const ImmutableString& other);

	// move constructor
	ImmutableString(ImmutableString&& other);

	// disallow assignment operator
	ImmutableString& operator=(const ImmutableString& other) = delete;

	// move assignment operator
	ImmutableString& operator=(ImmutableString&& other);

	// destructor
	~ImmutableString(void);

	uint32_t Find(char character) const;

	inline uint32_t GetHash(void) const
	{
		return m_data.longString.hash;
	}

	inline uint32_t GetLength(void) const
	{
		if (IsShortString())
		{
			// clear the MSB
			return static_cast<uint32_t>(m_data.shortString.length & ShortString::MSB_CLEAR_MASK);
		}
		else
		{
			return m_data.longString.length;
		}
	}

	inline const char* c_str(void) const
	{
		if (IsShortString())
		{
			return m_data.shortString.str;
		}
		else
		{
			return m_data.longString.str;
		}
	}

private:
	// short string optimization:
	// the layout of the long string is chosen so that we can store extra information in the first byte of length,
	// assuming that no string will ever be longer than 2^24-1 characters.
	// NOTE: on little-endian architectures, LongString::length conincides with the memory address of ShortString::length
	static inline bool FitsIntoShortString(size_t lengthWithoutNullTerminator)
	{
		return (lengthWithoutNullTerminator < sizeof(ShortString::str));
	}

	inline bool IsShortString(void) const
	{
		// short strings have the MSB set
		return ((m_data.shortString.length & ShortString::MSB_SET_MASK) != 0u);
	}

	void Free(void);

	// layout of a long string:
	// 4/8 bytes pointer to string, 3 byte length, 1 zero byte, 4 byte hash
	struct LongString
	{
		const char* str;
		uint32_t length;
		uint32_t hash;
	};

	// layout of a short string:
	// the actual string data (including the null terminator), 1 byte storing the length of the short string, 4 byte hash
	struct ShortString
	{
		static const uint8_t MSB_SET_MASK = 0x80u;			// mask for setting the MSB
		static const uint8_t MSB_CLEAR_MASK = 0x7Fu;		// mask for clearing the MSB

		char str[sizeof(const char*) + sizeof(uint32_t) - 1];
		uint8_t length;
		uint32_t hash;
	};

	static_assert(sizeof(LongString) == sizeof(ShortString), "Broken short string optimization");

	union Data
	{
		LongString longString;
		ShortString shortString;
	};

	Data m_data;
};


#endif