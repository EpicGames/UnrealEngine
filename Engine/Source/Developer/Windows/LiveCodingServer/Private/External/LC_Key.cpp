// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Key.h"
// BEGIN EPIC MOD
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

input::Key::Key(int vkCode)
	: m_vkCode(vkCode)
	, m_isPressed(false)
	, m_wasPressed(false)
{
}


void input::Key::AssignCode(int vkCode)
{
	if (m_vkCode != vkCode)
	{
		// assigning to a new key
		m_isPressed = false;
		m_wasPressed = false;
	}
	m_vkCode = vkCode;
}


void input::Key::Clear(void)
{
	m_wasPressed = m_isPressed;
}


void input::Key::Update(void)
{
	m_isPressed = ((::GetAsyncKeyState(m_vkCode) & 0x8000) != 0);
}


bool input::Key::IsPressed(void) const
{
	return m_isPressed;
}


bool input::Key::WentDown(void) const
{
	return (m_isPressed && !m_wasPressed);
}


#endif