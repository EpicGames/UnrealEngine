// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD

namespace input
{
	class Key
	{
	public:
		explicit Key(int vkCode);

		void AssignCode(int vkCode);

		void Clear(void);
		void Update(void);
		bool IsPressed(void) const;
		bool WentDown(void) const;

	private:
		int m_vkCode;
		bool m_isPressed;
		bool m_wasPressed;
	};
}


#endif