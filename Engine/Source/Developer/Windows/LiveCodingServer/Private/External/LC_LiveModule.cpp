// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_LiveModule.h"
#include "LC_Telemetry.h"
#include "LC_StringUtil.h"
#include "LC_Filesystem.h"
#include "LC_Environment.h"
#include "LC_SymbolReconstruction.h"
#include "LC_Thread.h"
#include "LC_Process.h"
#include "LC_Compiler.h"
#include "LC_SyncPoint.h"
#include "LC_ExecutablePatcher.h"
#include "LC_Executable.h"
#include "LC_Hook.h"
#include "LC_RelocationPatcher.h"
#include "LC_FunctionPatcher.h"
#include "LC_Disassembler.h"
#include "LC_Patch.h"
#include "LC_PointerUtil.h"
#include "LC_DuplexPipe.h"
#include "LC_CommandMap.h"
#include "LC_CoffDetail.h"
#include "LC_FileAttributeCache.h"
#include "LC_NameMangling.h"
#include "LC_DirectoryCache.h"
#include "LC_CompilerOptions.h"
#include "LC_ModulePatch.h"
#include "LC_Amalgamation.h"
// BEGIN EPIC MOD
//#include "LC_App.h"
// END EPIC MOD
#include "LC_Scheduler.h"
#include "LC_LiveProcess.h"
#include "LC_UniqueId.h"
// BEGIN EPIC MOD
//#include "LC_FASTBuild.h"
// END EPIC MOD
#include "LC_VirtualMemoryRange.h"
#include "LPP_API.h"
// BEGIN EPIC MOD
#include "LiveCodingServer.h"
#include "LC_AppSettings.h"
#include <process.h> // needed for _PVFV
// END EPIC MOD

// BEGIN EPIC MOD - Support for UE debug visualizers
#include "Misc/Paths.h"
// END EPIC MOD

namespace
{
#if LC_64_BIT
	static const void* GetLowerBoundIn4GBRange(const void* moduleBase)
	{
		// nothing can be loaded at address 0x0, 64 KB seems like a realistic minimum
		const uint64_t LOWEST_ADDRESS = 64u * 1024u;
		const uint64_t base = pointer::AsInteger<uint64_t>(moduleBase);

		// make sure we don't underflow
		if (base >= 0x80000000ull + LOWEST_ADDRESS)
		{
			return pointer::FromInteger<const void*>(base - 0x80000000ull);
		}

		// operation would underflow
		return pointer::FromInteger<const void*>(LOWEST_ADDRESS);
	}

	static const void* GetUpperBoundIn4GBRange(const void* moduleBase)
	{
		const uint64_t HighestPossibleAddress = 0x00007FFFFFFF0000ull;
		const uint64_t HighestPossibleAddressThreshold = HighestPossibleAddress - 2ull * 1024ull * 1024ull * 1024ull;

		const uint64_t base = pointer::AsInteger<uint64_t>(moduleBase);

		// make sure we don't overflow
		if (base <= HighestPossibleAddressThreshold)
		{
			return pointer::FromInteger<const void*>(base + 2ull * 1024ull * 1024ull * 1024ull);
		}

		// operation would overflow
		return pointer::FromInteger<const void*>(HighestPossibleAddress);
	}
#endif

	// common linker options:
	//	*) create x86/x64 code
	//	*) don't echo command-line options
	//	*) disable incremental linking, otherwise the linker will emit a warning
	//	*) no manifests needed
	//	*) generate debug information
	//	*) create a hot-patchable image
	//	*) we explicitly want the .dll to be loaded anywhere in the address space, because that forces the linker to
	//		include a relocation table in the PE image
	//	*) disable ASLR (address space layout randomization) to load the .dll at the preferred image base, if possible
	//	*) don't link against any of the default libraries
	//	*) turn on OPT:REF to keep .dll and .pdb as small as possible. /OPT:ICF is not used, because binary identical but
	//		otherwise different functions would get folded, leading to confusing call stacks and wrong debug information
	//	*) create a .dll

	static const wchar_t COMMON_LINKER_OPTIONS[] = L""
#if LC_64_BIT
		L"/MACHINE:X64 "
#else
		L"/MACHINE:X86 "
#endif
		L"/NOLOGO "
		L"/INCREMENTAL:NO "
		L"/MANIFEST:NO "
		L"/DEBUG "
		L"/FUNCTIONPADMIN "
		L"/FIXED:NO "
		L"/DYNAMICBASE:NO "
		L"/NODEFAULTLIB "
		L"/OPT:REF "
		L"/OPT:NOICF "
		L"/DLL\n";


	static CriticalSection g_compileOutputCS;

	struct CompileFlags
	{
		enum Enum
		{
			NONE = 0,
			SERIALIZE_PDB_ACCESS = 1u << 0u
		};
	};

	// helper function that returns the compiler path for a compiland, taking into account UI settings
	static std::wstring GetCompilerPath(const symbols::Compiland* compiland)
	{
		const std::wstring compilerPath = string::ToWideString(compiland->compilerPath.c_str());

		// check whether compiler path is overridden
		const std::wstring overriddenCompilerPath = appSettings::GetCompilerPath();
		if (overriddenCompilerPath.length() != 0u)		
		{
			// should the overridden path be used as fallback only?
			if (appSettings::g_useCompilerOverrideAsFallback->GetValue())
			{
				// yes, so test whether a compiler at the compiland's compiler path exists
				const Filesystem::PathAttributes& attributes = Filesystem::GetAttributes(compilerPath.c_str());
				if (Filesystem::DoesExist(attributes))
				{
					// compiler exists, use it
					return compilerPath;
				}
				else
				{
					// compiler does not exist, use the fallback
					return overriddenCompilerPath;
				}
			}
			else
			{
				// no, the override should always be used
				return overriddenCompilerPath;
			}
		}
		else
		{
			// not overridden, use the compiland's compiler
			return compilerPath;
		}
	}

	// helper function that returns the linker path, taking into account UI settings
	static std::wstring GetLinkerPath(const symbols::LinkerDB* linkerDb)
	{
		const std::wstring linkerPath = string::ToWideString(linkerDb->linkerPath.c_str());

		// check whether linker path is overridden
		const std::wstring overriddenLinkerPath = appSettings::GetLinkerPath();
		if (overriddenLinkerPath.length() != 0u)
		{
			// should the overridden path be used as fallback only?
			if (appSettings::g_useLinkerOverrideAsFallback->GetValue())
			{
				// yes, so test whether a linker at the given path exists
				const Filesystem::PathAttributes& attributes = Filesystem::GetAttributes(linkerPath.c_str());
				if (Filesystem::DoesExist(attributes))
				{
					// linker exists, use it
					return linkerPath;
				}
				else
				{
					// linker does not exist, use the fallback
					return overriddenLinkerPath;
				}
			}
			else
			{
				// no, the override should always be used
				return overriddenLinkerPath;
			}
		}
		else
		{
			// not overridden
			return linkerPath;
		}
	}

	// helper function that determines the type of symbol removal strategy to use, depending on the linker
	static coff::SymbolRemovalStrategy::Enum DetermineSymbolRemovalStrategy(const symbols::LinkerDB* linkerDb)
	{
		const std::wstring linkerPath = GetLinkerPath(linkerDb);
		compiler::LinkerType::Enum linkerType = compiler::DetermineLinkerType(linkerPath.c_str());

		if (linkerType == compiler::LinkerType::LLD)
		{
			return coff::SymbolRemovalStrategy::LLD_COMPATIBLE;
		}

		return coff::SymbolRemovalStrategy::MSVC_COMPATIBLE;
	}


	// helper function for creating a CallHooks command
	static commands::CallHooks MakeCallHooksCommand(hook::Type::Enum type, const void* moduleBase, uint32_t firstRva, uint32_t lastRva)
	{
		return commands::CallHooks { type, pointer::Offset<const void*>(moduleBase, firstRva), pointer::Offset<const void*>(moduleBase, lastRva) };
	}


	// helper function for creating a CallHooks command
	static commands::CallHooks MakeCallHooksCommand(hook::Type::Enum type, const void* moduleBase, const ModuleCache::FindHookData& hookData)
	{
		return MakeCallHooksCommand(type, moduleBase, hookData.firstRva, hookData.lastRva);
	}


	static LiveModule::CompileResult Compile(ModuleCache* moduleCache, const symbols::ObjPath& normalizedObjPath, symbols::Compiland* compiland, const LiveModule::PerProcessData* processData, size_t processCount, unsigned int flags, LiveModule::UpdateType::Enum updateType)
	{
		const std::wstring compilerPath = GetCompilerPath(compiland);
		const compiler::CompilerType::Enum compilerType = compiler::DetermineCompilerType(compilerPath.c_str());

		// AMALGAMATION
		// for files that are part of an amalgamation, check their current command-line options, file timestamps, etc. against
		// those stored in the database. if nothing has changed, then don't compile the file at all.
		const bool isPartOfAmalgamation = symbols::IsPartOfAmalgamation(compiland);
		if (isPartOfAmalgamation)
		{
			if (amalgamation::ReadAndCompareDatabase(normalizedObjPath, compilerPath, compiland, appSettings::g_compilerOptions->GetValue()))
			{
				// nothing has changed according to the amalgamation database, so we can skip compilation of this file
				LC_LOG_USER("Ignoring up-to-date split file %s", normalizedObjPath.c_str());
				return LiveModule::CompileResult { 0u, false };
			}
			else
			{
				// this split file is going to be compiled. delete its database to ensure that when this file fails
				// to compile or the process terminates, the file gets compiled in the next Live++ session because
				// no database will be found on disk.
				amalgamation::DeleteDatabase(normalizedObjPath);
			}
		}

		// the compiler command-line options potentially get very long, reserve enough space.
		// note that the compiler expects commands in a response file to be in ANSI, not UTF-16.
		std::string compilerOptions;
		compilerOptions.reserve(1u * 1024u * 1024u);

		if (compilerType == compiler::CompilerType::CL)
		{
			// add the "compile only" switch in any case. if it's already there, no harm done.
			// for compilands that were compiled AND linked using cl.exe (which can call the linker internally!), this
			// needs to be added.
			compilerOptions += "-c ";

			// add compiler options based on flags
			if (flags & CompileFlags::SERIALIZE_PDB_ACCESS)
			{
				compilerOptions += "-FS ";
			}
		}

		// add the real command line for this compiland.
		// note that this must always come first for Clang, because of the first "-cc1" command.
		compilerOptions += compiland->commandLine.c_str();
		compilerOptions += " ";

		// add custom compiler options
		{
			const std::wstring customOptions = appSettings::g_compilerOptions->GetValue();
			if (customOptions.length() != 0u)
			{
				compilerOptions += string::ToAnsiString(string::ToUtf8String(customOptions));
				compilerOptions += " ";
			}
		}

		if (compilerType == compiler::CompilerType::CL)
		{
			// add the command line that specifies the .pdb path in case its not contained in the compiland's command line.
			// note that for builds using /Z7, the PDB path is optional and not needed.
			const bool hasPdbPath = (compiland->pdbPath.GetLength() != 0u);
			const bool hasPdbCommandLine = string::Contains(compiland->commandLine.c_str(), "-Fd");
			if (hasPdbPath && !hasPdbCommandLine)
			{
				compilerOptions += "-Fd\"";

				// the .PDB path could contain UTF8 characters, but the response file wants ANSI
				compilerOptions += string::ToAnsiString(compiland->pdbPath);
				compilerOptions += "\" ";
			}

			// add the command line that specifies the output .obj path in case its not contained in the compiland's command line
			if (!string::Contains(compiland->commandLine.c_str(), "-Fo"))
			{
				compilerOptions += "-Fo\"";

				// the .obj path could contain UTF8 characters, but the response file wants ANSI
				compilerOptions += string::ToAnsiString(compiland->originalObjPath);
				compilerOptions += "\" ";
			}
		}
		else if (compilerType == compiler::CompilerType::CLANG)
		{
			// for Clang, the output .obj path must always be specified
			compilerOptions += "-o\"";

			// the .obj path could contain UTF8 characters, but the response file wants ANSI.
			// additionally, Clang wants double backslashes.
			compilerOptions += string::ToAnsiString(ImmutableString(string::ReplaceAll(string::ReplaceAll(compiland->originalObjPath.c_str(), "\\", "/"), "/", "\\\\").c_str()));
			compilerOptions += "\" ";
		}

		// add the name of the compiland's source
		compilerOptions += "\"";

		// prettify the source path so that e.g. error messages will read C:\Folder\File.cpp rather than c:\folder\file.cpp.
		// normalizing is NOT allowed, we don't want to follow reparse points!
		{
			const std::wstring wideSrcPath = string::ToWideString(compiland->srcPath);
			const Filesystem::Path prettyPath = Filesystem::NormalizePathWithoutResolvingLinks(wideSrcPath.c_str());

			if (compilerType == compiler::CompilerType::CL)
			{
				compilerOptions += string::ToAnsiString(string::ToUtf8String(prettyPath.GetString()));
			}
			else if (compilerType == compiler::CompilerType::CLANG)
			{
				// Clang wants double backslashes
				compilerOptions += string::ToAnsiString(string::ToUtf8String(string::ReplaceAll(string::ReplaceAll(prettyPath.GetString(), L"\\", L"/"), L"/", L"\\\\")));
			}
		}

		compilerOptions += "\"";

		// create a temporary file that acts as a so-called response file for the compiler, and contains
		// the whole compiler command-line. this is done because the latter can get very long, longer
		// than the limit of 32k characters.
		const Filesystem::Path responseFilePath = Filesystem::GenerateTempFilename();
		Filesystem::CreateFileWithData(responseFilePath.GetString(), compilerOptions.c_str(), compilerOptions.size() * sizeof(char));

		std::wstring compilerCommandLine;
		compilerCommandLine.reserve(256u);	

		// start command line with quoted name of cl.exe, e.g. "C:\Program Files (x86)\Microsoft Visual Studio 14\VC\bin\cl.exe"
		compilerCommandLine += L"\"";
		compilerCommandLine += compilerPath;
		compilerCommandLine += L"\" ";

		// add response file to command line
		compilerCommandLine += L"@\"";
		compilerCommandLine += responseFilePath.GetString();
		compilerCommandLine += L"\"";

		const Process::Environment environment = compiler::GetEnvironmentFromCache(compilerPath.c_str());
		const void* environmentData = environment.data;
		std::wstring workingDirectory = string::ToWideString(compiland->workingDirectory);

		// if the working directory does not exist, use the compiler's directory instead.
		// otherwise, remote/distributed builds would use working directories on remote machines.
		{
			const Filesystem::PathAttributes& attributes = Filesystem::GetAttributes(workingDirectory.c_str());
			if (!Filesystem::DoesExist(attributes))
			{
				workingDirectory = Filesystem::GetDirectory(compilerPath.c_str()).GetString();
			}
		}

		LC_LOG_USER("Compiling %s %s", isPartOfAmalgamation ? "split file" : "file", normalizedObjPath.c_str());

		Process::Context* processContext = Process::Spawn(compilerPath.c_str(), workingDirectory.c_str(), compilerCommandLine.c_str(), environmentData, Process::SpawnFlags::REDIRECT_STDOUT | Process::SpawnFlags::NO_WINDOW);
		const unsigned int exitCode = Process::Wait(processContext);

		const std::wstring processStdout = Process::GetStdOutData(processContext);
		const wchar_t* compilerOutput = processStdout.c_str();

		// log the complete command-line into the DEV log
		{
			LC_LOG_DEV("Compiler command-line: ");
			Logging::LogNoFormat<Logging::Channel::DEV>(compilerOptions.c_str());
			Logging::LogNoFormat<Logging::Channel::DEV>("\n");
		}

		{
			CriticalSection::ScopedLock lock(&g_compileOutputCS);

			// log to local UI
			Logging::LogNoFormat<Logging::Channel::USER>(compilerOutput);

			const size_t outputLength = processStdout.length();
			if (outputLength != 0u)
			{
				if (updateType != LiveModule::UpdateType::NO_CLIENT_COMMUNICATION)
				{
					// send log to host DLL
					for (size_t p = 0u; p < processCount; ++p)
					{
						const DuplexPipe* pipe = processData[p].liveProcess->GetPipe();

						commands::LogOutput cmd {};
						pipe->SendCommandAndWaitForAck(cmd, compilerOutput, (outputLength + 1u) * sizeof(wchar_t));
					}

					// send log to all registered hooks in case compilation was not successful
					if (exitCode != 0u)
					{
						const ModuleCache::FindHookData& hookData = moduleCache->FindHooksInSectionBackwards(ModuleCache::SEARCH_ALL_MODULES, ImmutableString(LPP_COMPILE_ERROR_MESSAGE_SECTION));
						if ((hookData.firstRva != 0u) && (hookData.lastRva != 0u))
						{
							const size_t count = hookData.data->processes.size();
							for (size_t p = 0u; p < count; ++p)
							{
								const ModuleCache::ProcessData& hookProcessData = hookData.data->processes[p];

								void* moduleBase = hookProcessData.moduleBase;
								const DuplexPipe* pipe = hookProcessData.pipe;

								pipe->SendCommandAndWaitForAck(MakeCallHooksCommand(hook::Type::COMPILE_ERROR_MESSAGE, moduleBase, hookData), compilerOutput, (outputLength + 1u) * sizeof(wchar_t));
							}
						}
					}
				}
			}
		}
	
		Process::Destroy(processContext);

		Filesystem::Delete(responseFilePath.GetString());

		return LiveModule::CompileResult { exitCode, true };
	}


	struct SymbolAndRelocation
	{
		const coff::Symbol* symbol;
		const coff::Relocation* relocation;
	};

	static const symbols::Symbol* FindOriginalSymbolForStrippedCandidate
	(
		const ModuleCache* moduleCache,
		const ImmutableString& symbolName,
		const coff::CoffDB* coffDb,
		const types::vector<SymbolAndRelocation>& cache
	)
	{
		if (!coffDb)
		{
			return nullptr;
		}

		// if the given symbol exists in the live module already, and all relocations to it would
		// be patched anyway, then we don't need it.
		const ModuleCache::FindSymbolData findData = moduleCache->FindSymbolByName(ModuleCache::SEARCH_ALL_MODULES, symbolName);
		if (!findData.symbol)
		{
			// this symbol does not exist in our live module yet, so we absolutely need it
			return nullptr;
		}

		if (!relocations::WouldPatchRelocation(symbolName))
		{
			// we would not patch relocations to this symbol, hence it's needed
			return nullptr;
		}

		// TODO: all relocations point to the same destination symbol, hence all relocations also have the same destination section index.
		// if a relocation wouldn't be patched because of certain characteristics of the section, we can do that ONCE here, and don't need to
		// check every relocation.

		const size_t relocationCount = cache.size();
		for (size_t i = 0u; i < relocationCount; ++i)
		{
			const coff::Symbol* symbol = cache[i].symbol;
			const coff::Relocation* relocation = cache[i].relocation;
			const ImmutableString& srcSymbolName = coff::GetSymbolName(coffDb, symbol);

			// this is a relocation to the symbol in question
			if (!relocations::WouldPatchRelocation(relocation, coffDb, srcSymbolName, findData))
			{
				// this relocation to the symbol would not be patched by us, hence we probably need this symbol.
				// however, there are special cases where we want to strip symbols even though we might not patch
				// all relocations to it.

				// special case #1: a relocation from an exception-related unwind symbol (?dtor$) to a
				// dynamic initializer (??__E), e.g. a relocation from ?dtor$0@?0???__ESomeGlobalVariable@@YAXXZ@4HA
				// ("int `void __cdecl `dynamic initializer for 'SomeGlobalVariable''(void)'::`1'::dtor$0")
				// in this case, the relocation is not patched, but the dynamic initializer refers to an already
				// existing symbol. that dynamic initializer will be removed by us later on anyway, hence
				// those relocations do not really need patching.
				if (symbols::IsExceptionUnwindSymbolForDynamicInitializer(srcSymbolName))
				{
					LC_LOG_DEV("Ignoring unpatched relocation from symbol %s", srcSymbolName.c_str());
					continue;
				}

				return nullptr;
			}
		}

		// the symbol exists already, and we would patch all relocations to it anyway, so remove it
		return findData.symbol;
	}


	struct CacheUpdate
	{
		enum Enum
		{
			ALL,
			NON_EXISTANT
		};
	};


	template <typename T>
	static types::vector<symbols::ObjPath> UpdateCoffCache(const T& compilands, CoffCache<coff::CoffDB>* coffCache, CacheUpdate::Enum updateType, const types::vector<symbols::ModifiedObjFile>& modifiedOrNewObjFiles)
	{
		LC_LOG_INDENT_DEV;

		types::vector<symbols::ObjPath> updatedCoffs;
		updatedCoffs.reserve(compilands.size());

		auto taskRoot = scheduler::CreateEmptyTask();

		types::vector<scheduler::TaskBase*> tasks;
		tasks.reserve(compilands.size());

		for (auto it = compilands.begin(); it != compilands.end(); ++it)
		{
			symbols::ObjPath objPath = it->first;
			const std::wstring& wideObjPath = string::ToWideString(objPath);
			const symbols::Compiland* compiland = it->second;
			const uint32_t compilandUniqueId = symbols::GetCompilandId(compiland, wideObjPath.c_str(), modifiedOrNewObjFiles);

			const bool shouldUpdate = (updateType == CacheUpdate::NON_EXISTANT)
				? (coffCache->Lookup(objPath) == nullptr)				// NON-EXISTANT: update cache only for files which don't have an entry yet
				: true;													// ALL: always update the entry

			if (shouldUpdate)
			{
				updatedCoffs.push_back(objPath);

				auto task = scheduler::CreateTask(taskRoot, [objPath, wideObjPath, coffCache, compilandUniqueId]()
				{
					LC_LOG_DEV("Updating COFF cache for file %s", objPath.c_str());

					coff::ObjFile* objFile = coff::OpenObj(wideObjPath.c_str());
					if (objFile && objFile->memoryFile)
					{
						coff::CoffDB* database = coff::GatherDatabase(objFile, compilandUniqueId);
						if (database)
						{
							coffCache->Update(objPath, database);
						}

						coff::CloseObj(objFile);
					}

					return true;
				});
				scheduler::RunTask(task);

				tasks.emplace_back(task);
			}
		}

		// wait for all tasks to end
		scheduler::RunTask(taskRoot);
		scheduler::WaitForTask(taskRoot);

		// destroy all tasks
		scheduler::DestroyTasks(tasks);
		scheduler::DestroyTask(taskRoot);

		return updatedCoffs;
	}


#if LC_64_BIT
	static executable::PreferredBase FindPreferredImageBase(uint32_t imageSize, Process::Id processId, Process::Handle processHandle, void* moduleBase)
	{
		// work out the lower and upper bound of the memory region into which a patch could be loaded
		const uint32_t exeSize = Process::GetModuleSize(processHandle, moduleBase);
		const uint32_t patchSize = imageSize;

		const void* lowerBound = GetLowerBoundIn4GBRange(pointer::Offset<const void*>(moduleBase, exeSize));
		const void* upperBound = GetUpperBoundIn4GBRange(moduleBase);

		LC_LOG_DEV("Scanning memory range from 0x%p to 0x%p (base: 0x%p, exeSize: 0x%X, patchSize: 0x%X, PID: %d)",
			lowerBound, upperBound, moduleBase, exeSize, patchSize, +processId);

		// modules can only be loaded at 64KB boundaries, so we should scan memory only at aligned addresses
		const size_t MODULE_ALIGNMENT = 64u * 1024u;
		void* preferredBase = Process::ScanMemoryRange(processHandle, lowerBound, upperBound, patchSize, MODULE_ALIGNMENT);

		const executable::PreferredBase preferredImageBase = pointer::AsInteger<executable::PreferredBase>(preferredBase);
		LC_LOG_DEV("Preferred base address for image: 0x%" PRIX64 " (PID: %d)", preferredImageBase, +processId);

		return preferredImageBase;
	}
#endif


	// helper function that returns the instruction pointers of all threads of a process
	static types::vector<const void*> EnumerateInstructionPointers(Process::Id processId)
	{
		const std::vector<Thread::Id>& threadIds = Process::EnumerateThreads(processId);
		const size_t threadCount = threadIds.size();

		types::vector<const void*> instructionPointers;
		instructionPointers.reserve(threadCount);

		for (size_t i=0u; i < threadCount; ++i)
		{
			const Thread::Id threadId = threadIds[i];
			Thread::Handle threadHandle = Thread::Open(threadId);

			const Thread::Context context = Thread::GetContext(threadHandle);
			const void* ip = Thread::ReadInstructionPointer(&context);

			instructionPointers.push_back(ip);

			Thread::Close(threadHandle);
		}

		return instructionPointers;
	}


	// helper function that checks whether a patch was loaded at a valid address
	static bool CheckPatchAddressValidity(void* originalModuleBase, void* patchBase, Process::Handle processHandle)
	{
		if (!patchBase)
		{
			return false;
		}

#if LC_64_BIT
		// even though we rebased the image, the OS might have decided to load the DLL at a different address (though that really
		// should not happen).
		// so for 64-bit applications, check whether the patch was loaded at an address that can be reached via +/-2GB offsets from
		// the original executable. if its outside this range, we cannot use it.
		else
		{
			if (patchBase >= originalModuleBase)
			{
				const uint32_t patchSize = Process::GetModuleSize(processHandle, patchBase);
				const uint64_t displacement = pointer::Displacement<uint64_t>(originalModuleBase, pointer::Offset<const char*>(patchBase, patchSize));
				if (displacement > 0x80000000ull)
				{
					LC_ERROR_USER("Patch was loaded outside 2GB range and cannot be activated.");
					LC_ERROR_DEV("Patch loaded outside range (disp: 0x%p, base: 0x%p, patch base: 0x%p, patch size: 0x%X)", displacement, originalModuleBase, patchBase, patchSize);
					return false;
				}
			}
			else
			{
				const uint32_t exeSize = Process::GetModuleSize(processHandle, originalModuleBase);
				const uint64_t displacement = pointer::Displacement<uint64_t>(patchBase, pointer::Offset<const char*>(originalModuleBase, exeSize));
				if (displacement > 0x80000000ull)
				{
					LC_ERROR_USER("Patch was loaded outside 2GB range and cannot be activated.");
					LC_ERROR_DEV("Patch loaded outside range (disp: 0x%p, base: 0x%p, patch base: 0x%p, exe size: 0x%X)", displacement, originalModuleBase, patchBase, exeSize);
					return false;
				}
			}
		}
#else
		LC_UNUSED(originalModuleBase);
		LC_UNUSED(processHandle);
#endif

		return true;
	}


	// helper strings that denote UE4-specific symbols.
	// note that the global object array is a reference type, which we don't want to put into our UE4-specific library.
	// therefore, the object array in the DLL (reference) has a different type than the object array in our LIB (pointer).
#if LC_64_BIT
	static const char* const g_ue4NameTableInDLL = "?GFNameTableForDebuggerVisualizers_MT@@3PEAPEAPEAUFNameEntry@@EA";
	static const char* const g_ue4ObjectArrayInDLL = "?GObjectArrayForDebugVisualizers@@3AEAPEAVFChunkedFixedUObjectArray@@EA";
	static const char* const g_ue4NameTableInLIB = "?GFNameTableForDebuggerVisualizers_MT@@3PEAPEAPEAUFNameEntry@@EA";
	static const char* const g_ue4ObjectArrayInLIB = "?GObjectArrayForDebugVisualizers@@3PEAVFChunkedFixedUObjectArray@@EA";
#else
	static const char* const g_ue4NameTableInDLL = "?GFNameTableForDebuggerVisualizers_MT@@3PAPAPAUFNameEntry@@A";
	static const char* const g_ue4ObjectArrayInDLL = "?GObjectArrayForDebugVisualizers@@3AAPAVFChunkedFixedUObjectArray@@A";
	static const char* const g_ue4NameTableInLIB = "?GFNameTableForDebuggerVisualizers_MT@@3PAPAPAUFNameEntry@@A";
	static const char* const g_ue4ObjectArrayInLIB = "?GObjectArrayForDebugVisualizers@@3PAVFChunkedFixedUObjectArray@@A";
#endif

	// helper function to patch UE4-specific symbols
	static void PatchUE4NatVisSymbols(void* originalModuleBase, void* patchBase, uint32_t originalRva, uint32_t patchRva, Process::Handle processHandle)
	{
		const void* originalAddr = pointer::Offset<const void*>(originalModuleBase, originalRva);
		void* patchAddr = pointer::Offset<void*>(patchBase, patchRva);

		const uintptr_t value = Process::ReadProcessMemory<uintptr_t>(processHandle, originalAddr);
		Process::WriteProcessMemory(processHandle, patchAddr, value);
	}


	// helper function to patch security cookies
	static void PatchSecurityCookie(void* originalModuleBase, void* patchBase, uint32_t originalRva, uint32_t patchRva, Process::Handle processHandle)
	{
		const void* cookieAddr = pointer::Offset<const void*>(originalModuleBase, originalRva);
		void* newCookieAddr = pointer::Offset<void*>(patchBase, patchRva);

#if LC_64_BIT
		typedef uint64_t CookieType;
#else
		typedef uint32_t CookieType;
#endif

		const CookieType cookie = Process::ReadProcessMemory<CookieType>(processHandle, cookieAddr);
		Process::WriteProcessMemory(processHandle, newCookieAddr, cookie);
	}


	// helper function to patch DllMain
	static void PatchDllMain(void* patchBase, uint32_t dllMainRva, Process::Handle processHandle)
	{
		LC_LOG_DEV("Disabling optional DLL entry point");

		// the code with which we replace DllMain is simply:
		//	return TRUE;

		// this needs to return 1 in the (e)ax register and return from the function (which is done differently
		// depending on the architecture)

#if LC_64_BIT
		// the code to inject on x64 is:
		//		B0 01		mov al, 1
		//		C3			ret				different calling convention than x86
		// BEGIN EPIC MOD
		const uint8_t PatchData[3u] = { 0xB0, 0x01, 0xC3 };
		// END EPIC MOD
#else
		// the code to inject on x86 is:
		//		B0 01		mov al, 1
		//		C2 0C 00	ret 0Ch			different calling convention than x64
		// BEGIN EPIC MOD
		const uint8_t PatchData[5u] = { 0xB0, 0x01, 0xC2, 0x0C, 0x00 };
		// END EPIC MOD
#endif

		uint8_t* address = pointer::Offset<uint8_t*>(patchBase, dllMainRva);
		// BEGIN EPIC MOD
		Process::WriteProcessMemory(processHandle, address, PatchData, sizeof(PatchData));
		// END EPIC MOD
	}


	// helper function that generates a threshold value when to split amalgamated files, based on global app settings
	static unsigned int GetAmalgamatedSplitThreshold(void)
	{
		// changing these settings during a Live++ session is not supported, hence we use their initial values
		// rather than their current values.
		const bool shouldSplit = appSettings::g_amalgamationSplitIntoSingleParts->GetInitialValue();
		if (!shouldSplit)
		{
			return 0u;
		}

		const int threshold = appSettings::g_amalgamationSplitMinCppCount->GetInitialValue();
		if (threshold <= 1)
		{
			// negative values are illegal, and we don't attempt any splitting for 0 or 1 files, obviously
			return 0u;
		}

		return static_cast<unsigned int>(threshold);
	}


	// helper function for calling compile start hooks
	static void CallCompileStartHooks(ModuleCache* moduleCache, LiveModule::UpdateType::Enum updateType)
	{
		if (updateType == LiveModule::UpdateType::NO_CLIENT_COMMUNICATION)
		{
			return;
		}

		const ModuleCache::FindHookData& hookData = moduleCache->FindHooksInSectionBackwards(ModuleCache::SEARCH_ALL_MODULES, ImmutableString(LPP_COMPILE_START_SECTION));
		if ((hookData.firstRva != 0u) && (hookData.lastRva != 0u))
		{
			const size_t count = hookData.data->processes.size();
			for (size_t p = 0u; p < count; ++p)
			{
				const ModuleCache::ProcessData& processData = hookData.data->processes[p];

				const Process::Id pid = processData.processId;
				void* moduleBase = processData.moduleBase;
				const DuplexPipe* pipe = processData.pipe;

				LC_LOG_USER("Calling compile start hooks (PID: %d)", +pid);
				pipe->SendCommandAndWaitForAck(MakeCallHooksCommand(hook::Type::COMPILE_START, moduleBase, hookData), nullptr, 0u);
			}
		}
	}


	// helper function for calling compile success hooks
	static void CallCompileSuccessHooks(ModuleCache* moduleCache, LiveModule::UpdateType::Enum updateType)
	{
		if (updateType == LiveModule::UpdateType::NO_CLIENT_COMMUNICATION)
		{
			return;
		}

		const ModuleCache::FindHookData& hookData = moduleCache->FindHooksInSectionBackwards(ModuleCache::SEARCH_ALL_MODULES, ImmutableString(LPP_COMPILE_SUCCESS_SECTION));
		if ((hookData.firstRva != 0u) && (hookData.lastRva != 0u))
		{
			const size_t count = hookData.data->processes.size();
			for (size_t p = 0u; p < count; ++p)
			{
				const ModuleCache::ProcessData& processData = hookData.data->processes[p];

				const Process::Id pid = processData.processId;
				void* moduleBase = processData.moduleBase;
				const DuplexPipe* pipe = processData.pipe;

				LC_LOG_USER("Calling compile success hooks (PID: %d)", +pid);
				pipe->SendCommandAndWaitForAck(MakeCallHooksCommand(hook::Type::COMPILE_SUCCESS, moduleBase, hookData), nullptr, 0u);
			}
		}
	}


	// helper function for calling compile error hooks
	static void CallCompileErrorHooks(ModuleCache* moduleCache, LiveModule::UpdateType::Enum updateType)
	{
		if (updateType == LiveModule::UpdateType::NO_CLIENT_COMMUNICATION)
		{
			return;
		}

		const ModuleCache::FindHookData& hookData = moduleCache->FindHooksInSectionBackwards(ModuleCache::SEARCH_ALL_MODULES, ImmutableString(LPP_COMPILE_ERROR_SECTION));
		if ((hookData.firstRva != 0u) && (hookData.lastRva != 0u))
		{
			const size_t count = hookData.data->processes.size();
			for (size_t p = 0u; p < count; ++p)
			{
				const ModuleCache::ProcessData& processData = hookData.data->processes[p];

				const Process::Id pid = processData.processId;
				void* moduleBase = processData.moduleBase;
				const DuplexPipe* pipe = processData.pipe;

				LC_LOG_USER("Calling compile error hooks (PID: %d)", +pid);
				pipe->SendCommandAndWaitForAck(MakeCallHooksCommand(hook::Type::COMPILE_ERROR, moduleBase, hookData), nullptr, 0u);
			}
		}
	}
}


LiveModule::LiveModule(const wchar_t* moduleName, const executable::Header& imageHeader, RunMode::Enum runMode)
	: m_moduleName(moduleName)
	, m_imageHeader(imageHeader)
	, m_runMode(runMode)
	, m_compiledModulePatches()
{
	m_modifiedFiles.reserve(16u);
	m_compiledCompilands.reserve(16u);
	m_compiledModulePatches.reserve(64u);
}


LiveModule::~LiveModule(void)
{
	delete m_coffCache;
	delete m_moduleCache;

	delete m_contributionDB;
	delete m_compilandDB;
	delete m_libraryDB;
	delete m_linkerDB;
	delete m_thunkDB;
	delete m_imageSectionDB;
}


void LiveModule::Load(symbols::Provider* provider, symbols::DiaCompilandDB* diaCompilandDb)
{
	telemetry::Scope loadLiveModuleScope("Loading live module");

	m_coffCache = new CoffCache<coff::CoffDB>;
	m_moduleCache = new ModuleCache;

	// this is so fast there's nothing to gain in doing this concurrently
	IDiaSymbol* linkerSymbol = symbols::FindLinkerSymbol(diaCompilandDb);


	auto taskRoot = scheduler::CreateEmptyTask();

	// because we only read from the PDB file, most of the functions that gather data from the
	// PDB can run concurrently. however, the msdia DLL will block in certain functions when
	// being called from more than one thread. this is why we open a second and third DIA provider
	// that allow us to gather different data streams from different threads.
	auto taskSymbolDB = scheduler::CreateTask(taskRoot, [provider]()
	{
		return symbols::GatherSymbols(provider);
	});
	scheduler::RunTask(taskSymbolDB);


	auto taskLibraryDB = scheduler::CreateTask(taskRoot, [diaCompilandDb]()
	{
		return symbols::GatherLibraries(diaCompilandDb);
	});
	scheduler::RunTask(taskLibraryDB);


	auto taskContributionDB = scheduler::CreateTask(taskRoot, [this]()
	{
		symbols::Provider* localProvider = symbols::OpenEXE(m_moduleName.c_str(), symbols::OpenOptions::NONE);
		symbols::DiaCompilandDB* localDiaCompilandDb = symbols::GatherDiaCompilands(localProvider);

		auto db = symbols::GatherContributions(localProvider);

		symbols::DestroyDiaCompilandDB(localDiaCompilandDb);
		symbols::Close(localProvider);

		return db;
	});
	scheduler::RunTask(taskContributionDB);


	auto taskCompilandDB = scheduler::CreateTask(taskRoot, [this]()
	{
		symbols::Provider* localProvider = symbols::OpenEXE(m_moduleName.c_str(), symbols::OpenOptions::NONE);
		symbols::DiaCompilandDB* localDiaCompilandDb = symbols::GatherDiaCompilands(localProvider);

		uint32_t options = 0u;
		if (appSettings::g_enableDevLogCompilands->GetValue())
		{
			options |= symbols::CompilandOptions::GENERATE_LOGS;
		}
		if (appSettings::g_compilerForcePchPdbs->GetValue())
		{
			options |= symbols::CompilandOptions::FORCE_PCH_PDBS;
		}

		// in case the user wants to use a completely external build system, we track .objs only
		if (m_runMode == RunMode::EXTERNAL_BUILD_SYSTEM)
		{
			options |= symbols::CompilandOptions::TRACK_OBJ_ONLY;
		}

		symbols::CompilandDB* db = nullptr;
		// BEGIN EPIC MOD
#if 1
		db = symbols::GatherCompilands(localProvider, localDiaCompilandDb, GetAmalgamatedSplitThreshold(), options);
#else
		const std::wstring fastBuildDatabasePath = appSettings::g_fastBuildDatabasePath->GetValue();
		const std::wstring fastBuildDllName = appSettings::g_fastBuildDllName->GetValue();
		if (fastBuildDatabasePath.length() > 0u)
		{
			db = FASTBuild::GatherCompilands(fastBuildDllName.c_str(), string::ToUtf8String(fastBuildDatabasePath).c_str(), string::ToUtf8String(m_moduleName).c_str(), GetAmalgamatedSplitThreshold(), options);

			// safety net
			if (!db)
			{
				db = new symbols::CompilandDB;
			}
		}
		else
		{
			db = symbols::GatherCompilands(localProvider, localDiaCompilandDb, GetAmalgamatedSplitThreshold(), options);
		}
#endif
		// END EPIC MOD

		symbols::DestroyDiaCompilandDB(localDiaCompilandDb);
		symbols::Close(localProvider);

		return db;
	});
	scheduler::RunTask(taskCompilandDB);


	auto taskThunkDB = scheduler::CreateTask(taskRoot, [linkerSymbol]()
	{
		return symbols::GatherThunks(linkerSymbol);
	});
	scheduler::RunTask(taskThunkDB);


	auto taskImageSectionDB = scheduler::CreateTask(taskRoot, [linkerSymbol]()
	{
		return symbols::GatherImageSections(linkerSymbol);
	});
	scheduler::RunTask(taskImageSectionDB);


	auto taskLinkerDB = scheduler::CreateTask(taskRoot, [linkerSymbol]()
	{
		return symbols::GatherLinker(linkerSymbol);
	});
	scheduler::RunTask(taskLinkerDB);


	// ensure asynchronous operations have finished
	scheduler::RunTask(taskRoot);
	scheduler::WaitForTask(taskRoot);

	m_symbolDB = taskSymbolDB->GetResult();
	m_contributionDB = taskContributionDB->GetResult();
	m_compilandDB = taskCompilandDB->GetResult();
	m_libraryDB = taskLibraryDB->GetResult();
	m_thunkDB = taskThunkDB->GetResult();
	m_imageSectionDB = taskImageSectionDB->GetResult();
	m_linkerDB = taskLinkerDB->GetResult();

	// kill tasks
	scheduler::DestroyTask(taskRoot);
	scheduler::DestroyTask(taskSymbolDB);
	scheduler::DestroyTask(taskContributionDB);
	scheduler::DestroyTask(taskCompilandDB);
	scheduler::DestroyTask(taskLibraryDB);
	scheduler::DestroyTask(taskThunkDB);
	scheduler::DestroyTask(taskImageSectionDB);
	scheduler::DestroyTask(taskLinkerDB);

	symbols::FinalizeContributions(m_compilandDB, m_contributionDB);

	// check linker command-line for missing/wrong linker options
	{
		// the command-line is optional
		if (m_linkerDB->commandLine.GetLength() != 0u)
		{
			const std::string& upperCaseCmdLine = string::ToUpper(m_linkerDB->commandLine.c_str());

			// check for /FUNCTIONPADMIN
			{
				// /FUNCTIONPADMIN is off by default
				const bool containsFunctionpadmin = string::Contains(upperCaseCmdLine.c_str(), "/FUNCTIONPADMIN");
				if (!containsFunctionpadmin)
				{
					LC_WARNING_USER("Linker option /FUNCTIONPADMIN seems to be missing for module %S, some functions might not be patchable", m_moduleName.c_str());
				}
			}

			// check for /OPT:NOREF and /OPT:NOICF
			{
				const bool containsOptRef = string::Contains(upperCaseCmdLine.c_str(), "/OPT:REF");
				const bool containsOptIcf = string::Contains(upperCaseCmdLine.c_str(), "/OPT:ICF");

				// having either of those one explicitly is wrong
				if (containsOptRef)
				{
					LC_WARNING_USER("Unsupported linker option /OPT:REF is set for module %S, some functions might not be patchable", m_moduleName.c_str());
				}
				if (containsOptIcf)
				{
					LC_WARNING_USER("Unsupported linker option /OPT:ICF is set for module %S, some functions might not be patchable", m_moduleName.c_str());
				}

				const bool containsDebug = string::Contains(upperCaseCmdLine.c_str(), "/DEBUG");

				// when /DEBUG is specified, /OPT defaults to NOREF, so it is ok if neither /OPT:NOREF nor /OPT:NOICF are specified.
				// in other builds however, both /OPT:NOREF and /OPT:NOICF must be set explicitly.
				if (!containsDebug)
				{
					const bool containsOptNoRef = string::Contains(upperCaseCmdLine.c_str(), "/OPT:NOREF");
					const bool containsOptNoIcf = string::Contains(upperCaseCmdLine.c_str(), "/OPT:NOICF");

					// not having those is wrong
					if (!containsOptNoRef)
					{
						LC_WARNING_USER("Linker option /OPT:NOREF seems to be missing for module %S, some functions might not be patchable", m_moduleName.c_str());
					}
					if (!containsOptNoIcf)
					{
						LC_WARNING_USER("Linker option /OPT:NOICF seems to be missing for module %S, some functions might not be patchable", m_moduleName.c_str());
					}
				}
			}
		}
	}

	symbols::DestroyLinkerSymbol(linkerSymbol);

	// build a cache that stores all external/public symbols for each compiland.
	// at the same time, build a list of precompiled header symbols and the compiland they're stored in.
	// this is done simultaneously because it touches the same data.
	// additionally, we *also* get all weak symbols that are part of a library. those need special treatment when
	// linking.
	{
		// we only know public symbols at this point, so walk all of them and find their corresponding contribution.
		// there are two ways to go about this:
		// 1) walk all symbols, find their contribution
		// 2) walk all contributions, find their symbol
		// this needs to be done using 1), otherwise some external symbols cannot be found because their contributions
		// have been merged.
		for (auto it : m_symbolDB->symbolsByRva)
		{
			const uint32_t rva = it.first;
			const symbols::Symbol* symbol = it.second;

			const symbols::Contribution* contribution = symbols::FindContributionByRVA(m_contributionDB, rva);
			if (contribution)
			{
				const ImmutableString& compilandName = symbols::GetContributionCompilandName(m_contributionDB, contribution);
				m_externalSymbolsPerCompilandCache[compilandName].push_back(symbol);

				// is this a symbol emitted from a precompiled header?
				if (symbols::IsPchSymbol(symbol->name))
				{
					// yes, store it in our database
					m_pchSymbolToCompilandName.emplace(symbol->name, compilandName);
				}
			}
		}
	}

	if (m_runMode == RunMode::EXTERNAL_BUILD_SYSTEM)
	{
		LC_LOG_DEV("Caching all .objs on Load() due to external build system being used");

		// the user wants to use an external build system. in this case, we only track .objs for changes and never
		// compile anything ourselves. we cannot load .objs lazily in this case, so we have to do that right now.
		struct GatherResult
		{
			coff::CoffDB* database;
			symbols::ObjPath objPath;

			GatherResult(void) = default;
			// BEGIN EPIC MOD
			GatherResult(coff::CoffDB* _database, symbols::ObjPath _objPath) : database(_database), objPath(_objPath) {}
			// END EPIC MOD
			GatherResult(const GatherResult& other) = default;
			GatherResult(GatherResult&& other) = default;

			GatherResult& operator=(const GatherResult&) = delete;
			GatherResult& operator=(GatherResult&&) = default;
		};

		scheduler::TaskBase* gatherTaskRoot = scheduler::CreateEmptyTask();

		types::vector<scheduler::Task<GatherResult>*> gatherTasks;
		gatherTasks.reserve(m_compilandDB->compilands.size());

		for (auto it : m_compilandDB->compilands)
		{
			const symbols::ObjPath& objPath = it.first;
			symbols::Compiland* compiland = it.second;

			LC_LOG_DEV("Updating COFF cache for %s", objPath.c_str());

			// do the loading and gathering concurrently
			auto task = scheduler::CreateTask(gatherTaskRoot, [objPath, compiland]()
			{
				const std::wstring& wideObjPath = string::ToWideString(objPath);
				coff::ObjFile* objFile = coff::OpenObj(wideObjPath.c_str());
				coff::CoffDB* database = coff::GatherDatabase(objFile, compiland->uniqueId);
				coff::CloseObj(objFile);

				return GatherResult { database, objPath };
			});
			scheduler::RunTask(task);

			gatherTasks.emplace_back(task);
		}

		// wait for all tasks to end
		scheduler::RunTask(gatherTaskRoot);
		scheduler::WaitForTask(gatherTaskRoot);

		// store the databases into the cache
		{
			const size_t count = gatherTasks.size();
			for (size_t i = 0u; i < count; ++i)
			{
				const GatherResult& result = gatherTasks[i]->GetResult();
				coff::CoffDB* database = result.database;
				if (database)
				{
					m_coffCache->Update(result.objPath, database);
				}
			}
		}

		// destroy tasks
		scheduler::DestroyTasks(gatherTasks);
		scheduler::DestroyTask(gatherTaskRoot);
	}

	// now that all the databases are built, store their info into the module cache
	// BEGIN EPIC MOD
	m_mainModuleToken = m_moduleCache->Insert(m_symbolDB, m_contributionDB, m_compilandDB, m_thunkDB, m_imageSectionDB, provider->lastModificationTime);
	// END EPIC MOD
}


void LiveModule::Unload(void)
{
	const size_t patchCount = m_moduleCache->GetSize();
	if (patchCount == 0u)
	{
		return;
	}

	// do not unload the first "patch", as it is the main module that the user unloads
	for (size_t i = 0u; i < patchCount - 1u; ++i)
	{
		// it is crucial to unload patches from last to first, because relocations probably link back
		// to the original module!
		const ModuleCache::Data& entry = m_moduleCache->GetEntry(patchCount - 1u - i);

		const size_t processCount = entry.processes.size();
		for (size_t p = 0u; p < processCount; ++p)
		{
			const ModuleCache::ProcessData& process = entry.processes[p];
			if (!Process::IsActive(process.processHandle))
			{
				// this process is no longer valid, ignore it
				continue;
			}

			const DuplexPipe* clientPipe = process.pipe;
			clientPipe->SendCommandAndWaitForAck(commands::UnloadPatch { static_cast<HMODULE>(process.moduleBase) }, nullptr, 0u);
		}
	}
}


void LiveModule::RegisterProcess(LiveProcess* liveProcess, void* moduleBase, const std::wstring& modulePath)
{
	m_moduleCache->RegisterProcess(m_mainModuleToken, liveProcess, moduleBase);

	liveProcess->ReserveVirtualMemoryPages(moduleBase);

	PerProcessData perProcessData = { liveProcess, moduleBase, modulePath };
	m_perProcessData.emplace_back(perProcessData);
}


void LiveModule::UnregisterProcess(LiveProcess* liveProcess)
{
	const Process::Id processId = liveProcess->GetProcessId();

	m_moduleCache->UnregisterProcess(liveProcess);
	m_patchedAddressesPerProcess.erase(processId);

	for (auto it = m_perProcessData.begin(); it != m_perProcessData.end(); ++it)
	{
		const PerProcessData& data = *it;
		if (data.liveProcess == liveProcess)
		{
			liveProcess->FreeVirtualMemoryPages(data.originalModuleBase);

			m_perProcessData.erase(it);
			break;
		}
	}
}


void LiveModule::DisableControlFlowGuard(LiveProcess* liveProcess, void* moduleBase)
{
	Process::Handle processHandle = liveProcess->GetProcessHandle();

	// disable control flow guard (CFG) checks
	// https://msdn.microsoft.com/en-us/library/windows/desktop/mt637065(v=vs.85).aspx
	{
		// all CFG-enabled builds use a function pointer __guard_check_icall_fptr that initially (at compile-time) points
		// to _guard_check_icall_nop. additionally, some code (e.g. in the CRT) will directly call _guard_check_icall.
		// when such a CFG-enabled executable is loaded by a CFG-aware OS, the module loader
		// will automatically patch this function pointer to point to _guard_check_icall, and let _guard_check_icall point
		// to ntdll.dll!LdrpValidateUserCallTarget, which is not exported by the DLL, unfortunately.
		// we could easily find the function pointer and patch it to _guard_check_icall_nop so that checks do nothing,
		// but other DLLs (e.g. the CRT) contain their own copy of this function pointer, which we cannot patch because
		// we don't have that DLL's symbols.
		// one solution is to patch ntdll.dll!LdrpValidateUserCallTarget directly, because all checks will ultimately call
		// this function, but first we have to get its address.
		const symbols::Symbol* cfgFuncPtr = symbols::FindSymbolByName(m_symbolDB, ImmutableString(LC_IDENTIFIER("__guard_check_icall_fptr")));
		if (cfgFuncPtr)
		{
			// read where the __guard_check_icall_fptr function pointer currently points to.
			// there are three possibilities:
			//	1) the compiler is CFG-aware, but /guard:CF was not set
			//	2) the compiler is CFG-aware, /guard:CF was set, but the module is loaded by an OS that is not CFG-aware
			//	3) the compiler is CFG-aware, /guard:CF was set, and the module is loaded by a CFG-aware OS
			// in cases 1) and 2), the function pointer will point to _guard_check_icall_nop, while in case 3) it will point to
			// ntdll.dll!LdrpValidateUserCallTarget.
			// this means that we can simply read the address the function pointer points to, and patch the function at that
			// address to return immediately. this works in all three cases, and effectively disables CFG for *all* modules
			// in this process.
			{
				// make sure the process gets suspended while writing to its memory.
				// otherwise, writing could change the page protection of an executable page while code is currently executing
				// (when using the lpp*Async API), which would lead to a crash.
				Process::Suspend(processHandle);

				void* addr = Process::ReadProcessMemory<void*>(processHandle, pointer::Offset<const void*>(moduleBase, cfgFuncPtr->rva));
				const uint8_t OPCODE_RET = 0xC3;
				Process::WriteProcessMemory(processHandle, addr, OPCODE_RET);

				Process::Resume(processHandle);
			}
		}
	}
}


void LiveModule::UpdateDirectoryCache(DirectoryCache* cache)
{
	// walk all dependencies and generate/update cache entries for them
	for (auto it : m_compilandDB->dependencies)
	{
		symbols::Dependency* dependency = it.second;
		if (!dependency->parentDirectory)
		{
			// dependency does not have a valid parent directory entry yet, create a new one
			const ImmutableString& path = it.first;
			UpdateDirectoryCache(path, dependency, cache);
		}

		// merge initial changes of dependencies into the parent directories.
		// this ensures that modifications to source files that were done before a module was loaded,
		// will automatically be picked up by us.
		dependency->parentDirectory->hadChange |= dependency->hadInitialChange;
	}
}

// BEGIN EPIC MOD
LiveModule::ErrorType::Enum LiveModule::Update(FileAttributeCache* fileCache, DirectoryCache* directoryCache, UpdateType::Enum updateType, const types::vector<symbols::ModifiedObjFile>& modifiedOrNewObjFiles, const types::vector<std::wstring>& additionalLibraries)
// END EPIC MOD
{
	telemetry::Scope updateScope("Update live module");

	LC_LOG_DEV("LiveModule Update: %S", m_moduleName.c_str());

	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Checking modified files...");
	// END EPIC MOD

	bool usesClang = false;
	bool forceAmalgamationPartsLinkage = false;

	// only check for modifications if no files have been handed to us
	if (modifiedOrNewObjFiles.size() == 0u)
	{
		// check all files whether they changed
		for (auto compilandIt = m_compilandDB->dependencies.begin(); compilandIt != m_compilandDB->dependencies.end(); ++compilandIt)
		{
			symbols::Dependency* dependency = compilandIt->second;
			if (!dependency->parentDirectory->hadChange)
			{
				// no need to check this compiland, the parent directory didn't notice a change
				continue;
			}

			const std::wstring filePath = string::ToWideString(compilandIt->first);
			const types::vector<symbols::ObjPath>& objPaths = dependency->objPaths;

			const FileAttributeCache::Data& cacheData = fileCache->UpdateCacheData(filePath);
			const uint64_t currentTime = cacheData.lastModificationTime;
			if (currentTime != dependency->lastModification)
			{
				dependency->lastModification = currentTime;
				{
					const Filesystem::Path prettyPath = Filesystem::NormalizePathWithoutResolvingLinks(filePath.c_str());
					LC_LOG_USER("File %S was modified", prettyPath.GetString());
				}

				// AMALGAMATION
				if (appSettings::g_amalgamationSplitIntoSingleParts->GetValue())
				{
					// look at each file individually and determine what to do
					for (auto it : objPaths)
					{
						symbols::Compiland* compiland = symbols::FindCompiland(m_compilandDB, it);
						if (compiland)
						{
							if (symbols::IsAmalgamation(compiland))
							{
								// split amalgamated file
								symbols::AmalgamatedCompiland* amalgamatedCompiland = symbols::FindAmalgamatedCompiland(m_compilandDB, it);
								if (amalgamatedCompiland)
								{
									// the amalgamated compiland needs to be split into its single parts.
									// add all compilands that are part of the amalgamation for compilation.
									// we always split in this case to trigger recompiles when included headers change.
									LC_LOG_USER("Splitting amalgamated/unity file %s", it.c_str());

									if (!amalgamatedCompiland->isSplit)
									{
										// this is the first time the amalgamation is split into single files
										forceAmalgamationPartsLinkage = true;
									}

									m_modifiedFiles.insert(amalgamatedCompiland->singleParts.begin(), amalgamatedCompiland->singleParts.end());
									amalgamatedCompiland->isSplit = true;
								}
							}
							else if (symbols::IsPartOfAmalgamation(compiland))
							{
								// this file is part of an amalgamation.
								// if the amalgamation needs to be split, do that now.
								// in any case, this file needs to be recompiled.
								m_modifiedFiles.insert(it);

								// find the amalgamated compiland this file belongs to
								const ImmutableString& amalgamatedObjPath = compiland->amalgamationPath;
								symbols::AmalgamatedCompiland* amalgamatedCompiland = symbols::FindAmalgamatedCompiland(m_compilandDB, amalgamatedObjPath);
								if (amalgamatedCompiland)
								{
									if (!amalgamatedCompiland->isSplit)
									{
										// this is the first time the amalgamation is split into single files
										forceAmalgamationPartsLinkage = true;

										// the amalgamated compiland needs to be split into its single parts.
										// add all compilands that are part of the amalgamation for compilation, and mark the
										// amalgamated compiland as being split.
										LC_LOG_USER("Splitting amalgamated/unity file %s", amalgamatedObjPath.c_str());

										m_modifiedFiles.insert(amalgamatedCompiland->singleParts.begin(), amalgamatedCompiland->singleParts.end());
										amalgamatedCompiland->isSplit = true;
									}
								}
							}
							else
							{
								m_modifiedFiles.insert(it);
							}
						}
					}
				}
				else
				{
					// don't need to do anything fancy, just add all affected .objs
					m_modifiedFiles.insert(objPaths.begin(), objPaths.end());
				}
			}
		}

		if (m_runMode == RunMode::DEFAULT)
		{
			if (m_modifiedFiles.size() == 0u)
			{
				if (m_compiledCompilands.size() == 0u)
				{
					// no change detected in this module
					return ErrorType::NO_CHANGE;
				}
				else
				{
					// there are still compiled files that haven't been linked
				}
			}
			else
			{
				// BEGIN EPIC MOD
				LC_LOG_USER("Detected %zu file(s) to be compiled for Live coding module %S", m_modifiedFiles.size(), m_moduleName.c_str());
				// END EPIC MOD
			}
		}
		else if (m_runMode == RunMode::EXTERNAL_BUILD_SYSTEM)
		{
			if (m_modifiedFiles.size() == 0u)
			{
				// no changed .obj detected in this module
				return ErrorType::NO_CHANGE;
			}
		}
	}
	else
	{
		for (size_t i = 0u; i < modifiedOrNewObjFiles.size(); ++i)
		{
			LC_LOG_USER("File %S was modified or is new", modifiedOrNewObjFiles[i].objPath.c_str());
		}

		// BEGIN EPIC MOD
		LC_LOG_USER("Building patch from %zu file(s) for Live coding module %S", modifiedOrNewObjFiles.size(), m_moduleName.c_str());
		// END EPIC MOD
	}

	// let the user know that we're about to compile
	CallCompileStartHooks(m_moduleCache, updateType);

	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Updating first time COFF cache...");
	// END EPIC MOD

	// before starting to compile, update the COFF cache for files that have been touched for the first time
	struct ModifiedFile
	{
		symbols::ObjPath amalgamatedObjPath;
		symbols::ObjPath objPath;
		symbols::Compiland* compiland;
		bool compiledOnce;

		LC_DISABLE_ASSIGNMENT(ModifiedFile);
	};

	// linearized version of all modified files which have their compiland stored in the database
	types::vector<ModifiedFile> availableModifiedFiles;
	availableModifiedFiles.reserve(m_modifiedFiles.size());

	// don't update the COFF cache in case some .obj files have been handed to us.
	// this is only allowed in external build system mode and all existing .objs will have been reconstructed already then.
	// new files will automatically get reconstructed when loading the patch and its PDB.
	if (modifiedOrNewObjFiles.size() == 0u)
	{
		telemetry::Scope updatingCoffCache("Updating first time COFF cache");

		struct GatherResult
		{
			size_t fileIndex;
			coff::CoffDB* database;
		};

		scheduler::TaskBase* taskRoot = scheduler::CreateEmptyTask();

		types::vector<scheduler::Task<GatherResult>*> gatherTasks;
		gatherTasks.reserve(m_modifiedFiles.size());

		{
			types::StringSet updatedFiles;
			updatedFiles.reserve(m_modifiedFiles.size());

			size_t fileIndex = 0u;
			for (auto fileIt = m_modifiedFiles.begin(); fileIt != m_modifiedFiles.end(); ++fileIt)
			{
				const symbols::ObjPath& objPath = *fileIt;
				symbols::Compiland* compiland = symbols::FindCompiland(m_compilandDB, objPath);
				if (!compiland)
				{
					LC_ERROR_DEV("Cannot determine compiland belonging to file %s", objPath.c_str());
					continue;
				}

				// AMALGAMATION
				// if this is the first time this .obj is touched, load it into our cache before compiling.
				// we need it for reconstructing symbols lazily later.
				// note that parts of amalgamated .obj must have their symbols reconstructed from the original
				// amalgamated file, not their single parts.
				const bool isPartOfAmalgamation = symbols::IsPartOfAmalgamation(compiland);
				const ImmutableString& amalgamatedObjPath = isPartOfAmalgamation
					? compiland->amalgamationPath
					: objPath;

				availableModifiedFiles.emplace_back(ModifiedFile { amalgamatedObjPath, objPath, compiland, false });

				if (!m_coffCache->Lookup(amalgamatedObjPath))
				{
					const auto updatedFileIt = updatedFiles.find(amalgamatedObjPath);
					if (updatedFileIt == updatedFiles.end())
					{
						updatedFiles.insert(amalgamatedObjPath);

						if (isPartOfAmalgamation)
						{
							LC_LOG_DEV("Touched %s for the first time, triggering COFF cache update for amalgamated file %s", objPath.c_str(), amalgamatedObjPath.c_str());
						}
						else
						{
							LC_LOG_DEV("Touched %s for the first time, updating COFF cache", objPath.c_str());
						}

						// do the loading and gathering concurrently
						auto task = scheduler::CreateTask(taskRoot, [fileIndex, amalgamatedObjPath, compiland]()
						{
							const std::wstring& wideObjPath = string::ToWideString(amalgamatedObjPath);
							coff::ObjFile* objFile = coff::OpenObj(wideObjPath.c_str());
							coff::CoffDB* database = coff::GatherDatabase(objFile, compiland->uniqueId);
							coff::CloseObj(objFile);

							return GatherResult { fileIndex, database };
						});
						scheduler::RunTask(task);

						gatherTasks.emplace_back(task);
					}
				}

				++fileIndex;
			}
		}

		// wait for all tasks to end
		scheduler::RunTask(taskRoot);
		scheduler::WaitForTask(taskRoot);

		// store the databases into the cache
		{
			const size_t count = gatherTasks.size();
			for (size_t i=0u; i < count; ++i)
			{
				const GatherResult& result = gatherTasks[i]->GetResult();
				const size_t fileIndex = result.fileIndex;
				coff::CoffDB* database = result.database;
				if (database)
				{
					const symbols::ObjPath& amalgamatedObjPath = availableModifiedFiles[fileIndex].amalgamatedObjPath;
					m_coffCache->Update(amalgamatedObjPath, database);
				}
			}
		}

		// destroy tasks
		scheduler::DestroyTasks(gatherTasks);
		scheduler::DestroyTask(taskRoot);
	}

	const PerProcessData* processData = m_perProcessData.data();
	const size_t processCount = m_perProcessData.size();

	// BEGIN EPIC MOD
	bool enableReinstancingFlow = false;
	if (updateType != LiveModule::UpdateType::NO_CLIENT_COMMUNICATION)
	{
		for (size_t p = 0u; p < processCount; ++p)
		{
			const PerProcessData& data = processData[p];
			enableReinstancingFlow |= data.liveProcess->IsReinstancingFlowEnabled();
		}
	}
	// END EPIC MOD

	// recompile changed files
	if (m_runMode == RunMode::DEFAULT)
	{
		// BEGIN EPIC MOD
		GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Compiling...");
		// END EPIC MOD

		struct LocalCompileResult
		{
			size_t fileIndex;
			double compileTime;
			CompileResult compileResult;
		};

		ModuleCache* moduleCache = m_moduleCache;
		double wholeCompileTime = 0.0;

		// now figure out which files can be compiled in parallel.
		// first, all PCHs (if any) have to be rebuilt.
		{
			telemetry::Scope compilingPCHs("Compiling PCHs");

			unsigned int failedCompiles = 0u;

			auto taskRoot = scheduler::CreateEmptyTask();

			types::vector<scheduler::Task<LocalCompileResult>*> compileTasks;
			compileTasks.reserve(m_modifiedFiles.size());

			const size_t count = availableModifiedFiles.size();
			for (size_t i = 0u; i < count; ++i)
			{
				const symbols::ObjPath& objPath = availableModifiedFiles[i].objPath;
				symbols::Compiland* compiland = availableModifiedFiles[i].compiland;

				if (compilerOptions::CreatesPrecompiledHeader(compiland->commandLine.c_str()))
				{
					auto task = scheduler::CreateTask(taskRoot, [i, moduleCache, objPath, compiland, processData, processCount, updateType]()
					{
						telemetry::Scope compileScope("Compile");
						const CompileResult& result = Compile(moduleCache, objPath, compiland, processData, processCount, 0u, updateType);
						return LocalCompileResult{ i, compileScope.ReadSeconds(), result };
					});
					scheduler::RunTask(task);

					availableModifiedFiles[i].compiledOnce = true;
					compileTasks.emplace_back(task);
				}
			}

			// wait for all tasks to end
			scheduler::RunTask(taskRoot);
			scheduler::WaitForTask(taskRoot);

			// if any of the PCHs failed to compile, we need to bail out and cannot compile other files
			const size_t taskCount = compileTasks.size();
			for (size_t i = 0u; i < taskCount; ++i)
			{
				const LocalCompileResult& result = compileTasks[i]->GetResult();
				const size_t fileIndex = result.fileIndex;
				const symbols::ObjPath& objPath = availableModifiedFiles[fileIndex].objPath;
				symbols::Compiland* compiland = availableModifiedFiles[fileIndex].compiland;
				const CompileResult& compileResult = result.compileResult;
				const double compileTime = result.compileTime;

				OnCompiledFile(objPath, compiland, compileResult, compileTime, forceAmalgamationPartsLinkage);

				if (compileResult.exitCode != 0u)
				{
					++failedCompiles;
				}
			}

			scheduler::DestroyTasks(compileTasks);
			scheduler::DestroyTask(taskRoot);

			// at least one of the files could not be compiled
			if (failedCompiles != 0u)
			{
				// note that the array of compilands compiled so far is not cleared - we need them for the next successful
				// run in order to link them.
				LC_ERROR_USER("Compilation failed, %u PCH(s) could not be compiled (%.3fs)", failedCompiles, compilingPCHs.ReadSeconds());

				CallCompileErrorHooks(m_moduleCache, updateType);

				return ErrorType::COMPILE_ERROR;
			}

			wholeCompileTime += compilingPCHs.ReadSeconds();
		}


		// second, all files that use /Z7 can be compiled in parallel, because the compiler does not write to any PDB file,
		// only to individual object files.
		{
			telemetry::Scope compilingZ7s("Compiling files using /Z7");

			unsigned int failedCompiles = 0u;

			auto taskRoot = scheduler::CreateEmptyTask();

			types::vector<scheduler::Task<LocalCompileResult>*> compileTasks;
			compileTasks.reserve(m_modifiedFiles.size());

			const size_t count = availableModifiedFiles.size();
			for (size_t i = 0u; i < count; ++i)
			{
				if (availableModifiedFiles[i].compiledOnce)
				{
					continue;
				}

				const symbols::ObjPath& objPath = availableModifiedFiles[i].objPath;
				symbols::Compiland* compiland = availableModifiedFiles[i].compiland;

				if (compilerOptions::UsesC7DebugFormat(compiland->commandLine.c_str()))
				{
					auto task = scheduler::CreateTask(taskRoot, [i, moduleCache, objPath, compiland, processData, processCount, updateType]()
					{
						telemetry::Scope compileScope("Compile");

						const CompileResult& result = Compile(moduleCache, objPath, compiland, processData, processCount, 0u, updateType);
						return LocalCompileResult{ i, compileScope.ReadSeconds(), result };
					});
					scheduler::RunTask(task);

					availableModifiedFiles[i].compiledOnce = true;
					compileTasks.emplace_back(task);
				}
			}

			// wait for all tasks to end
			scheduler::RunTask(taskRoot);
			scheduler::WaitForTask(taskRoot);

			// bail out if any of the files failed to compile
			const size_t taskCount = compileTasks.size();
			for (size_t i = 0u; i < taskCount; ++i)
			{
				const LocalCompileResult& result = compileTasks[i]->GetResult();
				const size_t fileIndex = result.fileIndex;
				const symbols::ObjPath& objPath = availableModifiedFiles[fileIndex].objPath;
				symbols::Compiland* compiland = availableModifiedFiles[fileIndex].compiland;
				const CompileResult& compileResult = result.compileResult;
				const double compileTime = result.compileTime;

				OnCompiledFile(objPath, compiland, compileResult, compileTime, forceAmalgamationPartsLinkage);

				if (compileResult.exitCode != 0u)
				{
					++failedCompiles;
				}
			}

			scheduler::DestroyTasks(compileTasks);
			scheduler::DestroyTask(taskRoot);

			// at least one of the files could not be compiled
			if (failedCompiles != 0u)
			{
				// note that the array of compilands compiled so far is not cleared - we need them for the next successful
				// run in order to link them.
				LC_ERROR_USER("Compilation failed, %u file(s) could not be compiled (%.3fs)", failedCompiles, compilingZ7s.ReadSeconds());

				CallCompileErrorHooks(m_moduleCache, updateType);

				return ErrorType::COMPILE_ERROR;
			}

			wholeCompileTime += compilingZ7s.ReadSeconds();
		}


		// third, all files that use either /Zi or /ZI need special treatment, because the compiler writes to a PDB file, and
		// accesses to that file need to be serialized by using the /FS option.
		// furthermore, all files that have /Gm (Enable Minimal Rebuild) set cannot be compiled in parallel at all. 
		{
			telemetry::Scope compilingZis("Compiling files using /Zi");

			unsigned int failedCompiles = 0u;

			auto taskRoot = scheduler::CreateEmptyTask();

			types::vector<scheduler::Task<LocalCompileResult>*> compileTasks;
			compileTasks.reserve(m_modifiedFiles.size());

			types::StringMap<types::vector<size_t>> filesPerPdb;
			filesPerPdb.reserve(m_modifiedFiles.size());

			const size_t count = availableModifiedFiles.size();
			for (size_t i = 0u; i < count; ++i)
			{
				if (availableModifiedFiles[i].compiledOnce)
				{
					continue;
				}
				availableModifiedFiles[i].compiledOnce = true;

				const symbols::ObjPath& objPath = availableModifiedFiles[i].objPath;
				symbols::Compiland* compiland = availableModifiedFiles[i].compiland;

				if (compilerOptions::UsesMinimalRebuild(compiland->commandLine.c_str()))
				{
					// this file cannot be compiled in parallel, tell the user
					LC_WARNING_USER("Compiland %s uses compiler option \"Enable Minimal Rebuild (/Gm)\" and cannot be compiled concurrently. It is generally recommended to disable this compiler option.", objPath.c_str());

					telemetry::Scope compileScope("Compile");

					const CompileResult& result = Compile(moduleCache, objPath, compiland, processData, processCount, 0u, updateType);
					OnCompiledFile(objPath, compiland, result, compileScope.ReadSeconds(), forceAmalgamationPartsLinkage);

					if (result.exitCode != 0u)
					{
						++failedCompiles;
					}
				}
				else
				{
					// this file uses /Zi and writes to a PDB file. store it into a map indexed by the PDB file.
					// files that write to the same PDB upon compilation need to be serialized using the /FS option.
					filesPerPdb[compiland->pdbPath].push_back(i);
				}
			}

			for (auto pdbIt = filesPerPdb.begin(); pdbIt != filesPerPdb.end(); ++pdbIt)
			{
				const types::vector<size_t>& indices = pdbIt->second;
				const size_t indexCount = indices.size();

				if (indexCount == 1u)
				{
					const size_t fileIndex = indices[0];
					const symbols::ObjPath& objPath = availableModifiedFiles[fileIndex].objPath;
					symbols::Compiland* compiland = availableModifiedFiles[fileIndex].compiland;

					// this PDB file is being written to by one compiland only, we can compile that without any extra options
					auto task = scheduler::CreateTask(taskRoot, [fileIndex, moduleCache, objPath, compiland, processData, processCount, updateType]()
					{
						telemetry::Scope compileScope("Compile");

						const CompileResult& result = Compile(moduleCache, objPath, compiland, processData, processCount, 0u, updateType);
						return LocalCompileResult{ fileIndex, compileScope.ReadSeconds(), result };
					});
					scheduler::RunTask(task);

					compileTasks.emplace_back(task);
				}
				else
				{
					// the corresponding PDB file is being written to by several compilands, serialize access using the /FS option
					for (size_t i = 0u; i < indexCount; ++i)
					{
						const size_t fileIndex = indices[i];
						const symbols::ObjPath& objPath = availableModifiedFiles[fileIndex].objPath;
						symbols::Compiland* compiland = availableModifiedFiles[fileIndex].compiland;

						auto task = scheduler::CreateTask(taskRoot, [fileIndex, moduleCache, objPath, compiland, processData, processCount, updateType]()
						{
							telemetry::Scope compileScope("Compile");

							const CompileResult& result = Compile(moduleCache, objPath, compiland, processData, processCount, CompileFlags::SERIALIZE_PDB_ACCESS, updateType);
							return LocalCompileResult{ fileIndex, compileScope.ReadSeconds(), result };
						});
						scheduler::RunTask(task);

						compileTasks.emplace_back(task);
					}
				}
			}

			// wait for all tasks to end
			scheduler::RunTask(taskRoot);
			scheduler::WaitForTask(taskRoot);

			// bail out if any of the files failed to compile
			const size_t taskCount = compileTasks.size();
			for (size_t i = 0u; i < taskCount; ++i)
			{
				const LocalCompileResult& result = compileTasks[i]->GetResult();
				const size_t fileIndex = result.fileIndex;
				const symbols::ObjPath& objPath = availableModifiedFiles[fileIndex].objPath;
				symbols::Compiland* compiland = availableModifiedFiles[fileIndex].compiland;
				const CompileResult& compileResult = result.compileResult;
				const double compileTime = result.compileTime;

				OnCompiledFile(objPath, compiland, compileResult, compileTime, forceAmalgamationPartsLinkage);

				if (compileResult.exitCode != 0u)
				{
					++failedCompiles;
				}
			}

			scheduler::DestroyTasks(compileTasks);
			scheduler::DestroyTask(taskRoot);

			// at least one of the files could not be compiled
			if (failedCompiles != 0u)
			{
				// note that the array of compilands compiled so far is not cleared - we need them for the next successful
				// run in order to link them.
				LC_ERROR_USER("Compilation failed, %u file(s) could not be compiled (%.3fs)", failedCompiles, compilingZis.ReadSeconds());

				CallCompileErrorHooks(m_moduleCache, updateType);

				return ErrorType::COMPILE_ERROR;
			}

			wholeCompileTime += compilingZis.ReadSeconds();
		}

		LC_SUCCESS_USER("Successfully compiled modified files (%.3fs)", wholeCompileTime);
	}
	else if (m_runMode == RunMode::EXTERNAL_BUILD_SYSTEM)
	{
		if (modifiedOrNewObjFiles.size() == 0u)
		{
			// files were compiled by an external build system, we just have to mark them appropriately
			const size_t count = availableModifiedFiles.size();
			for (size_t i = 0u; i < count; ++i)
			{
				const symbols::ObjPath& objPath = availableModifiedFiles[i].objPath;
				symbols::Compiland* compiland = availableModifiedFiles[i].compiland;

				m_compiledCompilands.emplace(objPath, compiland);
				symbols::MarkCompilandAsRecompiled(compiland);
			}
		}
		else
		{
			// files were compiled by an external build system and handed to us.
			// there could also be new files.
			const size_t count = modifiedOrNewObjFiles.size();
			for (size_t i = 0u; i < count; ++i)
			{
				const std::wstring& wideObjPath = modifiedOrNewObjFiles[i].objPath;
				const symbols::ObjPath& objPath = string::ToUtf8String(wideObjPath);
				symbols::Compiland* compiland = symbols::FindCompiland(m_compilandDB, objPath);

				// compiland will be nullptr for new files, this is OK
				m_compiledCompilands.emplace(objPath, compiland);
			}
		}

		m_modifiedFiles.clear();
	}



	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Updating compilands...");
	// END EPIC MOD

	// we want to link a minimal .DLL file that contains all modified .OBJ files and only those required for resolving symbols.
	// because we require users to use /OPT:NOREF and /OPT:NOICF, finding the set of files that need to be linked in is
	// easy.
	// primarily, this set consists of all files that have been modified, and precompiled header files which do not belong
	// to a library - those are needed to have precompiled debug information available.
	// secondarily, most of the modified files will have unresolved symbols that would need to pull in other files.
	// due to /OPT:NOREF though, all symbols (both data & code) which are part of any of the main .obj linked into the
	// .exe will be available. those symbols that aren't must be part of a library then, which will be linked in anyway.
	typedef std::pair<symbols::ObjPath, const symbols::Compiland*> CompilandInfo;

	// stores from which .OBJ an external symbol originated
	types::StringMap<CompilandInfo> externalSymbols;
	externalSymbols.reserve(16384u);

	// stores which compilands need to be linked in
	types::StringSet neededCompilands;
	neededCompilands.reserve(m_compilandDB->compilands.size());

	{
		telemetry::Scope gatherNeededCompilandsScope("Gather needed compilands");

		LC_LOG_DEV("Finding set of .obj files");
		LC_LOG_INDENT_DEV;

		auto UpdateExternalSymbolsAndNeededFiles = [&]
			(
				const symbols::ObjPath& objPath, const symbols::Compiland* compiland, uint32_t compilandUniqueId, const types::StringMap<ImmutableString>& pchSymbolToCompilandName,
				types::StringMap<CompilandInfo>& externalSymbols, types::StringSet& neededCompilands
			)
			{
				coff::ObjFile* coffFile = coff::OpenObj(string::ToWideString(objPath).c_str());
				if (coffFile && coffFile->memoryFile)
				{
					coff::ExternalSymbolDB* externalSymbolDb = coff::GatherExternalSymbolDatabase(coffFile, compilandUniqueId);
					types::vector<std::string> linkerDirectives = coff::ExtractLinkerDirectives(coffFile);
					coff::CloseObj(coffFile);

					if (externalSymbolDb)
					{
						LC_LOG_DEV("Updated external symbols for compiland %s", objPath.c_str());

						const size_t symbolCount = externalSymbolDb->symbols.size();
						for (size_t i = 0u; i < symbolCount; ++i)
						{
							const ImmutableString& symbolName = externalSymbolDb->symbols[i];
							externalSymbols.emplace(symbolName, CompilandInfo { objPath, compiland });
						}

						coff::DestroyDatabase(externalSymbolDb);
					}
					else
					{
						LC_ERROR_DEV("External symbol database for COFF %s is invalid", objPath.c_str());
					}

					// we need to pull in any precompiled headers that might be used by this compiland.
					// check the linker includes if they want to force-link any precompiled header symbol.
					for (size_t i = 0u; i < linkerDirectives.size(); ++i)
					{
						const std::string& directive = linkerDirectives[i];

						// note that directives appear in both lower- and upper-case, so convert to upper-case first
						const std::string& upperCaseDirective = string::ToUpper(directive);
						if (string::Contains(upperCaseDirective.c_str(), "INCLUDE:"))
						{
							const std::size_t colonPos = directive.find(':');
							const std::string symbolName(directive.c_str() + colonPos + 1u, directive.c_str() + directive.length());

							// is this a symbol emitted by a precompiled header?
							const auto compilandIt = pchSymbolToCompilandName.find(ImmutableString(symbolName.c_str()));
							if (compilandIt != pchSymbolToCompilandName.end())
							{
								// yes, so pull in this compiland as well
								const symbols::ObjPath& pchObjPath = compilandIt->second;
								LC_LOG_DEV("%s requires precompiled header %s", objPath.c_str(), pchObjPath.c_str());
								neededCompilands.emplace(pchObjPath);
							}
						}
					}
				}
			};

		if (modifiedOrNewObjFiles.size() == 0u)
		{
			// we haven't been given any modified or new files, so check which compilands were recompiled and work from there
			for (auto it = m_compilandDB->compilands.begin(); it != m_compilandDB->compilands.end(); ++it)
			{
				const symbols::ObjPath& objPath = it->first;
				const symbols::Compiland* compiland = it->second;

				if (IsCompilandRecompiled(compiland))
				{
					// this file was changed/recompiled, so the new .OBJ needs to be linked in, even
					// though the file might be contained in a library.
					// we need to gather the external symbols again and cannot take the ones stored in the cache.
					LC_LOG_DEV("%s is recompiled", objPath.c_str());
					neededCompilands.emplace(objPath);

					UpdateExternalSymbolsAndNeededFiles(objPath, compiland, compiland->uniqueId, m_pchSymbolToCompilandName, externalSymbols, neededCompilands);
				}
				else
				{
					// this file has not changed, so consult the cache for external symbols
					auto cacheIt = m_externalSymbolsPerCompilandCache.find(objPath);
					if (cacheIt != m_externalSymbolsPerCompilandCache.end())
					{
						const size_t symbolCount = cacheIt->second.size();
						for (size_t i = 0u; i < symbolCount; ++i)
						{
							const ImmutableString& symbolName = cacheIt->second[i]->name;
							externalSymbols.emplace(symbolName, CompilandInfo { objPath, compiland });
						}
					}
					else
					{
						// this compiland does not store any external symbol
					}
				}
			}
		}
		else
		{
			for (auto it = modifiedOrNewObjFiles.begin(); it != modifiedOrNewObjFiles.end(); ++it)
			{
				const symbols::ObjPath objPath(string::ToUtf8String(it->objPath));
				const symbols::Compiland* compiland = symbols::FindCompiland(m_compilandDB, objPath);

				// new compilands won't be found in the database, so there's no unique ID yet that we can use
				const uint32_t compilandUniqueId = symbols::GetCompilandId(compiland, it->objPath.c_str(), modifiedOrNewObjFiles);

				// this file was either modified or is new. in any case, the new .OBJ needs to be linked in, even
				// though the file might be contained in a library.
				// we need to gather the external symbols again and cannot take the ones stored in the cache.
				LC_LOG_DEV("%s %s", objPath.c_str(), compiland ? "was recompiled" : "is new");
				neededCompilands.emplace(objPath);

				UpdateExternalSymbolsAndNeededFiles(objPath, compiland, compilandUniqueId, m_pchSymbolToCompilandName, externalSymbols, neededCompilands);
			}
		}
	}

	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Reconstructing symbols...");
	// END EPIC MOD

	// we now have a list of all .obj files that are going to be part of the next patch.
	// reconstruct symbols lazily for those object files that have not been reconstructed yet from the initial main executable.
	{
		telemetry::Scope reconstructingSymbolsFromObjScope("Reconstructing symbols");

		LC_LOG_DEV("Reconstructing symbols from OBJ");
		LC_LOG_INDENT_DEV;

		// find out which .obj files haven't been reconstructed yet
		types::vector<symbols::ObjPath> objToReconstruct;
		objToReconstruct.reserve(neededCompilands.size());

		for (auto it = neededCompilands.begin(); it != neededCompilands.end(); ++it)
		{
			const symbols::ObjPath& objPath = *it;
			if (m_reconstructedCompilands.find(objPath) == m_reconstructedCompilands.end())
			{
				// AMALGAMATION
				if (appSettings::g_amalgamationSplitIntoSingleParts->GetValue())
				{
					// make sure that existing amalgamated .objs (if any) are reconstructed first
					const symbols::Compiland* compiland = symbols::FindCompiland(m_compilandDB, objPath);
					if (compiland && symbols::IsPartOfAmalgamation(compiland))
					{
						if (m_reconstructedCompilands.find(compiland->amalgamationPath) == m_reconstructedCompilands.end())
						{
							// no entry yet for the amalgamation, must be reconstructed
							LC_LOG_DEV("Amalgamated file %s not in cache yet", compiland->amalgamationPath.c_str());
							objToReconstruct.emplace_back(compiland->amalgamationPath);
							m_reconstructedCompilands.insert(compiland->amalgamationPath);
						}
					}
				}

				// no entry yet, must be reconstructed
				LC_LOG_DEV("%s not in cache yet", objPath.c_str());
				objToReconstruct.emplace_back(objPath);
				m_reconstructedCompilands.insert(objPath);
			}
		}

		const size_t count = objToReconstruct.size();
		if (count > 0u)
		{
			executable::Image* image = executable::OpenImage(m_moduleName.c_str(), Filesystem::OpenMode::READ);
			executable::ImageSectionDB* imageSections = executable::GatherImageSectionDB(image);

			// load and cache all .obj not in the cache yet concurrently
			{
				auto taskRoot = scheduler::CreateEmptyTask();

				types::vector<scheduler::TaskBase*> tasks;
				tasks.reserve(count);

				for (size_t i = 0u; i < count; ++i)
				{
					symbols::ObjPath objPath = objToReconstruct[i];
					if (!m_coffCache->Lookup(objPath))
					{
						// there is no entry yet for this COFF in the cache.
						// this means that this .obj was not recompiled (otherwise it would have an entry already),
						// but has been pulled in for the first time due to unresolved symbols.
						auto task = scheduler::CreateTask(taskRoot, [this, objPath, &modifiedOrNewObjFiles]()
						{
							const symbols::Compiland* compiland = symbols::FindCompiland(m_compilandDB, objPath);
							const std::wstring& wideObjPath = string::ToWideString(objPath);
							const uint32_t compilandUniqueId = symbols::GetCompilandId(compiland, wideObjPath.c_str(), modifiedOrNewObjFiles);

							LC_LOG_DEV("Need %s for the first time, updating COFF cache", objPath.c_str());

							coff::ObjFile* objFile = coff::OpenObj(wideObjPath.c_str());
							if (objFile && objFile->memoryFile)
							{
								// note that even though we might be dealing with a single-part .obj of an amalgamated .obj
								// here, the symbols will be disambiguated using the same uniqueId as the original amalgamated file.
								coff::CoffDB* database = coff::GatherDatabase(objFile, compilandUniqueId);
								if (database)
								{
									m_coffCache->Update(objPath, database);
								}

								coff::CloseObj(objFile);
							}

							return true;
						});
						scheduler::RunTask(task);

						tasks.emplace_back(task);
					}
				}

				// wait for all tasks to end
				scheduler::RunTask(taskRoot);
				scheduler::WaitForTask(taskRoot);

				// destroy tasks
				scheduler::DestroyTasks(tasks);
				scheduler::DestroyTask(taskRoot);
			}

			types::StringSet noSymbolsToIgnore;

			// with the COFF cache filled, gather the dynamic initializers and remaining symbols by walking the module
			symbols::Provider* provider = symbols::OpenEXE(m_moduleName.c_str(), symbols::OpenOptions::NONE);
			{
				symbols::GatherDynamicInitializers(provider, image, imageSections, m_imageSectionDB, m_contributionDB, m_compilandDB, m_coffCache, m_symbolDB);

				for (size_t i = 0u; i < count; ++i)
				{
					const symbols::ObjPath& objPath = objToReconstruct[i];
					const coff::CoffDB* database = m_coffCache->Lookup(objPath);
					if (!database)
					{
						LC_ERROR_USER("COFF database for compiland %s is invalid (lazy reconstruct)", objPath.c_str());
						continue;
					}

					symbols::ReconstructFromExecutableCoff(provider, image, imageSections, database, noSymbolsToIgnore, objPath, m_contributionDB, m_thunkDB, m_imageSectionDB, m_symbolDB);
				}
			}
			symbols::Close(provider);

			executable::DestroyImageSectionDB(imageSections);
			executable::CloseImage(image);
		}
	}

	// now that everything is loaded and reconstructed, transform the symbols containing anonymous namespace names
	symbols::TransformAnonymousNamespaceSymbols(m_symbolDB, m_contributionDB, m_compilandDB, modifiedOrNewObjFiles);


	// update the COFF cache for all compiled files
	UpdateCoffCache(m_compiledCompilands, m_coffCache, CacheUpdate::ALL, modifiedOrNewObjFiles);

	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Stripping COFFs...");
	// END EPIC MOD

	// strip symbols which are already part of any of the modules
	typedef types::StringSet StrippedSymbols;
	types::StringMap<StrippedSymbols> strippedSymbolsPerCompiland;
	strippedSymbolsPerCompiland.reserve(neededCompilands.size());

	types::StringMap<StrippedSymbols> forceStrippedSymbolsPerCompiland;
	forceStrippedSymbolsPerCompiland.reserve(neededCompilands.size());
	{
		telemetry::Scope strippingScope("Stripping COFFs");

		LC_LOG_DEV("Stripping .OBJ files");
		LC_LOG_INDENT_DEV;

		// decide symbol removal strategy once, based on the type of linker we have
		const coff::SymbolRemovalStrategy::Enum removalStrategy = DetermineSymbolRemovalStrategy(m_linkerDB);

		types::StringMap<coff::RawCoff*> rawCoffDb;

		// first pass, read raw COFFs for needed compilands
		for (auto compilandIt = neededCompilands.begin(); compilandIt != neededCompilands.end(); ++compilandIt)
		{
			const symbols::ObjPath& objPath = *compilandIt;
			symbols::Compiland* compiland = symbols::FindCompiland(m_compilandDB, objPath);
			const std::wstring wideObjPath = string::ToWideString(objPath);

			const uint32_t compilandUniqueId = symbols::GetCompilandId(compiland, wideObjPath.c_str(), modifiedOrNewObjFiles);

			coff::ObjFile* objFile = coff::OpenObj(wideObjPath.c_str());
			if (objFile && objFile->memoryFile)
			{
				coff::RawCoff* rawCoff = coff::ReadRaw(objFile, compilandUniqueId);
				coff::CloseObj(objFile);

				if (rawCoff)
				{
					rawCoffDb.emplace(objPath, rawCoff);
				}
			}
		}

		// a simple cache that stores the symbol and relocation per destination symbol.
		// i.e. the cache is indexed by the destination symbol of a relocation, and stores all symbols and relocations
		// that relocate to that destination symbol.
		typedef types::vector<types::vector<SymbolAndRelocation>> RelocationsPerDestinationSymbolCache;
		types::StringMap<RelocationsPerDestinationSymbolCache> relocationsCachePerCompiland;

		// second pass, strip symbols for each raw COFF
		for (auto coffIt = rawCoffDb.begin(); coffIt != rawCoffDb.end(); ++coffIt)
		{
			const symbols::ObjPath& objPath = coffIt->first;
			const std::wstring wideObjPath = string::ToWideString(objPath);
			coff::RawCoff* rawCoff = coffIt->second;

			LC_LOG_DEV("Stripping file %s", objPath.c_str());

			// before stripping the file, move the original one to a backup location.
			// we need it after linking has finished  
			{
				const std::wstring bakPath = wideObjPath + L".bak";
				Filesystem::Move(wideObjPath.c_str(), bakPath.c_str());
			}

			// remove linker directives which we don't want or need.
			// *) /EDITANDCONTINUE will cause a warning in combination with OPT:REF and OPT:ICF, which we use.
			// *) /EXPORT will cause a .lib and .exp to be written for files which originally
			// are part of a DLL and export at least one symbol. we don't need those files.
			// *) /INCLUDE can cause symbols we already have to be pulled in again from .lib files.
			// this leads to code and data duplication, so it must be removed for symbols which are
			// already known to us.
			// *) /ENTRY sets a different entry point in the patch DLL, which would cause all kinds of problems.
			{
				types::vector<std::string> linkerDirectives = coff::ExtractLinkerDirectives(rawCoff);
				for (auto it = linkerDirectives.begin(); it != linkerDirectives.end(); /*nothing*/)
				{
					const std::string& directive = *it;

					// note that directives appear in both lower- and upper-case, so convert to upper-case first
					const std::string& upperCaseDirective = string::ToUpper(directive);
					if (string::Contains(upperCaseDirective.c_str(), "EDITANDCONTINUE"))
					{
						it = linkerDirectives.erase(it);
						continue;
					}
					else if (string::Contains(upperCaseDirective.c_str(), "EXPORT:"))
					{
						it = linkerDirectives.erase(it);
						continue;
					}
					else if (string::Contains(upperCaseDirective.c_str(), "INCLUDE:"))
					{
						const std::size_t colonPos = directive.find(':');
						// BEGIN EPIC MOD - This fix is already in Live++ 2
						std::string symbolName(directive.c_str() + colonPos + 1u, directive.c_str() + directive.length());
						if (symbolName.length() >= 2 && symbolName.front() == '\"' && symbolName.back() == '\"')
						{
							symbolName = symbolName.substr(1, symbolName.length() - 2);
						}
						// END EPIC MOD

						const ModuleCache::FindSymbolData findData = m_moduleCache->FindSymbolByName(ModuleCache::SEARCH_ALL_MODULES, ImmutableString(symbolName.c_str()));
						if (findData.symbol)
						{
							LC_LOG_DEV("Removing linker /INCLUDE directive to symbol %s", symbolName.c_str());
							it = linkerDirectives.erase(it);
							continue;
						}
					}
					else if (string::Contains(upperCaseDirective.c_str(), "ENTRY:"))
					{
						it = linkerDirectives.erase(it);
						continue;
					}

					++it;
				}

				coff::ReplaceLinkerDirectives(rawCoff, linkerDirectives);
			}

			// fill relocations cache
			const size_t symbolCount = coff::GetSymbolCount(rawCoff);
			RelocationsPerDestinationSymbolCache relocationsPerDstSymbol;
			relocationsPerDstSymbol.resize(symbolCount);

			const coff::CoffDB* coffDb = m_coffCache->Lookup(objPath);
			if (coffDb)
			{
				const size_t count = coffDb->symbols.size();
				for (size_t i = 0u; i < count; ++i)
				{
					const coff::Symbol* symbol = coffDb->symbols[i];
					const size_t relocationCount = symbol->relocations.size();
					for (size_t j = 0u; j < relocationCount; ++j)
					{
						const coff::Relocation* relocation = symbol->relocations[j];

						relocationsPerDstSymbol[relocation->dstSymbolNameIndex].push_back(SymbolAndRelocation { symbol, relocation });
					}
				}
			}

			StrippedSymbols& strippedSymbols = strippedSymbolsPerCompiland[objPath];
			strippedSymbols.reserve(symbolCount);

			StrippedSymbols& forceStrippedSymbols = forceStrippedSymbolsPerCompiland[objPath];
			forceStrippedSymbols.reserve(symbolCount);

			for (size_t i = 0u; i < symbolCount; i += coff::GetAuxSymbolCount(rawCoff, i) + 1u)
			{
				if (coff::IsAbsoluteSymbol(rawCoff, i))
				{
					continue;
				}
				else if (coff::IsDebugSymbol(rawCoff, i))
				{
					continue;
				}
				else if (coff::IsSectionSymbol(rawCoff, i))
				{
					continue;
				}

				const ImmutableString& symbolName = coff::GetSymbolName(rawCoff, i);
				if (symbols::IsStringLiteral(symbolName))
				{
					continue;
				}
				else if (symbols::IsFloatingPointSseAvxConstant(symbolName))
				{
					continue;
				}
				else if (symbols::IsLineNumber(symbolName))
				{
					continue;
				}

				if (symbols::IsPchSymbol(symbolName))
				{
					// never strip symbols that force-link the PCH
					continue;
				}
				else if (symbols::IsRttiObjectLocator(symbolName))
				{
					// never strip RTTI object locators, because its relocations are not handled
					// by our COFF mechanism.
					continue;
				}
				else if (symbols::IsPointerToDynamicInitializer(symbolName))
				{
					// never strip $initializer$ symbols. these are only (very small) function pointers
					// to dynamic initializers so stripping them doesn't yield much.
					// additionally - and this is more important! - we need them to be intact so we can
					// reconstruct symbols from them in case we cannot find certain dynamic initializer symbols.
					continue;
				}
				else if (symbols::IsExceptionRelatedSymbol(symbolName))
				{
					// never strip symbols belonging to any exception mechanism.
					// in x64, throwing an exception calls _CxxThrowException, which (later on) ends up
					// relying on __CxxFrameHandler3 - if we strip that function, relocations inside exception
					// data structures will not be patched properly, and the code will crash with the following
					// callstack:
					/*
						ExeDynamicRuntime.exe!__CxxFrameHandler3()
						ntdll.dll!RtlpExecuteHandlerForException()
						ntdll.dll!RtlDispatchException()
						ntdll.dll!KiUserExceptionDispatch()
						KernelBase.dll!RaiseException()
						vcruntime140d.dll!_CxxThrowException(void * pExceptionObject, const _s__ThrowInfo * pThrowInfo)
					*/
					continue;
				}
				// BEGIN EPIC MOD
				else if (symbols::IsUENoStripSymbol(symbolName))
				{
					continue;
				}
				// END EPIC MOD

				LC_LOG_DEV("Considering symbol %s for stripping", symbolName.c_str());
				LC_LOG_INDENT_DEV;

				bool tryStrip = false;
				bool doStrip = false;

				const coff::SymbolType::Enum type = coff::GetSymbolType(rawCoff, i);
				if (coff::IsUndefinedSymbol(rawCoff, i))
				{
					// this is an undefined symbol to any other translation unit.
					// if the symbol is not part of any of the .obj we recompiled, but comes from an .obj
					// that would otherwise be linked in (e.g. the PCH), we strip this symbol and force a relocation
					// to it later on. because its file wasn't recompiled, it couldn't possible have changed,
					// therefore it is safe to relocate to it.
					const auto& symbolIt = externalSymbols.find(symbolName);
					if (symbolIt != externalSymbols.end())
					{
						const symbols::Compiland* otherCompiland = symbolIt->second.second;
						if (otherCompiland)
						{
							if (symbols::IsCompilandRecompiled(otherCompiland))
							{
								// the external symbol comes from one of the *other* recompiled .obj.
								// in this case, the symbol might have changed, so we are only allowed to strip it
								// if all relocations to it would be patched anyway.
								tryStrip = true;
								LC_LOG_DEV("Symbol comes from recompiled compiland");
							}
							else
							{
								// the external symbol comes from an .obj that was not recompiled.
								// in this case, the symbol couldn't have changed, so we strip it directly
								// in case it exists in our live module already.
								const ModuleCache::FindSymbolData findData = m_moduleCache->FindSymbolByName(ModuleCache::SEARCH_ALL_MODULES, symbolName);
								if (findData.symbol)
								{
									doStrip = true;
									forceStrippedSymbols.insert(symbolName);
								}
								else
								{
									LC_LOG_DEV("Symbol seems to be new (compiland)");
								}
							}
						}
						else
						{
							// the symbol must come from a new .obj, so we aren't allowed to strip it
							LC_LOG_DEV("Symbol comes from new compiland");
						}
					}
					else
					{
						// the symbol doesn't come from any of the translation units, so it must be a new
						// symbol or one coming from a library. if it exists already, it cannot have changed,
						// so we strip it directly.
						const ModuleCache::FindSymbolData findData = m_moduleCache->FindSymbolByName(ModuleCache::SEARCH_ALL_MODULES, symbolName);
						if (findData.symbol)
						{
							doStrip = true;
							forceStrippedSymbols.insert(symbolName);
						}
						else
						{
							LC_LOG_DEV("Symbol seems to be new (library)");
						}
					}
				}
				else
				{
					// this is a symbol defined in this translation unit.
					// data symbols can be stripped if they already exist and we would relocate to it anyway,
					// functions are always kept.
					if ((type == coff::SymbolType::EXTERNAL_DATA) || (type == coff::SymbolType::STATIC_DATA))
					{
						// don't try stripping MSVC JustMyCode symbols
						const uint32_t symbolSectionIndex = coff::GetSymbolSectionIndex(rawCoff, i);
						const ImmutableString& sectionName = coff::GetSectionName(rawCoff, symbolSectionIndex);
						if (coff::IsMSVCJustMyCodeSection(sectionName.c_str()))
						{
							LC_LOG_DEV("Symbol is an MSVC JustMyCode symbol");
						}
						else
						{
							tryStrip = true;
						}
					}
					else
					{
						LC_LOG_DEV("Symbol is a function defined in this compiland");
					}
				}

				if (tryStrip)
				{
					LC_LOG_DEV("Trying to strip symbol %s", symbolName.c_str());

					// if this symbol already exists and we would relocate to it, then strip it from the OBJ
					const symbols::Symbol* strippedSymbol = FindOriginalSymbolForStrippedCandidate(m_moduleCache, symbolName, coffDb, relocationsPerDstSymbol[i]);
					if (strippedSymbol)
					{
						doStrip = true;
					}
				}

				if (doStrip)
				{
					coff::RemoveSymbol(rawCoff, i, removalStrategy);
					strippedSymbols.insert(symbolName);

					// we deliberately do not remove the relocations to this symbol, otherwise the debug
					// information is incorrect, and the patch PDB will contain wrong addresses, which would
					// ultimately lead to us patching relocations and functions with a wrong address.
				}
			}

			relocationsCachePerCompiland.emplace(objPath, relocationsPerDstSymbol);
		}

		// third pass, make sure that symbols that have been stripped in one COFF are stripped in all COFFs where they are undefined.
		// otherwise, we would run into linker errors due to unresolved symbols.
		// this only needs to be done if there is more than one needed compiland.
		if (neededCompilands.size() > 1u)
		{
			LC_LOG_DEV("Performing global COFF stripping");
			LC_LOG_INDENT_DEV;

			// merge all stripped symbols into one set
			StrippedSymbols allStrippedSymbols;
			StrippedSymbols allForceStrippedSymbols;

			for (auto symbolsIt = strippedSymbolsPerCompiland.begin(); symbolsIt != strippedSymbolsPerCompiland.end(); ++symbolsIt)
			{
				const StrippedSymbols& strippedSymbols = symbolsIt->second;
				allStrippedSymbols.insert(strippedSymbols.begin(), strippedSymbols.end());
			}

			for (auto symbolsIt = forceStrippedSymbolsPerCompiland.begin(); symbolsIt != forceStrippedSymbolsPerCompiland.end(); ++symbolsIt)
			{
				const StrippedSymbols& strippedSymbols = symbolsIt->second;
				allForceStrippedSymbols.insert(strippedSymbols.begin(), strippedSymbols.end());
			}

			// walk all COFFs and strip all symbols that were stripped in other COFFs
			for (auto coffIt = rawCoffDb.begin(); coffIt != rawCoffDb.end(); ++coffIt)
			{
				const symbols::ObjPath& objPath = coffIt->first;

				LC_LOG_DEV("Compiland %s", objPath.c_str());
				LC_LOG_INDENT_DEV;

				StrippedSymbols& strippedSymbols = strippedSymbolsPerCompiland[objPath];
				StrippedSymbols& forceStrippedSymbols = forceStrippedSymbolsPerCompiland[objPath];

				coff::RawCoff* rawCoff = coffIt->second;
				
				const size_t symbolCount = coff::GetSymbolCount(rawCoff);
				for (size_t i = 0u; i < symbolCount; i += coff::GetAuxSymbolCount(rawCoff, i) + 1u)
				{
					if (coff::IsAbsoluteSymbol(rawCoff, i))
					{
						continue;
					}
					else if (coff::IsDebugSymbol(rawCoff, i))
					{
						continue;
					}
					else if (coff::IsSectionSymbol(rawCoff, i))
					{
						continue;
					}
					else if (coff::IsRemovedSymbol(rawCoff, i, removalStrategy))
					{
						// this symbol has been removed already
						continue;
					}
					else if (!coff::IsUndefinedSymbol(rawCoff, i))
					{
						// we are only allowed to consider undefined symbols
						continue;
					}

					const ImmutableString& symbolName = coff::GetSymbolName(rawCoff, i);
					{
						const auto findIt = allStrippedSymbols.find(symbolName);
						const auto forceFindIt = allForceStrippedSymbols.find(symbolName);

						if ((findIt != allStrippedSymbols.end()) || (forceFindIt != allForceStrippedSymbols.end()))
						{
							// this is an undefined symbol that needs to be stripped.
							// because it's undefined, we need to make sure that *all* relocations to it are always patched,
							// hence we mark the symbol as force stripped.
							LC_LOG_DEV("Stripping symbol %s", symbolName.c_str());

							coff::RemoveSymbol(rawCoff, i, removalStrategy);
							strippedSymbols.insert(symbolName);
							forceStrippedSymbols.insert(symbolName);
						}
					}
				}
			}
		}

		// last pass, strip all sections that no longer contain symbols
		for (auto coffIt = rawCoffDb.begin(); coffIt != rawCoffDb.end(); ++coffIt)
		{
			const symbols::ObjPath& objPath = coffIt->first;
			const std::wstring wideObjPath = string::ToWideString(objPath);
			coff::RawCoff* rawCoff = coffIt->second;
			const size_t symbolCount = coff::GetSymbolCount(rawCoff);
			const coff::CoffDB* coffDb = m_coffCache->Lookup(objPath);
			const RelocationsPerDestinationSymbolCache& relocationsPerDstSymbol = relocationsCachePerCompiland[objPath];

			StrippedSymbols& strippedSymbols = strippedSymbolsPerCompiland[objPath];

			// now that we removed symbols (and corresponding relocations), strip all sections that no longer
			// store any meaningful information.
			types::unordered_set<size_t> sectionsWithMeaningfulSymbols;
			for (size_t i = 0u; i < symbolCount; i += coff::GetAuxSymbolCount(rawCoff, i) + 1u)
			{
				if (coff::IsAbsoluteSymbol(rawCoff, i))
				{
					continue;
				}
				else if (coff::IsDebugSymbol(rawCoff, i))
				{
					continue;
				}
				else if (coff::IsUndefinedSymbol(rawCoff, i))
				{
					continue;
				}
				else if (coff::IsSectionSymbol(rawCoff, i))
				{
					continue;
				}
				else if (coff::IsRemovedSymbol(rawCoff, i, removalStrategy))
				{
					continue;
				}

				// if this symbol is not one we deleted, this section stores at least one meaningful symbol
				const uint32_t symbolSectionIndex = coff::GetSymbolSectionIndex(rawCoff, i);
				sectionsWithMeaningfulSymbols.insert(symbolSectionIndex);
			}

			// COFF compiled with Clang often have sections that don't contain any symbol in the symbol table, but relocations that refer to these sections.
			// we must make sure that those sections are not stripped in case any non-stripped symbol still refers to them.
			// TODO: we should not be doing this for each compiland, this is overhead just for Clang.
			const symbols::Compiland* compiland = symbols::FindCompiland(m_compilandDB, objPath);

			// in external build system mode, we don't have a compiland for completely new files.
			// in this case, we just assume that the file was built with MSVC.
			compiler::CompilerType::Enum compilerType = compiler::CompilerType::CL;
			if (compiland)
			{
				const std::wstring compilerPath = GetCompilerPath(compiland);
				compilerType = compiler::DetermineCompilerType(compilerPath.c_str());

				if (compilerType == compiler::CompilerType::CLANG)
				{
					usesClang = true;

					for (size_t i = 0u; i < relocationsPerDstSymbol.size(); ++i)
					{
						const auto& relocationsForSymbol = relocationsPerDstSymbol[i];
						if (relocationsForSymbol.size() == 0u)
						{
							continue;
						}

						const size_t relocationCount = relocationsForSymbol.size();
						for (size_t j = 0u; j < relocationCount; ++j)
						{
							const coff::Relocation* relocation = relocationsForSymbol[j].relocation;
							if (relocation->dstIsSection)
							{
								// this relocation refers to a section.
								// if the source symbol was not stripped by us, then this section still contains meaningful data and must be kept.
								const coff::Symbol* symbol = relocationsForSymbol[j].symbol;
								const ImmutableString& srcSymbolName = coff::GetSymbolName(coffDb, symbol);

								// TODO: can this be made better?
								if (strippedSymbolsPerCompiland.find(srcSymbolName) == strippedSymbolsPerCompiland.end())
								{
									// the source symbol was not stripped, so we must keep this section
									if (relocation->dstSectionIndex >= 0)
									{
										sectionsWithMeaningfulSymbols.insert(static_cast<uint32_t>(relocation->dstSectionIndex));
									}

									// all relocations refer to the same destination symbol, so we can stop checking more source symbols
									break;
								}
							}
						}
					}
				}
			}

			const size_t sectionCount = coff::GetSectionCount(rawCoff);
			for (size_t i = 0u; i < sectionCount; ++i)
			{
				const IMAGE_SECTION_HEADER* header = &rawCoff->sections[i].header;
				if (coffDetail::IsDirectiveSection(header))
				{
					continue;
				}
				else if (coffDetail::IsDiscardableSection(header))
				{
					// usually, having discardable COMDAT sections is not a problem - this is what .debug$S sections are.
					// however, discardable COMDAT sections which are marked 'pick any' by using __declspec(selectany)
					// must hold at least one symbol, otherwise they must be removed.
					// if they are not removed, the linker will complain with:
					//   LNK1143: invalid or corrupt file: no symbol for COMDAT section 0x4
					if (!coff::IsSelectAnyComdatSection(rawCoff, i))
					{
						// probably a debug section. we are only allowed to remove these via their corresponding COMDAT section.
						continue;
					}
				}
				else if (!coffDetail::IsPartOfImage(header))
				{
					// probably a debug section. we are only allowed to remove these via their corresponding COMDAT section
					continue;
				}

				// don't strip sections implicitly needed by the linker.
				// on Clang, they don't contain any symbols, but relocations to meaningful symbols.
				if (compilerType == compiler::CompilerType::CLANG)
				{
					const ImmutableString& sectionName = coff::GetSectionName(rawCoff, i);

					// dynamic initializers
					if (string::StartsWith(sectionName.c_str(), ".CRT$"))
					{
						continue;
					}
					// exception data
					else if (string::StartsWith(sectionName.c_str(), ".xdata"))
					{
						continue;
					}
					else if (string::StartsWith(sectionName.c_str(), ".pdata"))
					{
						continue;
					}
				}

				if (sectionsWithMeaningfulSymbols.find(i) == sectionsWithMeaningfulSymbols.end())
				{
					// this section has no more meaningful symbols, remove it
					coff::RemoveSection(rawCoff, i);

					// also remove all COMDAT sections that can only be linked in case this section exists
					coff::RemoveAssociatedComdatSections(rawCoff, i);
				}
			}

			// walk over the symbols one last time, and remove the ones that now live in a section that has been
			// removed in the last step due to removing associated COMDAT sections.
			for (size_t i = 0u; i < symbolCount; i += coff::GetAuxSymbolCount(rawCoff, i) + 1u)
			{
				if (coff::IsAbsoluteSymbol(rawCoff, i))
				{
					continue;
				}
				else if (coff::IsDebugSymbol(rawCoff, i))
				{
					continue;
				}
				else if (coff::IsUndefinedSymbol(rawCoff, i))
				{
					continue;
				}
				else if (coff::IsSectionSymbol(rawCoff, i))
				{
					continue;
				}
				else if (coff::IsRemovedSymbol(rawCoff, i, removalStrategy))
				{
					continue;
				}

				const uint32_t symbolSectionIndex = coff::GetSymbolSectionIndex(rawCoff, i);
				const coff::RawSection& section = rawCoff->sections[symbolSectionIndex];
				if (section.wasRemoved)
				{
					const ImmutableString& symbolName = coff::GetSymbolName(rawCoff, i);
					const symbols::Symbol* strippedSymbol = FindOriginalSymbolForStrippedCandidate(m_moduleCache, symbolName, coffDb, relocationsPerDstSymbol[i]);
					if (strippedSymbol)
					{
						coff::RemoveSymbol(rawCoff, i, removalStrategy);
						coff::RemoveRelocations(rawCoff, i);
						strippedSymbols.insert(symbolName);
					}
				}

				// Clang workaround: in certain cases, we don't find the _GLOBAL__sub_I_ dynamic initializer symbol, which can lead to
				// a crash in user code, unfortunately.
				// as a workaround, we patch the code of the function itself to return immediately, so that even when it is called, it doesn't do anything.
				if (usesClang)
				{
					const ImmutableString& symbolName = coff::GetSymbolName(rawCoff, i);
					const bool isClangDynamicInitializer = (string::StartsWith(symbolName.c_str(), "_GLOBAL__sub_I_"));
					if (isClangDynamicInitializer)
					{
						coff::PatchFunctionSymbol(rawCoff, i, symbolSectionIndex);
					}
				}
			}

			coff::WriteRaw(wideObjPath.c_str(), rawCoff, removalStrategy);
			coff::DestroyRaw(rawCoff);
		}
	}

	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Generating linker command line...");
	// END EPIC MOD

	telemetry::Scope generateLinkerCommandLine("Generate linker command line");

	// link all .obj files into a single executable. the linker command-line options potentially get very long,
	// reserve enough space.
	std::wstring linkerOptions;
	linkerOptions.reserve(4u * 1024u * 1024u);

	// UTF-16 response files must include a byte-order mark
	const wchar_t BOM_0xFFFE = 65279u;		// ends up as FF FE in the file
	linkerOptions.push_back(BOM_0xFFFE);

	// add custom linker options
	linkerOptions += appSettings::g_linkerOptions->GetValue();
	linkerOptions += L" ";
	linkerOptions += COMMON_LINKER_OPTIONS;

	// compilation of all files succeeded. grab their external symbols database and update the cache entry.
	// additionally build a list of all external functions to be included by the linker.
	LC_LOG_DEV("Gathering external symbols");

	for (auto compilandIt = m_compiledCompilands.begin(); compilandIt != m_compiledCompilands.end(); ++compilandIt)
	{
		const symbols::ObjPath& objPath = compilandIt->first;
		const std::wstring& wideObjPath = string::ToWideString(objPath);
		const symbols::Compiland* compiland = compilandIt->second;
		const uint32_t compilandUniqueId = symbols::GetCompilandId(compiland, wideObjPath.c_str(), modifiedOrNewObjFiles);

		coff::ObjFile* coffFile = coff::OpenObj(wideObjPath.c_str());
		
		if (coffFile && coffFile->memoryFile)
		{
			coff::ExternalSymbolDB* externalSymbolDb = coff::GatherExternalSymbolDatabase(coffFile, compilandUniqueId);
			coff::CloseObj(coffFile);

			// force the linker to include references to all external functions which we're going to hook,
			// so they're not kicked out by OPT:REF.
			if (externalSymbolDb)
			{
				const size_t symbolCount = externalSymbolDb->symbols.size();
				for (size_t i = 0u; i < symbolCount; ++i)
				{
					const coff::SymbolType::Enum type = externalSymbolDb->types[i];
					if (type == coff::SymbolType::EXTERNAL_FUNCTION)
					{
						const ImmutableString& function = externalSymbolDb->symbols[i];
						linkerOptions += L"/INCLUDE:";
						linkerOptions += string::ToWideString(function);
						linkerOptions += L"\n";
					}
				}

				coff::DestroyDatabase(externalSymbolDb);
			}
			else
			{
				LC_ERROR_USER("External symbol database for COFF %s is invalid", objPath.c_str());
			}
		}
	}

	// generate path for .pdb and .exe file with monotonically increasing counter
	std::wstring pdbPath;
	std::wstring exePath;
	bool isExeOrPdbFileStillThere = false;
	do
	{
		std::wstring patchInstanceStr(L".patch_");
		patchInstanceStr += std::to_wstring(m_patchCounter);

		// depending on the Visual Studio version and project settings, PDB files may be generated incrementally!
		// this means that if the PDB file exists (perhaps from a previous Live++ session), it will contain much more info
		// than necessary and be significantly larger.
		// we therefore delete leftover files from previous sessions to make the linker write completely new outputs.

		// additionally, when unloading live modules, the debugger might still have a lock on the PDB file, even
		// though the corresponding DLL has been unloaded already.
		// in this case, we increase the counter until we find a PDB file that was either deleted successfully or
		// did not exist yet.

		// safety net: in case the PDB path does not exist, we generate the PDB filename from the module name
		const std::wstring originalPdbPath = (m_linkerDB->pdbPath.GetLength() != 0u)
			? string::ToWideString(m_linkerDB->pdbPath)			// we have a valid, original PDB path for the module
			: std::wstring(Filesystem::RemoveExtension(m_moduleName.c_str()).GetString()) + L".pdb";	// create our own PDB path based on the module's name

		isExeOrPdbFileStillThere = false;
		pdbPath = string::Replace(originalPdbPath, L".pdb", patchInstanceStr + std::wstring(L".pdb"));
		exePath = string::Replace(originalPdbPath, L".pdb", patchInstanceStr + std::wstring(L".exe"));
		
		const Filesystem::PathAttributes& pdbAttributes = Filesystem::GetAttributes(pdbPath.c_str());
		const Filesystem::PathAttributes& exeAttributes = Filesystem::GetAttributes(exePath.c_str());

		if (Filesystem::DoesExist(pdbAttributes))
		{
			if (!Filesystem::DeleteIfExists(pdbPath.c_str()))
			{
				// PDB file could not be deleted
				isExeOrPdbFileStillThere = true;
			}
		}
		
		if (Filesystem::DoesExist(exeAttributes))
		{
			if (!Filesystem::DeleteIfExists(exePath.c_str()))
			{
				// EXE file could not be deleted
				isExeOrPdbFileStillThere = true;
			}
		}

		if (isExeOrPdbFileStillThere)
		{
			++m_patchCounter;
		}
	}
	while (isExeOrPdbFileStillThere);

	// path of output .exe file
	linkerOptions += L"/OUT:\"";
	linkerOptions += exePath;
	linkerOptions += L"\" ";

	// path of output .pdb file
	linkerOptions += L"/PDB:\"";
	linkerOptions += pdbPath;
	linkerOptions += L"\"\n";

	// add all needed .obj files to the command line
	{
		for (auto it = neededCompilands.begin(); it != neededCompilands.end(); ++it)
		{
			const symbols::ObjPath& objPath = *it;
			LC_LOG_DEV("Pulling in OBJ file %s", objPath.c_str());

			linkerOptions += L"\"";
			linkerOptions += string::ToWideString(objPath);
			linkerOptions += L"\"\n";
		}
	}

	// add all libraries to the command line
	{
		const size_t count = m_libraryDB->libraries.size();
		for (size_t i = 0u; i < count; ++i)
		{
			const symbols::FilePath& libPath = m_libraryDB->libraries[i];
			LC_LOG_DEV("Pulling in LIB file %s", libPath.c_str());

			linkerOptions += L"\"";
			linkerOptions += string::ToWideString(libPath);
			linkerOptions += L"\"\n";
		}
	}
	// BEGIN EPIC MOD
	{
		const size_t count = additionalLibraries.size();
		for (size_t i = 0u; i < count; ++i)
		{
			const std::wstring& libPath = additionalLibraries[i];
			LC_LOG_DEV("Pulling in additional LIB file %s", libPath.c_str());

			linkerOptions += L"\"";
			linkerOptions += libPath;
			linkerOptions += L"\"\n";
		}
	}
	// END EPIC MOD

	// add UE4-specific helper library to support NatVis visualizers when debugging
	if (appSettings::g_ue4EnableNatVisSupport->GetValue())
	{
		const std::wstring& lppExePath = Process::Current::GetImagePath().GetString();
		std::wstring libPath = Filesystem::GetDirectory(lppExePath.c_str()).GetString();
		libPath += L"\\UE4_NatVisHelper.lib";

		// check whether the .lib file exists and output a warning if not
		{
			const Filesystem::PathAttributes& attributes = Filesystem::GetAttributes(libPath.c_str());
			if (!Filesystem::DoesExist(attributes))
			{
				LC_WARNING_USER("Cannot find UE4 NatVis helper library at %S", libPath.c_str());
			}
		}

		linkerOptions += L"\"";
		linkerOptions += libPath;
		linkerOptions += L"\"\n";

		// force the linker to include the needed symbols
		linkerOptions += L"/INCLUDE:";
		linkerOptions += string::ToWideString(g_ue4NameTableInLIB);
		linkerOptions += L"\n";
		linkerOptions += L"/INCLUDE:";
		linkerOptions += string::ToWideString(g_ue4ObjectArrayInLIB);
		linkerOptions += L"\n";
	}

	// BEGIN EPIC MOD - Support for UE debug visualizers
	linkerOptions += L"\"";
#if LC_64_BIT
	linkerOptions += *FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / L"Extras/NatvisHelpers/Win64/NatvisHelpers.lib");
#else
	linkerOptions += *FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / L"Extras/NatvisHelpers/Win32/NatvisHelpers.lib");
#endif
	linkerOptions += L"\"\n";

	linkerOptions += L"/INCLUDE:InitNatvisHelpers\n";
	// END EPIC MOD

	generateLinkerCommandLine.End();

	telemetry::Scope linkScope("Linking");

	std::wstring linkerPath = GetLinkerPath(m_linkerDB);
	std::wstring linkerWorkingDirectory = (m_linkerDB->workingDirectory.GetLength() != 0u)
		? string::ToWideString(m_linkerDB->workingDirectory)	// we have a valid working directory
		: Filesystem::GetDirectory(linkerPath.c_str()).GetString();						// no valid working directory, take the linker directory instead

	// create a temporary file that acts as a so-called response file for the linker, and contains
	// the whole linker command-line. this is done because the latter can get very long, longer
	// than the limit of 32k characters.
	const Filesystem::Path responseFilePath = Filesystem::GenerateTempFilename();
	Filesystem::CreateFileWithData(responseFilePath.GetString(), linkerOptions.c_str(), linkerOptions.size() * sizeof(wchar_t));

	// BEGIN EPIC MOD
	const Process::Environment linkerEnvironment = compiler::GetEnvironmentFromCache(linkerPath.c_str());
	const void* linkerEnvironmentData = linkerEnvironment.data;

	std::wstring linkerCommandLine;

	// If there are vfs entries, we need to run the linker through ubacli with -vfs options to make sure linker result is the same as when running through UBT
	if (!Filesystem::GetVfsEntries().empty())
	{
		for (auto& entry : Filesystem::GetVfsEntries())
		{
			linkerCommandLine += L" -vfs=\"" + entry.virtualPath + L";" + entry.localPath + L"\"";
		}
		linkerCommandLine += L" local \"";
		linkerCommandLine += linkerPath.c_str();
		linkerCommandLine += '\"';
		linkerPath = *FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / L"Binaries/Win64/UnrealBuildAccelerator/x64/UbaCli.exe");
		linkerWorkingDirectory = Filesystem::Devirtualize(linkerWorkingDirectory);
	}
	else
	{
		linkerCommandLine = Filesystem::GetFilename(linkerPath.c_str()).GetString();
	}

	linkerCommandLine += L" @\"";
	linkerCommandLine += responseFilePath.GetString();
	linkerCommandLine += L"\"";

	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Linking patch...");
	// END EPIC MOD

	Process::Context* linkerProcessContext = Process::Spawn(linkerPath.c_str(), linkerWorkingDirectory.c_str(), linkerCommandLine.c_str(), linkerEnvironmentData, Process::SpawnFlags::REDIRECT_STDOUT | Process::SpawnFlags::NO_WINDOW);
	const unsigned int linkerExitCode = Process::Wait(linkerProcessContext);

	const double linkerTime = linkScope.ReadSeconds();

	// for all the following operations, make sure to restore the original .obj files from their backup location
	for (auto compilandIt = neededCompilands.begin(); compilandIt != neededCompilands.end(); ++compilandIt)
	{
		const symbols::ObjPath& objPath = *compilandIt;
		const std::wstring originalPath(string::ToWideString(objPath));
		const std::wstring bakPath = originalPath + L".bak";

		const Filesystem::PathAttributes& attributes = Filesystem::GetAttributes(bakPath.c_str());
		if (Filesystem::DoesExist(attributes))
		{
			Filesystem::Delete(originalPath.c_str());
			Filesystem::Move(bakPath.c_str(), originalPath.c_str());
		}
		// END EPIC MOD
	}

	const std::wstring linkerProcessStdout = Process::GetStdOutData(linkerProcessContext);
	const wchar_t* linkerOutput = linkerProcessStdout.c_str();

	// send linker output to main executable
	{
		Logging::LogNoFormat<Logging::Channel::USER>(linkerOutput);

		const size_t outputLength = linkerProcessStdout.length();
		if (outputLength != 0u)
		{
			if (updateType != LiveModule::UpdateType::NO_CLIENT_COMMUNICATION)
			{
				for (size_t p = 0u; p < processCount; ++p)
				{
					const DuplexPipe* pipe = processData[p].liveProcess->GetPipe();

					commands::LogOutput cmd {};
					pipe->SendCommandAndWaitForAck(cmd, linkerOutput, (outputLength + 1u) * sizeof(wchar_t));
				}
			}
		}
	}

	Process::Destroy(linkerProcessContext);

	Filesystem::Delete(responseFilePath.GetString());

	linkScope.End();

	if (linkerExitCode != 0u)
	{
		LC_ERROR_USER("Failed to link patch (%.3fs) (Exit code: 0x%X)", linkerTime, linkerExitCode);

		CallCompileErrorHooks(m_moduleCache, updateType);

		return ErrorType::LINK_ERROR;
	}

	LC_SUCCESS_USER("Successfully linked patch (%.3fs)", linkerTime);

	// linking was successful, clear the compiled compilands' status and bump the patch version for the next patch
	for (auto compilandIt = m_compiledCompilands.begin(); compilandIt != m_compiledCompilands.end(); ++compilandIt)
	{
		symbols::Compiland* compiland = compilandIt->second;
		if (compiland)
		{
			symbols::ClearCompilandAsRecompiled(compiland);
		}
	}
	++m_patchCounter;

	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Preparing patch image...");
	// END EPIC MOD

	// try to load patch image
	executable::Image* image = executable::OpenImage(exePath.c_str(), Filesystem::OpenMode::READ_WRITE);
	if (!image)
	{
		LC_ERROR_USER("Cannot load patch executable %S", exePath.c_str());

		// clear the set for the next update
		m_modifiedFiles.clear();
		m_compiledCompilands.clear();

		CallCompileErrorHooks(m_moduleCache, updateType);

		return ErrorType::LOAD_PATCH_ERROR;
	}

	executable::ImageSectionDB* imageSections = executable::GatherImageSectionDB(image);

	// before loading the DLL, disable its entry point so we can load it without initializing anything.
	// we first want to reconstruct symbol information and patch dynamic initializers, only then do
	// we want to call the entry point.
	LC_LOG_DEV("Patching entry point");

	ExecutablePatcher executablePatcher(image, imageSections);
	const uint32_t entryPointRva = executablePatcher.DisableEntryPointInImage(image, imageSections);
	executable::DestroyImageSectionDB(imageSections);

	// note that the image needs to be closed before it can be loaded into a process
	const uint32_t patchImageSize = executable::GetSize(image);
	executable::CloseImage(image);

	// the patch's entry point is disabled. tell the processes to load the patch
	LC_LOG_DEV("Loading code into process");

	types::vector<void*> loadedPatches;
	{
#if LC_64_BIT
		executable::PreferredBase currentPreferredImageBase = 0u;
#endif

		for (size_t i = 0u; i < processCount; ++i)
		{
			const PerProcessData& data = processData[i];

			commands::LoadPatch cmd = {};
			wcscpy_s(cmd.path, Filesystem::Devirtualize(exePath).c_str());

#if LC_64_BIT
			// before doing anything further, we need to ensure that the patch can be loaded into the address space at a suitable location.
			// for 64-bit applications, this means that the patch must lie in a +/-2GB range of the main executable.
			// 32-bit executables can reach the whole address space due to modulo addressing.
			LC_LOG_DEV("Scanning memory for suitable patch location (PID: %d)", data.liveProcess->GetProcessId());

			// disable the main process before scanning its memory to ensure that no operation allocates/frees virtual memory concurrently
			Process::Suspend(data.liveProcess->GetProcessHandle());

			// free our page reservations in the +-2GB range of the main module
			data.liveProcess->FreeVirtualMemoryPages(data.originalModuleBase);

			const executable::PreferredBase preferredImageBase = FindPreferredImageBase(patchImageSize, data.liveProcess->GetProcessId(), data.liveProcess->GetProcessHandle(), data.originalModuleBase);

			// rather than constantly copying images for processes, check whether they need to be rebased to a different address for this process
			const bool imageNeedsToBeRebased = (currentPreferredImageBase != preferredImageBase);
			const bool imageNeedsToBeCopied = (currentPreferredImageBase == 0u)
				? false						// this is the first image, so no copying needed
				: imageNeedsToBeRebased;	// image has been rebased and now potentially needs to be rebased to a different address

			std::wstring rebasedExePath = exePath;
			if (imageNeedsToBeCopied)
			{
				// this image needs to be copied. create a new name based on the process ID, which must be unique
				rebasedExePath = string::Replace(rebasedExePath, L".exe", std::wstring(L"_PID_") + std::to_wstring(+data.liveProcess->GetProcessId()) + L".exe");
				Filesystem::Copy(exePath.c_str(), rebasedExePath.c_str());
				wcscpy_s(cmd.path, Filesystem::Devirtualize(rebasedExePath).c_str());
			}
			
			if (imageNeedsToBeRebased)
			{
				// rebase the patch image to its preferred base address
				executable::Image* rebasedImage = executable::OpenImage(rebasedExePath.c_str(), Filesystem::OpenMode::READ_WRITE);
				LC_LOG_DEV("Rebasing patch executable to image base 0x%" PRIX64 " (PID: %d)", preferredImageBase, data.liveProcess->GetProcessId());
				executable::RebaseImage(rebasedImage, preferredImageBase);
				executable::CloseImage(rebasedImage);

				currentPreferredImageBase = preferredImageBase;
			}

			// resume the main process so that it can respond to our command. if we're *really* unlucky, a concurrent operation
			// will allocate virtual memory at the patch's preferred image base, possibly rendering the patch unusable because
			// it cannot be loaded.
			// the chances of that happening are *very* rare though, and we can always load the next patch then.
			Process::Resume(data.liveProcess->GetProcessHandle());
#endif

			data.liveProcess->GetPipe()->SendCommandAndWaitForAck(cmd, nullptr, 0u);

			// receive command with patch info
			CommandMap commandMap;
			commandMap.RegisterAction<actions::LoadPatchInfo>();
			commandMap.HandleCommands(data.liveProcess->GetPipe(), &loadedPatches);
		}
	}

#if LC_64_BIT
	// immediately reserve pages in the +-2GB range of the main module again
	for (size_t i = 0u; i < processCount; ++i)
	{
		const PerProcessData& data = processData[i];
		data.liveProcess->ReserveVirtualMemoryPages(data.originalModuleBase);
	}
#endif

	if (processCount != loadedPatches.size())
	{
		// communication with the client broke down while trying to load the patch, bail out
		LC_ERROR_USER("Client communication broken, patch could not be loaded.");

		// clear the set for the next update
		m_modifiedFiles.clear();
		m_compiledCompilands.clear();

		CallCompileErrorHooks(m_moduleCache, updateType);

		return ErrorType::LOAD_PATCH_ERROR;
	}

	bool patchesLoadedSuccessfully = true;
	for (size_t i = 0u; i < processCount; ++i)
	{
		const PerProcessData& data = processData[i];
		void* patchBase = loadedPatches[i];
		LC_LOG_DEV("Loaded patch at 0x%p (PID: %d)", patchBase, data.liveProcess->GetProcessId());

		patchesLoadedSuccessfully = CheckPatchAddressValidity(data.originalModuleBase, patchBase, data.liveProcess->GetProcessHandle());
		if (!patchesLoadedSuccessfully)
		{
			break;
		}
	}

	if (!patchesLoadedSuccessfully)
	{
		LC_ERROR_USER("Patch could not be activated.");

		// one of the patches cannot be used, unload all of them and bail out
		for (size_t i = 0u; i < processCount; ++i)
		{
			const DuplexPipe* clientPipe = processData[i].liveProcess->GetPipe();
			clientPipe->SendCommandAndWaitForAck(commands::UnloadPatch { static_cast<HMODULE>(loadedPatches[i]) }, nullptr, 0u);
		}

		// clear the set for the next update
		m_modifiedFiles.clear();
		m_compiledCompilands.clear();

		CallCompileErrorHooks(m_moduleCache, updateType);

		return ErrorType::ACTIVATE_PATCH_ERROR;
	}



	// enter sync point in all processes
	if (updateType != LiveModule::UpdateType::NO_CLIENT_COMMUNICATION)
	{
		for (size_t p = 0u; p < processCount; ++p)
		{
			const PerProcessData& data = processData[p];
			data.liveProcess->GetPipe()->SendCommandAndWaitForAck(commands::EnterSyncPoint {}, nullptr, 0u);
		}
	}


	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Loading patch PDB...");
	// END EPIC MOD

	LC_LOG_DEV("Loading patch PDB");

	telemetry::Scope loadPatchPDBScope("Loading PDB database");

	symbols::Provider* patchSymbolProvider = symbols::OpenEXE(Filesystem::Devirtualize(exePath).c_str(), symbols::OpenOptions::ACCUMULATE_SIZE);
	symbols::DiaCompilandDB* patch_diaCompilandDb = symbols::GatherDiaCompilands(patchSymbolProvider);
	IDiaSymbol* patch_linkerSymbol = symbols::FindLinkerSymbol(patch_diaCompilandDb);


	auto taskRootPatchLoading = scheduler::CreateEmptyTask();

	// similar to the initial reading of PDB files, we open separate providers to enable
	// multi-threaded loading of PDB data.
	auto taskPatch_symbolDB = scheduler::CreateTask(taskRootPatchLoading, [patchSymbolProvider]()
	{
		return symbols::GatherSymbols(patchSymbolProvider);
	});
	scheduler::RunTask(taskPatch_symbolDB);


	auto taskPatch_contributionDB = scheduler::CreateTask(taskRootPatchLoading, [exePath]()
	{
		symbols::Provider* localProvider = symbols::OpenEXE(Filesystem::Devirtualize(exePath).c_str(), symbols::OpenOptions::NONE);
		symbols::DiaCompilandDB* localDiaCompilandDb = symbols::GatherDiaCompilands(localProvider);

		auto db = symbols::GatherContributions(localProvider);

		symbols::DestroyDiaCompilandDB(localDiaCompilandDb);
		symbols::Close(localProvider);

		return db;
	});
	scheduler::RunTask(taskPatch_contributionDB);


	// note that we only gather symbols from .obj contained in the new patch executable.
	// therefore we need to extract its compiland database as well, and cannot use the one from
	// the original executable.
	auto taskPatch_compilandDB = scheduler::CreateTask(taskRootPatchLoading, [this, exePath]()
	{
		symbols::Provider* localProvider = symbols::OpenEXE(Filesystem::Devirtualize(exePath).c_str(), symbols::OpenOptions::NONE);
		symbols::DiaCompilandDB* localDiaCompilandDb = symbols::GatherDiaCompilands(localProvider);

		uint32_t options = 0u;
		if (appSettings::g_enableDevLogCompilands->GetValue())
		{
			options |= symbols::CompilandOptions::GENERATE_LOGS;
		}
		if (appSettings::g_compilerForcePchPdbs->GetValue())
		{
			options |= symbols::CompilandOptions::FORCE_PCH_PDBS;
		}

		// in case the user wants to use a completely external build system, we track .objs only
		if (m_runMode == RunMode::EXTERNAL_BUILD_SYSTEM)
		{
			options |= symbols::CompilandOptions::TRACK_OBJ_ONLY;
		}

		// TODO: what if somebody uses FASTBuild? how to get the new dependencies?
		// does this step even have to be done? seems like this might not even be needed in ANY CASE.
		// -> this has to be done to figure out new #include files for existing compilands!
		// => but it should be split in two: figuring out compilands, and figuring out include files.
		symbols::CompilandDB* db = symbols::GatherCompilands(localProvider, localDiaCompilandDb, GetAmalgamatedSplitThreshold(), options);

		symbols::DestroyDiaCompilandDB(localDiaCompilandDb);
		symbols::Close(localProvider);

		return db;
	});
	scheduler::RunTask(taskPatch_compilandDB);


	auto taskPatch_thunkDB = scheduler::CreateTask(taskRootPatchLoading, [patch_linkerSymbol]()
	{
		return symbols::GatherThunks(patch_linkerSymbol);
	});
	scheduler::RunTask(taskPatch_thunkDB);


	auto taskPatch_imageSectionDB = scheduler::CreateTask(taskRootPatchLoading, [patch_linkerSymbol]()
	{
		return symbols::GatherImageSections(patch_linkerSymbol);
	});
	scheduler::RunTask(taskPatch_imageSectionDB);


	// ensure asynchronous operations have finished
	scheduler::RunTask(taskRootPatchLoading);
	scheduler::WaitForTask(taskRootPatchLoading);

	// fetch results
	symbols::SymbolDB* patch_symbolDB = taskPatch_symbolDB->GetResult();
	symbols::ContributionDB* patch_contributionDB = taskPatch_contributionDB->GetResult();
	symbols::CompilandDB* patch_compilandDB = taskPatch_compilandDB->GetResult();
	symbols::ThunkDB* patch_thunkDB = taskPatch_thunkDB->GetResult();
	symbols::ImageSectionDB* patch_imageSectionDB = taskPatch_imageSectionDB->GetResult();
	
	symbols::DestroyLinkerSymbol(patch_linkerSymbol);

	// destroy tasks
	scheduler::DestroyTask(taskRootPatchLoading);
	scheduler::DestroyTask(taskPatch_symbolDB);
	scheduler::DestroyTask(taskPatch_contributionDB);
	scheduler::DestroyTask(taskPatch_compilandDB);
	scheduler::DestroyTask(taskPatch_thunkDB);
	scheduler::DestroyTask(taskPatch_imageSectionDB);

	symbols::FinalizeContributions(patch_compilandDB, patch_contributionDB);

	LC_LOG_DEV("Updating cache of external symbols");

	// update the cache that stores all external/public symbols for each compiland
	{
		// clear the cache for all files that were compiled, but not the ones that were pulled in for linking only
		// without them having changed (e.g. a PCH).
		for (auto it : m_compiledCompilands)
		{
			const symbols::ObjPath& objPath = it.first;
			m_externalSymbolsPerCompilandCache.erase(objPath);
		}

		// we only know public symbols at this point, so walk all of them and find their corresponding contribution.
		// there are two ways to go about this:
		// 1) walk all symbols, find their contribution
		// 2) walk all contributions, find their symbol
		// this needs to be done using 1), otherwise some external symbols cannot be found because their contributions
		// have been merged.
		for (auto it : patch_symbolDB->symbolsByRva)
		{
			const uint32_t rva = it.first;
			const symbols::Symbol* symbol = it.second;
			const symbols::Contribution* contribution = symbols::FindContributionByRVA(patch_contributionDB, rva);
			if (contribution)
			{
				const ImmutableString& compilandName = symbols::GetContributionCompilandName(patch_contributionDB, contribution);
				m_externalSymbolsPerCompilandCache[compilandName].push_back(symbol);
			}
		}
	}

	loadPatchPDBScope.End();



	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Updating COFF cache...");
	// END EPIC MOD

	{
		LC_LOG_DEV("Updating COFF cache for new patch compilands");

		// update the COFF cache for new patch compilands.
		// there may be files for which we don't have a database yet, even though we updated the database for all compiled files.
		// this can happen when a new .obj that is part of a library is linked in for the first time.
		types::vector<symbols::ObjPath> updatedCoffs = UpdateCoffCache(patch_compilandDB->compilands, m_coffCache, CacheUpdate::NON_EXISTANT, modifiedOrNewObjFiles);


		// similarly, reconstruct symbols and dynamic initializers for new .obj that have been pulled in for the first time.
		// otherwise, dynamic initializers from these files will never be reconstructed, which would inevitably lead to
		// symbols being constructed twice.
		LC_LOG_DEV("Reconstructing symbols from original OBJ");
		{
			LC_LOG_INDENT_DEV;

			executable::Image* originalImage = executable::OpenImage(m_moduleName.c_str(), Filesystem::OpenMode::READ);
			executable::ImageSectionDB* originalImageSections = executable::GatherImageSectionDB(originalImage);

			types::StringSet noSymbolsToIgnore;

			symbols::Provider* provider = symbols::OpenEXE(m_moduleName.c_str(), symbols::OpenOptions::NONE);
			{
				symbols::GatherDynamicInitializers(provider, originalImage, originalImageSections, m_imageSectionDB, m_contributionDB, m_compilandDB, m_coffCache, m_symbolDB);

				const size_t count = updatedCoffs.size();
				for (size_t i = 0u; i < count; ++i)
				{
					const symbols::ObjPath& objPath = updatedCoffs[i];

					if (m_reconstructedCompilands.find(objPath) == m_reconstructedCompilands.end())
					{
						// no entry yet, must be reconstructed
						LC_LOG_DEV("COFF %s not in cache yet", objPath.c_str());

						const coff::CoffDB* database = m_coffCache->Lookup(objPath);
						if (!database)
						{
							LC_ERROR_USER("COFF database for compiland %s is invalid (lazy reconstruct)", objPath.c_str());
							continue;
						}

						m_reconstructedCompilands.emplace(objPath);

						symbols::ReconstructFromExecutableCoff(provider, originalImage, originalImageSections, database, noSymbolsToIgnore, objPath, m_contributionDB, m_thunkDB, m_imageSectionDB, m_symbolDB);
					}
				}
			}

			symbols::Close(provider);

			executable::DestroyImageSectionDB(originalImageSections);
			executable::CloseImage(originalImage);
		}
	}

	// new symbols have been reconstructed, so transform the symbols containing anonymous namespace names
	symbols::TransformAnonymousNamespaceSymbols(m_symbolDB, m_contributionDB, m_compilandDB, modifiedOrNewObjFiles);


	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Reconstructing patch symbols...");
	// END EPIC MOD

	// reconstruct symbols for all compilands that are part of the new patch executable
	executable::Image* patchImage = executable::OpenImage(exePath.c_str(), Filesystem::OpenMode::READ);
	executable::ImageSectionDB* patchImageSections = executable::GatherImageSectionDB(patchImage);

	// gather the dynamic initializers and remaining symbols by walking the module
	const symbols::DynamicInitializerDB initializerDb = symbols::GatherDynamicInitializers(patchSymbolProvider, patchImage, patchImageSections, patch_imageSectionDB, patch_contributionDB, patch_compilandDB, m_coffCache, patch_symbolDB);
	{
		LC_LOG_DEV("Reconstructing patch symbols from OBJ");
		LC_LOG_INDENT_DEV;

		for (auto it = patch_compilandDB->compilands.begin(); it != patch_compilandDB->compilands.end(); ++it)
		{
			const symbols::ObjPath& patchObjPath = it->first;
			const coff::CoffDB* database = m_coffCache->Lookup(patchObjPath);
			if (!database)
			{
				LC_ERROR_USER("COFF database for compiland %s is invalid", patchObjPath.c_str());
				continue;
			}

			symbols::ReconstructFromExecutableCoff(patchSymbolProvider, patchImage, patchImageSections,
				database, strippedSymbolsPerCompiland[patchObjPath], patchObjPath, patch_contributionDB, patch_thunkDB, patch_imageSectionDB, patch_symbolDB);
		}

		// merge compilands and dependencies with existing ones to account for new files and e.g. new #includes.
		symbols::MergeCompilandsAndDependencies(m_compilandDB, patch_compilandDB);

		// update directory cache for new compilands
		UpdateDirectoryCache(directoryCache);

		// AMALGAMATION
		// for files that are part of an amalgamation, we write a new database in case the file compiled successfully.
		// this ensures that files split once don't need to be recompiled again in case nothing changed, even when
		// restarting a new Live++ session.
		// when a file fails to compile, no database exists on disk, so the file will be recompiled next time automatically.
		for (auto it = patch_compilandDB->compilands.begin(); it != patch_compilandDB->compilands.end(); ++it)
		{
			const symbols::ObjPath& patchObjPath = it->first;
			const bool isPartOfAmalgamation = amalgamation::IsPartOfAmalgamation(patchObjPath.c_str());
			if (isPartOfAmalgamation)
			{
				auto originalIt = m_compilandDB->compilands.find(patchObjPath);
				if (originalIt != m_compilandDB->compilands.end())
				{
					// this compiland had its source files updated, write a database
					const symbols::ObjPath& originalObjPath = originalIt->first;
					const symbols::Compiland* compiland = originalIt->second;
					amalgamation::WriteDatabase(originalObjPath, GetCompilerPath(compiland), compiland, appSettings::g_compilerOptions->GetValue());
				}
			}
		}

		symbols::DestroyDiaCompilandDB(patch_diaCompilandDb);
		patch_diaCompilandDb = nullptr;
	}

	executable::DestroyImageSectionDB(patchImageSections);
	executable::CloseImage(patchImage);

	// BEGIN EPIC MOD
	uint64_t patchLastModicationTime = patchSymbolProvider->lastModificationTime;
	// END EPIC MOD
	symbols::Close(patchSymbolProvider);


	// now that everything is loaded and reconstructed in the patch, transform the symbols containing anonymous namespace names.
	// note that we need to used the merged main compiland database here.
	TransformAnonymousNamespaceSymbols(patch_symbolDB, patch_contributionDB, m_compilandDB, modifiedOrNewObjFiles);


	// store the new databases into the module cache
	ModulePatch* compiledModulePatch = nullptr;
	// BEGIN EPIC MOD
	const size_t token = m_moduleCache->Insert(patch_symbolDB, patch_contributionDB, patch_compilandDB, patch_thunkDB, patch_imageSectionDB, patchLastModicationTime);
	// END EPIC MOD
	{
		for (size_t p = 0u; p < processCount; ++p)
		{
			const PerProcessData& data = processData[p];
			m_moduleCache->RegisterProcess(token, data.liveProcess, loadedPatches[p]);
		}

		// now that the patch has been loaded, store a new module patch and record the data needed for
		// loading it into another process at a later time.
		compiledModulePatch = new ModulePatch(exePath, pdbPath, token);
		m_compiledModulePatches.push_back(compiledModulePatch);
	}


	// record entry point code for patching the entry point when loading this image into a different process later
	{
		compiledModulePatch->RegisterEntryPointCode(executablePatcher.GetEntryPointCode());
	}


	{
		// pre-patch hooks must not be called on the current executable because the hooks want to use the old memory layout of
		// data structures.
		if (updateType != LiveModule::UpdateType::NO_CLIENT_COMMUNICATION)
		{
			const ModuleCache::FindHookData& hookData = m_moduleCache->FindHooksInSectionBackwards(token, ImmutableString(LPP_PREPATCH_SECTION));
			if ((hookData.firstRva != 0u) && (hookData.lastRva != 0u))
			{
				const size_t count = hookData.data->processes.size();
				for (size_t p = 0u; p < count; ++p)
				{
					const ModuleCache::ProcessData& hookProcessData = hookData.data->processes[p];

					const Process::Id pid = hookProcessData.processId;
					void* moduleBase = hookProcessData.moduleBase;
					const DuplexPipe* pipe = hookProcessData.pipe;

					LC_LOG_USER("Calling pre-patch hooks (PID: %d)", +pid);
					pipe->SendCommandAndWaitForAck(MakeCallHooksCommand(hook::Type::PREPATCH, moduleBase, hookData), nullptr, 0u);
				}

				compiledModulePatch->RegisterPrePatchHooks(hookData.data->index, hookData.firstRva, hookData.lastRva);
			}
		}
	}



	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Patching relocations...");
	// END EPIC MOD

	// BEGIN EPIC MOD
	auto patchRelocationsPreEntryPoint = [&](bool forceBackwards)
	{
	// END EPIC MOD
		LC_LOG_DEV("Patching relocations before calling entry point");

		// walk all relocations in the .OBJ files, find their current locations in the .exe,
		// and patch the relocations to point to the original symbols in the original .exe.
		// we need to patch relocations *before* calling the DLL entry point, because global
		// initializer code might refer to symbols that have been stripped by us.
		// note that we only patch relocations to data symbols at this time, because functions haven't been
		// hooked yet, and we need to ensure that dynamic initializers end up using new code paths (if available), while
		// still referring to existing data symbols.
		{
			telemetry::Scope patchingRelocationsScope("Patching relocations");

			uint32_t relocationsHandledCount = 0u;
			size_t relocationsCount = 0u;

			for (auto it = patch_compilandDB->compilands.begin(); it != patch_compilandDB->compilands.end(); ++it)
			{
				const symbols::ObjPath& objPath = it->first;

				LC_LOG_DEV("Patching relocations for file %s", objPath.c_str());
				LC_LOG_INDENT_DEV;

				const coff::CoffDB* coffDb = m_coffCache->Lookup(objPath);
				if (!coffDb)
				{
					LC_ERROR_USER("Could not find COFF database for file %s", objPath.c_str());
					continue;
				}

				const std::wstring wideObjPath = string::ToWideString(objPath);

				// note that we need to take the original compiland here, to make sure that single split-files get assigned
				// the same unique ID as the corresponding amalgamated file.
				symbols::Compiland* compiland = symbols::FindCompiland(m_compilandDB, objPath);
				const uint32_t compilandUniqueId = symbols::GetCompilandId(compiland, wideObjPath.c_str(), modifiedOrNewObjFiles);

				const types::StringSet& strippedSymbols = strippedSymbolsPerCompiland[objPath];
				const types::StringSet& forceStrippedSymbols = forceStrippedSymbolsPerCompiland[objPath];

				const size_t symbolCount = coffDb->symbols.size();
				for (size_t i = 0u; i < symbolCount; ++i)
				{
					const coff::Symbol* symbol = coffDb->symbols[i];
					relocationsCount += symbol->relocations.size();

					// check if the patch knows this symbol.
					// if not, it has probably been stripped and there is no need to walk all its relocations.
					const ImmutableString& symbolName = symbols::TransformAnonymousNamespacePattern(coff::GetSymbolName(coffDb, symbol), compilandUniqueId);
					const symbols::Symbol* realSymbol = symbols::FindSymbolByName(patch_symbolDB, symbolName);
					if (!realSymbol)
					{
						// this symbol has been stripped from the executable.
						// in optimized builds, the compiler will sometimes e.g. leave a static function in an OBJ file,
						// which will be kicked out by the linker.
						continue;
					}

					// before patching relocations, check whether the symbol which relocations we want to patch originated from
					// a compiland that is the same as the file we're working on.
					// this might not be the case, especially when using static libraries, COMDATs, and compilands that use the
					// same inline function but have slightly different compiler options (/hotpatch vs. no /hotpatch, e.g.
					// __local_stdio_printf_options in the main module vs. in the dynamic runtime)
					const symbols::Contribution* originalContribution = symbols::FindContributionByRVA(patch_contributionDB, realSymbol->rva);
					if (originalContribution)
					{
						const ImmutableString& compilandName = symbols::GetContributionCompilandName(patch_contributionDB, originalContribution);
						if (compilandName != objPath)
						{
							LC_LOG_DEV("Ignoring relocations for symbol %s in file %s (original compiland: %s)",
								symbolName.c_str(), objPath.c_str(), compilandName.c_str());
							continue;
						}
					}

					const size_t relocationCount = symbol->relocations.size();
					for (size_t j = 0u; j < relocationCount; ++j)
					{
						const coff::Relocation* relocation = symbol->relocations[j];
						if (relocation->dstIsSection)
						{
							// don't patch relocations to sections
							continue;
						}

						// ignore relocations to symbols in .msvcjmc (MSVC JustMyCode) sections
						if (relocation->dstSectionIndex >= 0)
						{
							const uint32_t index = static_cast<uint32_t>(relocation->dstSectionIndex);
							const coff::Section& section = coffDb->sections[index];
							if (coff::IsMSVCJustMyCodeSection(section.name.c_str()))
							{
								LC_LOG_DEV("Ignoring relocation to symbol in section %s", section.name.c_str());
								continue;
							}
						}

						ImmutableString dstSymbolName = symbols::TransformAnonymousNamespacePattern(GetRelocationDstSymbolName(coffDb, relocation), compilandUniqueId);
						const bool refersToDataSymbol = !coff::IsFunctionSymbol(coff::GetRelocationDstSymbolType(relocation));
						const bool refersToStrippedSymbol = (strippedSymbols.find(dstSymbolName) != strippedSymbols.end());
						if (refersToDataSymbol || refersToStrippedSymbol)
						{
							// BEGIN EPIC MOD
							const relocations::Record& relocationRecord = relocations::PatchRelocation(relocation, coffDb, forceStrippedSymbols, m_moduleCache, symbolName, dstSymbolName, realSymbol, token, &loadedPatches[0], forceBackwards);
							if (!forceBackwards && relocations::IsValidRecord(relocationRecord))
							{
								compiledModulePatch->RegisterPreEntryPointRelocation(relocationRecord);
							}
							// END EPIC MOD

							++relocationsHandledCount;
						}
					}
				}
			}

			LC_LOG_TELEMETRY("Handled %d of %d relocations in %.3fms (avg: %.3fus)", relocationsHandledCount, relocationsCount,
				patchingRelocationsScope.ReadMilliSeconds(), (patchingRelocationsScope.ReadMicroSeconds() / relocationsHandledCount));
		}
	// BEGIN EPIC MOD
	};
	// END EPIC MOD

	// BEGIN EPIC MOD
	patchRelocationsPreEntryPoint(enableReinstancingFlow);
	// END EPIC MOD

	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Patching public functions in lib compilands...");
	// END EPIC MOD

	LC_LOG_DEV("Patching public functions in lib compilands");
	{
		telemetry::Scope patchingFunctionsScope("Patching functions");

		uint32_t functionsPatchedCount = 0u;
		size_t functionsCount = patch_symbolDB->patchableFunctionSymbols.size();

		// functions in lib compilands cannot have changed, per definition. but there can be code linked in from libraries
		// that calls these functions, therefore they need to be patched to their original function, otherwise
		// there would be functions working on new data. this can be the case when linking with static libraries we don't have
		// the .obj files for.
		LC_LOG_INDENT_DEV;

		for (auto it : patch_symbolDB->patchableFunctionSymbols)
		{
			const symbols::Symbol* symbol = it;
			const ImmutableString& functionName = symbol->name;

			// never patch the entry point, we need to call that later on
			if (symbol->rva == entryPointRva)
			{
				LC_LOG_DEV("Ignoring entry point function %s", functionName.c_str());
				continue;
			}

			// note that when patching new functions to original ones, the same rules as for patching relocations apply,
			// i.e. not all functions should be patched.
			if (symbols::IsExceptionRelatedSymbol(functionName))
			{
				LC_LOG_DEV("Ignoring exception-related function %s", functionName.c_str());
				continue;
			}
			else if (symbols::IsRuntimeCheckRelatedSymbol(functionName))
			{
				LC_LOG_DEV("Ignoring runtime check function %s", functionName.c_str());
				continue;
			}
			else if (symbols::IsSdlCheckRelatedSymbol(functionName))
			{
				LC_LOG_DEV("Ignoring SDL check function %s", functionName.c_str());
				continue;
			}

			// check whether the function is at least 5 bytes long to consider it for patching
			const symbols::Contribution* contribution = symbols::FindContributionByRVA(patch_contributionDB, symbol->rva);
			if (!contribution)
			{
				LC_ERROR_DEV("Ignoring function %s because its contribution cannot be found", functionName.c_str());
				continue;
			}

			if (contribution->size < 5u)
			{
				LC_LOG_DEV("Ignoring function %s that is only %d bytes long", functionName.c_str(), contribution->size);
				continue;
			}

			const ModuleCache::FindSymbolData& originalData = m_moduleCache->FindSymbolByName(token, functionName);
			if (!originalData.symbol)
			{
				LC_LOG_DEV("Ignoring new function %s", functionName.c_str());
				continue;
			}

			// if the original function to be patched did not come from a compiland, it cannot possibly have changed and
			// therefore needs to be patched.
			const symbols::Contribution* originalContribution = symbols::FindContributionByRVA(originalData.data->contributionDb, originalData.symbol->rva);
			if (originalContribution)
			{
				const ImmutableString& compilandName = symbols::GetContributionCompilandName(originalData.data->contributionDb, originalContribution);
				const symbols::Compiland* originalCompiland = symbols::FindCompiland(originalData.data->compilandDb, compilandName);
				if (!originalCompiland)
				{
					const size_t moduleProcessCount = originalData.data->processes.size();
					for (size_t p = 0u; p < moduleProcessCount; ++p)
					{
						++functionsPatchedCount;

						const Process::Id pid = originalData.data->processes[p].processId;
						void* moduleBase = originalData.data->processes[p].moduleBase;
						Process::Handle processHandle = originalData.data->processes[p].processHandle;

						const symbols::Symbol* patchSymbol = symbol;

						char* srcAddress = pointer::Offset<char*>(loadedPatches[p], patchSymbol->rva);
						char* destAddress = pointer::Offset<char*>(moduleBase, originalData.symbol->rva);

						LC_LOG_DEV("Patching library function %s at 0x%p (0x%X) (PID: %d)", functionName.c_str(), moduleBase, patchSymbol->rva, +pid);

						const functions::LibraryRecord& record = functions::PatchLibraryFunction(srcAddress, destAddress,
							patchSymbol->rva, originalData.symbol->rva,
							contribution, processHandle, originalData.data->index);
						if (functions::IsValidRecord(record))
						{
							compiledModulePatch->RegisterLibraryFunctionPatch(record);
						}
					}
				}
			}
		}

		LC_LOG_TELEMETRY("Patched %d of %d functions in %.3fms (avg: %.3fus)", functionsPatchedCount, functionsCount,
			patchingFunctionsScope.ReadMilliSeconds(), (patchingFunctionsScope.ReadMicroSeconds() / functionsPatchedCount));
	}



	// now that the .dll is loaded and symbols have been relocated, finally patch the dynamic initializers
	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Patching dynamic initializers...");
	// END EPIC MOD
	{
		const size_t count = initializerDb.dynamicInitializers.size();

		LC_LOG_DEV("Scanning %d dynamic initializer candidates", count);
		LC_LOG_INDENT_DEV;

		for (size_t i = 0u; i < count; ++i)
		{
			const symbols::Symbol* initializerSymbol = initializerDb.dynamicInitializers[i];
			const ImmutableString& name = initializerSymbol->name;

			// BEGIN EPIC MOD
			if (symbols::IsUEInitializerSymbol(name))
			{
				LC_LOG_DEV("Skipping dynamic initializer symbol %s", name.c_str());
				continue;
			}
			// END EPIC MOD

			const ModuleCache::FindSymbolData& originalData = m_moduleCache->FindSymbolByName(token, name);
			if (originalData.symbol)
			{
				// this initializer has been called already, overwrite it in all processes
				const uint32_t rva = initializerSymbol->rva;

				for (size_t p = 0u; p < processCount; ++p)
				{
					const PerProcessData& data = processData[p];

					LC_LOG_DEV("Patching dynamic initializer symbol %s at RVA 0x%X (PID: %d)", name.c_str(), rva, data.liveProcess->GetProcessId());

					void* initializerAddress = pointer::Offset<void*>(loadedPatches[p], rva);
					Process::WriteProcessMemory(data.liveProcess->GetProcessHandle(), initializerAddress, nullptr);
				}

				compiledModulePatch->RegisterPatchedDynamicInitializer(rva);
			}
			else
			{
				LC_WARNING_DEV("Cannot find symbol %s in original executable", name.c_str());
			}
		}

		// on Clang, CRT sections that store individual dynamic initializers don't expose any symbols in the COFF,
		// but only relocations to the initializers. therefore, the initializer database is empty.
		// however, the dynamic initializers should have been reconstructed at this point, so we know their RVA.
		// in this case, we scan all symbols from __xc_a to __xc_z, check to which symbol RVA they refer, fetch the symbol at this RVA,
		// and check if this symbol is already known to us. if so, we need to disable the corresponding initializer.
		if (usesClang)
		{
			const symbols::Symbol* firstInitializerSymbol = symbols::FindSymbolByName(patch_symbolDB, ImmutableString(LC_IDENTIFIER("__xc_a")));
			const symbols::Symbol* lastInitializerSymbol = symbols::FindSymbolByName(patch_symbolDB, ImmutableString(LC_IDENTIFIER("__xc_z")));
			if (firstInitializerSymbol && lastInitializerSymbol)
			{
				executable::Image* thisImage = executable::OpenImage(exePath.c_str(), Filesystem::OpenMode::READ);
				executable::ImageSectionDB* sectionDb = executable::GatherImageSectionDB(thisImage);

				// this is defined in the CRT, which also defines all the special sections
				// TODO: move this to somewhere else, e.g. Windows Internals
				typedef _PVFV DynamicInitializer;

				// the first symbol is always __xc_a, which we are not interested in.
				// similarly, the last symbol is always __xc_z, which we are also not interested in.
				const uint32_t firstRva = firstInitializerSymbol->rva + sizeof(DynamicInitializer);
				const uint32_t lastRva = lastInitializerSymbol->rva - sizeof(DynamicInitializer);

				for (uint32_t rva = firstRva; rva <= lastRva; rva += sizeof(DynamicInitializer))
				{
#if LC_64_BIT
					const uint64_t initializerAddress = executable::ReadFromImage<uint64_t>(thisImage, sectionDb, rva);
#else
					const uint32_t initializerAddress = executable::ReadFromImage<uint32_t>(thisImage, sectionDb, rva);
#endif

					// the relocations from "$initializer$" to a dynamic initializer are always absolute, so its
					// easy to reconstruct the dynamic initializer's RVA.
					const uint32_t dynamicInitializerRva = static_cast<uint32_t>(initializerAddress - executable::GetPreferredBase(thisImage));
					const symbols::Symbol* symbol = symbols::FindSymbolByRVA(patch_symbolDB, dynamicInitializerRva);
					if (symbol)
					{
						// check if we already know a symbol with this name
						const ModuleCache::FindSymbolData& findData = m_moduleCache->FindSymbolByName(token, symbol->name);

						const bool symbolFound = (findData.symbol != nullptr);

						// special case for amalgamated files: Clang assigns dynamic initializer functions based on filenames.
						// for amalgamated files, the initializers will already have executed, but the initializers in the
						// split files will get a different name.
						const bool isClangDynamicInitializer = (string::Contains(symbol->name.c_str(), "_GLOBAL__sub_I_"));
						if (symbolFound || isClangDynamicInitializer)
						{
							// this symbol already exists, so it must have been called during initialization earlier.
							// patch the dynamic initializer in all processes.
							for (size_t p = 0u; p < processCount; ++p)
							{
								const PerProcessData& data = processData[p];

								void* address = pointer::Offset<void*>(loadedPatches[p], rva);
								Process::WriteProcessMemory(data.liveProcess->GetProcessHandle(), address, nullptr);
							}

							compiledModulePatch->RegisterPatchedDynamicInitializer(rva);
						}
					}
				}

				executable::DestroyImageSectionDB(sectionDb);
				executable::CloseImage(thisImage);
			}
		}
	}

	
	{
		// patch security cookies in all processes.
		// when "Buffer Security Checks" (/GS) and/or "Enable Additional Security Checks" (/sdl) are enabled in a build,
		// the compiler inserts security cookies and a call to "__security_check_cookie" to check whether this cookie has
		// been overwritten. each EXE and DLL gets its own cookie, and this poses a problem.
		// when patching relocations, the original version of __security_check_cookie will be called with a check
		// against the security cookie stored in the patch DLL, which will of course fail.
		// we could special-case relocations to __security_check_cookie to never touch such relocations, but this doesn't
		// work under x64.
		// the reason for that is that under x86, __security_check_cookie will be called by __ehhandler$SomeFunctionName,
		// which means the call is always "embedded" into the code and we can therefore ignore such relocations.
		// under x64 however, throwing an exception always calls the GSHandler responsible for doing security checks,
		// but this handler lives in the original executable and is called by the kernel.
		// we therefore choose the simpler solution to overwrite patch DLL security cookies with their original values,
		// ensuring that a call to __security_check_cookie for a patch DLL will never fail.
		const symbols::Symbol* originalCookie = symbols::FindSymbolByName(m_symbolDB, ImmutableString(LC_IDENTIFIER("__security_cookie")));
		const symbols::Symbol* newCookie = symbols::FindSymbolByName(patch_symbolDB, ImmutableString(LC_IDENTIFIER("__security_cookie")));
		if (originalCookie && newCookie)
		{
			for (size_t p = 0u; p < processCount; ++p)
			{
				const PerProcessData& data = processData[p];
				PatchSecurityCookie(data.originalModuleBase, loadedPatches[p], originalCookie->rva, newCookie->rva, data.liveProcess->GetProcessHandle());
			}

			compiledModulePatch->RegisterSecurityCookie(originalCookie->rva, newCookie->rva);
		}
	}

	// patch UE4-specific symbols needed by NatVis visualizers
	if (appSettings::g_ue4EnableNatVisSupport->GetValue())
	{
		{
			const symbols::Symbol* originalNameTable = symbols::FindSymbolByName(m_symbolDB, ImmutableString(g_ue4NameTableInDLL));
			const symbols::Symbol* newNameTable = symbols::FindSymbolByName(patch_symbolDB, ImmutableString(g_ue4NameTableInLIB));

			if (originalNameTable && newNameTable)
			{
				for (size_t p = 0u; p < processCount; ++p)
				{
					const PerProcessData& data = processData[p];
					PatchUE4NatVisSymbols(data.originalModuleBase, loadedPatches[p], originalNameTable->rva, newNameTable->rva, data.liveProcess->GetProcessHandle());
				}

				compiledModulePatch->RegisterUe4NameTable(originalNameTable->rva, newNameTable->rva);
			}
		}
		{
			const symbols::Symbol* originalObjectArray = symbols::FindSymbolByName(m_symbolDB, ImmutableString(g_ue4ObjectArrayInDLL));
			const symbols::Symbol* newObjectArray = symbols::FindSymbolByName(patch_symbolDB, ImmutableString(g_ue4ObjectArrayInLIB));

			if (originalObjectArray && newObjectArray)
			{
				for (size_t p = 0u; p < processCount; ++p)
				{
					const PerProcessData& data = processData[p];
					PatchUE4NatVisSymbols(data.originalModuleBase, loadedPatches[p], originalObjectArray->rva, newObjectArray->rva, data.liveProcess->GetProcessHandle());
				}

				compiledModulePatch->RegisterUe4ObjectArray(originalObjectArray->rva, newObjectArray->rva);
			}
		}
	}

	// now that relocations are done, it is safe to call the entry point.
	// restore the original entry point and tell the process to call it.
	// BEGIN EPIC MOD
	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Restoring and calling entry point...");
	// END EPIC MOD
	{
		// disable user entry point DllMain (if it exists).
		// the DllMain function is named differently depending on the architecture.
#if LC_64_BIT
		const symbols::Symbol* dllMainSymbol = symbols::FindSymbolByName(patch_symbolDB, ImmutableString("DllMain"));
#else
		const symbols::Symbol* dllMainSymbol = symbols::FindSymbolByName(patch_symbolDB, ImmutableString("_DllMain@12"));
#endif

		if (dllMainSymbol)
		{
			// this is a DLL that has a user entry point. disable it in all processes.
			for (size_t p = 0u; p < processCount; ++p)
			{
				const PerProcessData& data = processData[p];
				PatchDllMain(loadedPatches[p], dllMainSymbol->rva, data.liveProcess->GetProcessHandle());
			}

			compiledModulePatch->RegisterDllMain(dllMainSymbol->rva);
		}

		LC_LOG_DEV("Restoring original entry point");

		// restore entry point in all processes
		for (size_t p = 0u; p < processCount; ++p)
		{
			const PerProcessData& data = processData[p];
			executablePatcher.RestoreEntryPoint(data.liveProcess->GetProcessHandle(), loadedPatches[p], entryPointRva);
		}

		LC_LOG_DEV("Calling original entry point");

		// call entry points in all processes
		for (size_t p = 0u; p < processCount; ++p)
		{
			const PerProcessData& data = processData[p];
			data.liveProcess->GetPipe()->SendCommandAndWaitForAck(commands::CallEntryPoint { loadedPatches[p], entryPointRva }, nullptr, 0u);
		}

		// disable entry point in all processes again.
		// this is done because otherwise the process would crash when "detaching" the DLL on shutdown.
		// the reason is that _DllMainCRTStartup is called when detaching the DLL, and somewhere down the callstack, this
		// function calls __scrt_dllmain_uninitialize_c - which has been patched by us (to point to the original exe) and then
		// tries to free stuff already freed. instead of trying to handle edge cases like __scrt_dllmain_uninitialize_c manually,
		// we simply disable this entry point completely.
		// note that this does NOT disable global destructors of symbols living in patch DLLs to be called!
		// because we relocate _atexit to the original function, those destructors are all registered with the original
		// atexit table, meaning they will be properly destroyed.
		for (size_t p = 0u; p < processCount; ++p)
		{
			const PerProcessData& data = processData[p];
			executablePatcher.DisableEntryPoint(data.liveProcess->GetProcessHandle(), loadedPatches[p], entryPointRva);
		}
	}


	// BEGIN EPIC MOD
	auto patchRelocations = [&](bool forceBackwards)
	{
	// END EPIC MOD
		// dynamic initializers have run, patch the remaining relocations
		// BEGIN EPIC MOD
		GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Patching remaining relocations...");
		// END EPIC MOD
		{
			telemetry::Scope patchingRelocationsScope("Patching remaining relocations");

			uint32_t relocationsHandledCount = 0u;
			size_t relocationsCount = 0u;

			LC_LOG_DEV("Patching relocations after calling entry point");

			for (auto it = patch_compilandDB->compilands.begin(); it != patch_compilandDB->compilands.end(); ++it)
			{
				const symbols::ObjPath& objPath = it->first;

				LC_LOG_DEV("Patching relocations for file %s", objPath.c_str());
				LC_LOG_INDENT_DEV;

				const coff::CoffDB* coffDb = m_coffCache->Lookup(objPath);
				if (!coffDb)
				{
					LC_ERROR_USER("Could not find COFF database for file %s", objPath.c_str());
					continue;
				}

				const std::wstring wideObjPath = string::ToWideString(objPath);

				// note that we need to take the original compiland here, to make sure that single split-files get assigned
				// the same unique ID as the corresponding amalgamated file.
				symbols::Compiland* compiland = symbols::FindCompiland(m_compilandDB, objPath);
				const uint32_t compilandUniqueId = symbols::GetCompilandId(compiland, wideObjPath.c_str(), modifiedOrNewObjFiles);

				const types::StringSet& strippedSymbols = strippedSymbolsPerCompiland[objPath];
				const types::StringSet& forceStrippedSymbols = forceStrippedSymbolsPerCompiland[objPath];

				const size_t symbolCount = coffDb->symbols.size();
				for (size_t i = 0u; i < symbolCount; ++i)
				{
					const coff::Symbol* symbol = coffDb->symbols[i];
					relocationsCount += symbol->relocations.size();

					// check if the patch knows this symbol.
					// if not, it has probably been stripped and there is no need to walk all its relocations.
					const ImmutableString& symbolName = coff::GetSymbolName(coffDb, symbol);
					const symbols::Symbol* realSymbol = symbols::FindSymbolByName(patch_symbolDB, symbolName);
					if (!realSymbol)
					{
						// this symbol has been stripped from the executable.
						// in optimized builds, the compiler will sometimes e.g. leave a static function in an OBJ file,
						// which will be kicked out by the linker.
						continue;
					}

					// before patching relocations, check whether the symbol which relocations we want to patch originated from
					// a compiland that is the same as the file we're working on.
					// this might not be the case, especially when using static libraries, COMDATs, and compilands that use the
					// same inline function but have slightly different compiler options (/hotpatch vs. no /hotpatch, e.g.
					// __local_stdio_printf_options in the main module vs. in the dynamic runtime)
					const symbols::Contribution* originalContribution = symbols::FindContributionByRVA(patch_contributionDB, realSymbol->rva);
					if (originalContribution)
					{
						const ImmutableString& compilandName = symbols::GetContributionCompilandName(patch_contributionDB, originalContribution);
						if (compilandName != objPath)
						{
							LC_LOG_DEV("Ignoring relocations for symbol %s in file %s (original compiland: %s)",
								symbolName.c_str(), objPath.c_str(), compilandName.c_str());
							continue;
						}
					}

					const size_t relocationCount = symbol->relocations.size();
					for (size_t j = 0u; j < relocationCount; ++j)
					{
						const coff::Relocation* relocation = symbol->relocations[j];
						if (relocation->dstIsSection)
						{
							// don't patch relocations to sections
							continue;
						}

						// ignore relocations to symbols in .msvcjmc (MSVC JustMyCode) sections
						if (relocation->dstSectionIndex >= 0)
						{
							const uint32_t index = static_cast<uint32_t>(relocation->dstSectionIndex);
							const coff::Section& section = coffDb->sections[index];
							if (coff::IsMSVCJustMyCodeSection(section.name.c_str()))
							{
								LC_LOG_DEV("Ignoring relocation to symbol in section %s", section.name.c_str());
								continue;
							}
						}

						ImmutableString dstSymbolName = symbols::TransformAnonymousNamespacePattern(GetRelocationDstSymbolName(coffDb, relocation), compilandUniqueId);

						// relocations to data symbols and stripped symbols have already been done
						const bool refersToFunctionSymbol = coff::IsFunctionSymbol(coff::GetRelocationDstSymbolType(relocation));
						const bool refersToStrippedSymbol = (strippedSymbols.find(dstSymbolName) != strippedSymbols.end());
						if (refersToFunctionSymbol && !refersToStrippedSymbol)
						{
							// BEGIN EPIC MOD
							const relocations::Record& relocationRecord = relocations::PatchRelocation(relocation, coffDb, forceStrippedSymbols, m_moduleCache, symbolName, dstSymbolName, realSymbol, token, &loadedPatches[0], forceBackwards);
							if (!forceBackwards && relocations::IsValidRecord(relocationRecord))
							{
								compiledModulePatch->RegisterPostEntryPointRelocation(relocationRecord);
							}
							// END EPIC MOD

							++relocationsHandledCount;
						}
					}
				}
			}

			LC_LOG_TELEMETRY("Handled %d of %d remaining relocations in %.3fms (avg: %.3fus)", relocationsHandledCount, relocationsCount,
				patchingRelocationsScope.ReadMilliSeconds(), (patchingRelocationsScope.ReadMicroSeconds() / relocationsHandledCount));
		}
	// BEGIN EPIC MOD
	};
	// END EPIC MOD

	// BEGIN EPIC MOD
	patchRelocations(enableReinstancingFlow);
	// END EPIC MOD

	// BEGIN EPIC MOD
	auto patchFunctions = [&](bool forceBackwards)
	// END EPIC MOD
	{

		// BEGIN EPIC MOD
		GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Patching functions...");
		// END EPIC MOD

		// suspend the main processes before patching functions, because they might not use synchronization points.
		for (size_t p = 0u; p < processCount; ++p)
		{
			const PerProcessData& data = processData[p];
			Process::Suspend(data.liveProcess->GetProcessHandle());
		}


		// determining which functions have changed (or lead to a different execution path) would be very hard
		// to do, therefore we hook all functions.
		// even though internal functions can only be referenced from external ones, it is not enough to hook
		// only those. the reason for that is that global/static instances might refer to internal functions
		// by function-pointer, address, etc., so internal functions must also be hooked.
		{
			telemetry::Scope patchingFunctionsScope("Patching functions");

			// the processes are all halted. fetch instruction pointers from all their threads.
			typedef types::vector<const void*> PerProcessThreadIPs;
			types::vector<PerProcessThreadIPs> processThreadIPs;
			processThreadIPs.reserve(processCount);
			for (size_t p = 0u; p < processCount; ++p)
			{
				const PerProcessData& data = processData[p];
				processThreadIPs.emplace_back(EnumerateInstructionPointers(data.liveProcess->GetProcessId()));
			}

			uint32_t functionsPatchedCount = 0u;
			size_t functionsCount = 0u;

			// we deliberately do not hook functions in lib compilands because they cannot have changed, per definition.
			// they are part of a static library that won't be recompiled during a Live++ session.
			for (auto it = patch_compilandDB->compilands.begin(); it != patch_compilandDB->compilands.end(); ++it)
			{
				const symbols::ObjPath& objPath = it->first;

				LC_LOG_DEV("Patching functions for file %s", objPath.c_str());
				LC_LOG_INDENT_DEV;

				const coff::CoffDB* coffDb = m_coffCache->Lookup(objPath);
				if (!coffDb)
				{
					LC_ERROR_USER("Could not find COFF database for file %s", objPath.c_str());
					continue;
				}

				const std::wstring& wideObjPath = string::ToWideString(objPath);
				const symbols::Compiland* compiland = it->second;
				const uint32_t compilandUniqueId = symbols::GetCompilandId(compiland, wideObjPath.c_str(), modifiedOrNewObjFiles);

				for (size_t i = 0u; i < coffDb->symbols.size(); ++i)
				{
					const coff::Symbol* symbol = coffDb->symbols[i];
					if (!coff::IsFunctionSymbol(symbol->type))
					{
						continue;
					}

					++functionsCount;

					const ImmutableString& functionName = symbols::TransformAnonymousNamespacePattern(coff::GetSymbolName(coffDb, symbol), compilandUniqueId);
					if (symbols::IsExceptionRelatedSymbol(functionName))
					{
						LC_LOG_DEV("Ignoring exception-related function %s", functionName.c_str());
						continue;
					}

					const symbols::Symbol* patchSymbol = symbols::FindSymbolByName(patch_symbolDB, functionName);
					if (!patchSymbol)
					{
						LC_WARNING_DEV("Cannot find function %s in patch, possibly stripped by linker", functionName.c_str());
						continue;
					}

					const ModuleCache::FindSymbolData& originalData = m_moduleCache->FindSymbolByName(token, functionName);
					if (!originalData.symbol)
					{
						LC_LOG_DEV("Ignoring new function %s", functionName.c_str());
						continue;
					}

					// if the original function to be patched did not come from a compiland, it cannot possibly have changed and
					// therefore can be ignored.
					const symbols::Contribution* originalContribution = symbols::FindContributionByRVA(originalData.data->contributionDb, originalData.symbol->rva);
					if (originalContribution)
					{
						const ImmutableString& compilandName = symbols::GetContributionCompilandName(originalData.data->contributionDb, originalContribution);
						const symbols::Compiland* originalCompiland = symbols::FindCompiland(originalData.data->compilandDb, compilandName);
						if (!originalCompiland)
						{
							LC_LOG_DEV("Ignoring function %s originally contributed from lib compiland %s", functionName.c_str(), compilandName.c_str());
							continue;
						}
					}

					const size_t moduleProcessCount = originalData.data->processes.size();
					for (size_t p = 0u; p < moduleProcessCount; ++p)
					{
						++functionsPatchedCount;

						const ModuleCache::ProcessData& hookProcessData = originalData.data->processes[p];

						const Process::Id pid = hookProcessData.processId;
						void* moduleBase = hookProcessData.moduleBase;
						Process::Handle processHandle = hookProcessData.processHandle;

						char* originalAddress = pointer::Offset<char*>(moduleBase, originalData.symbol->rva);
						char* patchAddress = pointer::Offset<char*>(loadedPatches[p], patchSymbol->rva);
						types::unordered_set<const void*>& patchedAddresses = m_patchedAddressesPerProcess[pid];

						const functions::Record& record = functions::PatchFunction(originalAddress, patchAddress, originalData.symbol->rva, patchSymbol->rva,
							originalData.data->thunkDb, originalContribution, processHandle, moduleBase, originalData.data->index,
							patchedAddresses, processThreadIPs[p], pid, functionName.c_str());

						// BEGIN EPIC MOD
						if (!forceBackwards && functions::IsValidRecord(record))
						{
							compiledModulePatch->RegisterFunctionPatch(record);
						}
						// END EPIC MOD
					}
				}
			}
		}

		// resume the main processes again
		for (size_t p = 0u; p < processCount; ++p)
		{
			const PerProcessData& data = processData[p];
			Process::Resume(data.liveProcess->GetProcessHandle());
		}
	// BEGIN EPIC MOD
	};
	// END EPIC MOD

	// BEGIN EPIC MOD
	if (!enableReinstancingFlow)
	{
		patchFunctions(enableReinstancingFlow);
	}
	// END EPIC MOD

	{
		// post-patch hooks must be called on the current executable because the hooks want to use the newest memory layout of
		// data structures. therefore we do not ignore any executables in our search.
		if (updateType != LiveModule::UpdateType::NO_CLIENT_COMMUNICATION)
		{
			const ModuleCache::FindHookData& hookData = m_moduleCache->FindHooksInSectionBackwards(ModuleCache::SEARCH_ALL_MODULES, ImmutableString(LPP_POSTPATCH_SECTION));
			if ((hookData.firstRva != 0u) && (hookData.lastRva != 0u))
			{
				const size_t count = hookData.data->processes.size();
				for (size_t p = 0u; p < count; ++p)
				{
					const Process::Id pid = hookData.data->processes[p].processId;
					void* moduleBase = hookData.data->processes[p].moduleBase;
					const DuplexPipe* pipe = hookData.data->processes[p].pipe;

					LC_LOG_USER("Calling post-patch hooks (PID: %d)", +pid);
					pipe->SendCommandAndWaitForAck(MakeCallHooksCommand(hook::Type::POSTPATCH, moduleBase, hookData), nullptr, 0u);
				}

				compiledModulePatch->RegisterPostPatchHooks(hookData.data->index, hookData.firstRva, hookData.lastRva);
			}
		}
	}

	// BEGIN EPIC MOD
	// pulse the sync point in all processes
	if (enableReinstancingFlow)
	{
		if (updateType != LiveModule::UpdateType::NO_CLIENT_COMMUNICATION)
		{
			for (size_t p = 0u; p < processCount; ++p)
			{
				const PerProcessData& data = processData[p];
				data.liveProcess->GetPipe()->SendCommandAndWaitForAck(commands::TriggerReload{}, nullptr, 0u);
			}
		}

		// Re-patch everything so that things are patched as normally expected
		patchRelocationsPreEntryPoint(false);
		patchRelocations(false);
		patchFunctions(false);
	}
	// END EPIC MOD

	// leave sync point in all processes
	if (updateType != LiveModule::UpdateType::NO_CLIENT_COMMUNICATION)
	{
		for (size_t p = 0u; p < processCount; ++p)
		{
			const PerProcessData& data = processData[p];
			data.liveProcess->GetPipe()->SendCommandAndWaitForAck(commands::LeaveSyncPoint {}, nullptr, 0u);
		}
	}

	// clear the set for the next update
	m_modifiedFiles.clear();
	m_compiledCompilands.clear();

	LC_SUCCESS_USER("Patch creation for module %S successful (%.3fs)", m_moduleName.c_str(), updateScope.ReadSeconds());

	// log all processes that were patched in case we have more than one
	if (processCount > 1u)
	{
		for (size_t p = 0u; p < processCount; ++p)
		{
			const PerProcessData& data = processData[p];
			LC_SUCCESS_USER("Patched process %S (PID: %d)", data.modulePath.c_str(), data.liveProcess->GetProcessId());
		}
	}

	CallCompileSuccessHooks(m_moduleCache, updateType);

	return ErrorType::SUCCESS;
}


bool LiveModule::InstallCompiledPatches(LiveProcess* liveProcess, void* originalModuleBase)
{
	if (!appSettings::g_installCompiledPatchesMultiProcess->GetValue())
	{
		// don't install any patches
		return true;
	}

	if (m_compiledModulePatches.size() == 0u)
	{
		// nothing to install
		return true;
	}

	LC_LOG_DEV("LiveModule InstallCompiledPatches: %S", m_moduleName.c_str());

	telemetry::Scope wholeScope("Installing patches");

	Process::Handle processHandle = liveProcess->GetProcessHandle();
	const Process::Id processId = liveProcess->GetProcessId();
	const DuplexPipe* pipe = liveProcess->GetPipe();

// BEGIN EPIC MOD
#if LC_64_BIT
	// free our page reservations in the +-2GB range of the main module
	liveProcess->FreeVirtualMemoryPages(originalModuleBase);
#endif
// END EPIC MOD

	for (auto modulePatch : m_compiledModulePatches)
	{
		const std::wstring& originalExePath = modulePatch->GetExePath();

		LC_LOG_USER("Installing patch %S (PID: %d)", originalExePath.c_str(), +processId);

		// this image needs to be copied because it is loaded already.
		// create a new name based on the process ID, which must be unique.
		std::wstring exePath = originalExePath;
		{
			exePath = string::Replace(exePath, L".exe", std::wstring(L"_PID_") + std::to_wstring(+processId) + L".exe");
			Filesystem::Copy(originalExePath.c_str(), exePath.c_str());
		}

		const size_t token = modulePatch->GetToken();
		const ModulePatch::Data& patchData = modulePatch->GetData();


		// note that the image on disk we are trying to load had its entry point patched already when it was
		// loaded for the first time, so we don't have to do that at this point.
		executable::Image* image = executable::OpenImage(exePath.c_str(), Filesystem::OpenMode::READ);
		if (!image)
		{
			LC_ERROR_USER("Cannot load patch executable %S", exePath.c_str());
			return false;
		}

		const uint32_t entryPointRva = executable::GetEntryPointRva(image);
		const uint32_t patchImageSize = executable::GetSize(image);
		executable::CloseImage(image);

		// the patch's entry point is disabled. tell the processes to load the patch
		LC_LOG_DEV("Loading code into process");

		types::vector<void*> loadedPatches;
		{
			commands::LoadPatch cmd = {};
			wcscpy_s(cmd.path, exePath.c_str());

#if LC_64_BIT
			// before doing anything further, we need to ensure that the patch can be loaded into the address space at a suitable location.
			// for 64-bit applications, this means that the patch must lie in a +/-2GB range of the main executable.
			// 32-bit executables can reach the whole address space due to modulo addressing.
			LC_LOG_DEV("Scanning memory for suitable patch location (PID: %d)", +processId);

			// disable the main process before scanning its memory to ensure that no operation allocates/frees virtual memory concurrently
			Process::Suspend(processHandle);

			const executable::PreferredBase preferredImageBase = FindPreferredImageBase(patchImageSize, processId, processHandle, originalModuleBase);

			// rebase the patch image to its preferred base address
			executable::Image* rebasedImage = executable::OpenImage(exePath.c_str(), Filesystem::OpenMode::READ_WRITE);
			LC_LOG_DEV("Rebasing patch executable to image base 0x%" PRIX64 " (PID: %d)", preferredImageBase, +processId);
			executable::RebaseImage(rebasedImage, preferredImageBase);
			executable::CloseImage(rebasedImage);

			// resume the main process so that it can respond to our command. if we're *really* unlucky, a concurrent operation
			// will allocate virtual memory at the patch's preferred image base, possibly rendering the patch unusable because
			// it cannot be loaded.
			// the chances of that happening are *very* rare though, and we can always load the next patch then.
			Process::Resume(processHandle);
#endif

			pipe->SendCommandAndWaitForAck(cmd, nullptr, 0u);

			// receive command with patch info
			CommandMap commandMap;
			commandMap.RegisterAction<actions::LoadPatchInfo>();
			commandMap.HandleCommands(pipe, &loadedPatches);
		}

		void* moduleBase = loadedPatches[0];
		const bool patchesLoadedSuccessfully = CheckPatchAddressValidity(originalModuleBase, moduleBase, processHandle);
		if (!patchesLoadedSuccessfully)
		{
			LC_ERROR_USER("Patch could not be activated.");

			pipe->SendCommandAndWaitForAck(commands::UnloadPatch { static_cast<HMODULE>(moduleBase) }, nullptr, 0u);
			return false;
		}


		// enter sync point
		pipe->SendCommandAndWaitForAck(commands::EnterSyncPoint {}, nullptr, 0u);


		// store the new databases into the module cache
		m_moduleCache->RegisterProcess(token, liveProcess, moduleBase);

		types::vector<void*> processModuleBases = m_moduleCache->GatherModuleBases(processId);


		LC_LOG_DEV("Calling pre-patch hooks");
		{
			void* hookModule = processModuleBases[patchData.prePatchHookModuleIndex];
			pipe->SendCommandAndWaitForAck(MakeCallHooksCommand(hook::Type::PREPATCH, hookModule, patchData.firstPrePatchHook, patchData.lastPrePatchHook), nullptr, 0u);
		}


		LC_LOG_DEV("Patching relocations before calling entry point");
		for (auto record : patchData.preEntryPointRelocations)
		{
			relocations::PatchRelocation(record, processHandle, &processModuleBases[0], moduleBase);
		}


		LC_LOG_DEV("Patching dynamic initializers");
		for (auto rva : patchData.patchedInitializers)
		{
			LC_LOG_DEV("Patching dynamic initializer symbol at RVA 0x%X (PID: %d)", rva, +processId);

			void* initializerAddress = pointer::Offset<void*>(moduleBase, rva);
			Process::WriteProcessMemory(processHandle, initializerAddress, nullptr);
		}


		LC_LOG_DEV("Patching security cookie");
		PatchSecurityCookie(originalModuleBase, moduleBase, patchData.originalCookieRva, patchData.patchCookieRva, processHandle);

		if (appSettings::g_ue4EnableNatVisSupport->GetValue())
		{
			LC_LOG_DEV("Patching symbols for UE4 NatVis visualizers");
			PatchUE4NatVisSymbols(originalModuleBase, moduleBase, patchData.originalUe4NameTableRva, patchData.patchUe4NameTableRva, processHandle);
			PatchUE4NatVisSymbols(originalModuleBase, moduleBase, patchData.originalUe4ObjectArrayRva, patchData.patchUe4ObjectArrayRva, processHandle);
		}

		// now that relocations are done, it is safe to call the entry point.
		// restore the original entry point and tell the process to call it.
		{
			// disable user entry point DllMain (if it exists)
			if (patchData.dllMainRva != 0u)
			{
				PatchDllMain(moduleBase, patchData.dllMainRva, processHandle);
			}

			LC_LOG_DEV("Restoring original entry point");

			// restore entry point in all processes.
			// the module patch stores the original entry point code from the original image, before it had
			// its entry point patched.
			ExecutablePatcher executablePatcher(patchData.entryPointCode);
			executablePatcher.RestoreEntryPoint(processHandle, moduleBase, entryPointRva);

			LC_LOG_DEV("Calling original entry point");

			pipe->SendCommandAndWaitForAck(commands::CallEntryPoint { moduleBase, entryPointRva }, nullptr, 0u);

			executablePatcher.DisableEntryPoint(processHandle, moduleBase, entryPointRva);
		}


		LC_LOG_DEV("Patching relocations after calling entry point");
		for (auto record : patchData.postEntryPointRelocations)
		{
			relocations::PatchRelocation(record, processHandle, &processModuleBases[0], moduleBase);
		}
		

		// suspend the main processes before patching functions, because they might not use synchronization points.
		Process::Suspend(processHandle);


		// patch all functions
		types::unordered_set<const void*>& patchedAddresses = m_patchedAddressesPerProcess[processId];
		types::vector<const void*> threadIPs = EnumerateInstructionPointers(processId);

		LC_LOG_DEV("Patching functions");
		for (auto record : patchData.functionPatches)
		{
			functions::PatchFunction(record, processHandle, &processModuleBases[0], moduleBase, patchedAddresses, threadIPs);
		}

		LC_LOG_DEV("Patching public functions in lib compilands");
		for (auto record : patchData.libraryFunctionPatches)
		{
			functions::PatchLibraryFunction(record, processHandle, &processModuleBases[0], moduleBase);
		}


		// resume the main processes again
		Process::Resume(processHandle);


		LC_LOG_DEV("Calling post-patch hooks");
		{
			void* hookModule = processModuleBases[patchData.postPatchHookModuleIndex];
			pipe->SendCommandAndWaitForAck(MakeCallHooksCommand(hook::Type::POSTPATCH, hookModule, patchData.firstPostPatchHook, patchData.lastPostPatchHook), nullptr, 0u);
		}


		// leave sync point
		pipe->SendCommandAndWaitForAck(commands::LeaveSyncPoint {}, nullptr, 0u);
	}

// BEGIN EPIC MOD
#if LC_64_BIT
	// immediately reserve pages in the +-2GB range of the main module again
	liveProcess->ReserveVirtualMemoryPages(originalModuleBase);
#endif
// END EPIC MOD

	LC_SUCCESS_USER("Successfully installed %zu patches (%.3fs)", m_compiledModulePatches.size(), wholeScope.ReadSeconds());

	return true;
}


const std::wstring& LiveModule::GetModuleName(void) const
{
	return m_moduleName;
}


const executable::Header& LiveModule::GetImageHeader(void) const
{
	return m_imageHeader;
}


const symbols::CompilandDB* LiveModule::GetCompilandDatabase(void) const
{
	return m_compilandDB;
}


const symbols::LinkerDB* LiveModule::GetLinkerDatabase(void) const
{
	return m_linkerDB;
}


bool LiveModule::HasInstalledPatches(void) const
{
	return (m_patchCounter != 0u);
}


bool LiveModule::actions::LoadPatchInfo::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	types::vector<void*>* loadedPatches = static_cast<types::vector<void*>*>(context);
	loadedPatches->emplace_back(command->module);

	pipe->SendAck();

	return false;
}


void LiveModule::UpdateDirectoryCache(const ImmutableString& path, symbols::Dependency* dependency, DirectoryCache* cache)
{
	const std::wstring directoryOnly = Filesystem::GetDirectory(string::ToWideString(path).c_str()).GetString();
	dependency->parentDirectory = cache->AddDirectory(directoryOnly);
}


void LiveModule::OnCompiledFile(const symbols::ObjPath& objPath, symbols::Compiland* compiland, const CompileResult& compileResult, double compileTime, bool forceAmalgamationPartsLinkage)
{
	if (compileResult.exitCode == 0u)
	{
		if (compileResult.wasCompiled)
		{
			LC_SUCCESS_USER("Successfully compiled %s (%.3fs)", objPath.c_str(), compileTime);
		}

		// AMALGAMATION
		// files which are part of an amalgamation only need to be linked in when initially splitting the unity file.
		// this happens the first time some .cpp file is touched during a session.
		// even though up-to-date .cpp files don't need to be recompiled, they need to be linked in order to
		// handle inlining across translation units.
		if (compileResult.wasCompiled || forceAmalgamationPartsLinkage)
		{
			// compilation was successful, store this compiland for linking later
			m_compiledCompilands.emplace(objPath, compiland);
			symbols::MarkCompilandAsRecompiled(compiland);
		}

		// remove this file from the set of modified files. it need not be compiled in the next run, unless
		// it has been modified again. if so, it will be picked up automatically by checking the modification time.
		m_modifiedFiles.erase(objPath);
	}
	else
	{
		// compilation failed. remove the compiland from the set of previously compiled compilands, because it
		// might have compiled successfully in an earlier call to Update().
		// note that we do not remove this file from the set of modified files, so it is automatically compiled again
		// upon the next call to Update().
		m_compiledCompilands.erase(objPath);
		symbols::ClearCompilandAsRecompiled(compiland);
		LC_ERROR_USER("Failed to compile %s (%.3fs) (Exit code: 0x%X)", objPath.c_str(), compileTime, compileResult.exitCode);
	}
}

// BEGIN EPIC MOD
bool LiveModule::IsModifiedSource(const wchar_t* sourceFile) const
{
	if (m_moduleCache == nullptr || m_moduleCache->GetSize() == 0)
	{
		return false;
	}

	Filesystem::PathAttributes attr = Filesystem::GetAttributes(sourceFile);
	if (!Filesystem::DoesExist(attr))
	{
		return false;
	}

	for (size_t i = 0; i < m_moduleCache->GetSize(); ++i)
	{
		const ModuleCache::Data& data = m_moduleCache->GetEntry(i);
		if (data.lastModificationTime > attr.lastModificationTime)
		{
			return false;
		}
	}
	return true;
}
// END EPIC MOD

#endif