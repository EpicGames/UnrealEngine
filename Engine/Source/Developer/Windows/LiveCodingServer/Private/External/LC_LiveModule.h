// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_CoffCache.h"
#include "LC_ModuleCache.h"
#include "LC_Symbols.h"
#include "LC_ProcessTypes.h"
#include "LC_Commands.h"
#include "LC_Executable.h"
#include "LC_Semaphore.h"
#include "LC_RunMode.h"


class DuplexPipe;
class FileAttributeCache;
class DirectoryCache;
class ModulePatch;
class LiveProcess;
class VirtualMemoryRange;

class LiveModule
{
public:
	struct PerProcessData
	{
		LiveProcess* liveProcess;
		void* originalModuleBase;
		std::wstring modulePath;
	};

	struct ErrorType
	{
		enum Enum
		{
			NO_CHANGE,
			COMPILE_ERROR,
			LINK_ERROR,
			LOAD_PATCH_ERROR,
			ACTIVATE_PATCH_ERROR,
			SUCCESS
		};
	};

	struct UpdateType
	{
		enum Enum
		{
			DEFAULT,
			NO_CLIENT_COMMUNICATION
		};
	};

	struct CompileResult
	{
		unsigned int exitCode;
		bool wasCompiled;
	};

	LiveModule(const wchar_t* moduleName, const executable::Header& imageHeader, RunMode::Enum runMode);
	~LiveModule(void);

	void Load(symbols::Provider* provider, symbols::DiaCompilandDB* diaCompilandDb);

	// unloads all patches that have been loaded into all processes so far
	void Unload(void);

	void RegisterProcess(LiveProcess* liveProcess, void* moduleBase, const std::wstring& modulePath);
	void UnregisterProcess(LiveProcess* liveProcess);

	void DisableControlFlowGuard(LiveProcess* liveProcess, void* moduleBase);

	void UpdateDirectoryCache(DirectoryCache* cache);

	// builds a patch for this module according to changed files.
	// in DEFAULT mode, this checks for file modifications, compiles files automatically, builds a patch containing changes, and loads them into the host application.
	// in EXTERNAL_BUILD_SYSTEM mode, this does not compile files but builds a patch containing modified .objs, loading the patch into the host application.
	// optionally, an array of modified or new .objs can be given in this mode, which builds a patch containing these files, not checking for any other modifications.
	// BEGIN EPIC MOD
	ErrorType::Enum Update(FileAttributeCache* fileCache, DirectoryCache* directoryCache, UpdateType::Enum updateType, const types::vector<symbols::ModifiedObjFile>& modifiedOrNewObjFiles, const types::vector<std::wstring>& additionalLibraries);
	// END EPIC MOD
	bool InstallCompiledPatches(LiveProcess* liveProcess, void* originalModuleBase);

	const std::wstring& GetModuleName(void) const;
	const executable::Header& GetImageHeader(void) const;

	const symbols::CompilandDB* GetCompilandDatabase(void) const;
	const symbols::LinkerDB* GetLinkerDatabase(void) const;

	bool HasInstalledPatches(void) const;

	// BEGIN EPIC MOD
	bool IsModifiedSource(const wchar_t* sourceFile) const;
	// END EPIC MOD

private:
	void UpdateDirectoryCache(const ImmutableString& path, symbols::Dependency* dependency, DirectoryCache* cache);

	void OnCompiledFile(const symbols::ObjPath& objPath, symbols::Compiland* compiland, const CompileResult& compileResult, double compileTime, bool forceAmalgamationPartsLinkage);

	// actions
	struct actions
	{
		#define DECLARE_ACTION(_name)																													\
			struct _name																																\
			{																																			\
				typedef ::commands::_name CommandType;																									\
				static bool Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void* payload, size_t payloadSize);		\
			}

		DECLARE_ACTION(LoadPatchInfo);

		#undef DECLARE_ACTION
	};


	std::wstring m_moduleName;
	executable::Header m_imageHeader;
	RunMode::Enum m_runMode;
	size_t m_mainModuleToken = 0u;

	// data that stores which processes loaded this module at which address originally
	types::vector<PerProcessData> m_perProcessData;

	// caches
	CoffCache<coff::CoffDB>* m_coffCache = nullptr;
	ModuleCache* m_moduleCache = nullptr;
	types::StringSet m_reconstructedCompilands;

	// main databases
	symbols::SymbolDB* m_symbolDB = nullptr;
	symbols::ContributionDB* m_contributionDB = nullptr;
	symbols::CompilandDB* m_compilandDB = nullptr;
	symbols::LibraryDB* m_libraryDB = nullptr;
	symbols::LinkerDB* m_linkerDB = nullptr;
	symbols::ThunkDB* m_thunkDB = nullptr;
	symbols::ImageSectionDB* m_imageSectionDB = nullptr;
	types::StringMap<types::vector<const symbols::Symbol*>> m_externalSymbolsPerCompilandCache;
	types::StringMap<ImmutableString> m_pchSymbolToCompilandName;

	// patch data
	types::unordered_map<Process::Id, types::unordered_set<const void*>> m_patchedAddressesPerProcess;
	unsigned int m_patchCounter = 0u;

	// data pertaining to the next patch
	types::StringSet m_modifiedFiles;
	types::StringMap<symbols::Compiland*> m_compiledCompilands;

	// all patches loaded so far along with recorded data how to load them into other processes
	types::vector<ModulePatch*> m_compiledModulePatches;

	// BEGIN EPIC MOD - Allow mapping from object files to their unity object file
	types::StringMap<uint32_t> m_objFileToCompilandId;
	// END EPIC MOD
};


#endif