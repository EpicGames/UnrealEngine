// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_LiveProcess.h"
#include "LC_HeartBeat.h"
#include "LC_CodeCave.h"
#include "LC_Event.h"
#include "LC_PrimitiveNames.h"
#include "LC_VisualStudioAutomation.h"
#include "LC_Process.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
// END EPIC MOD


#if LC_64_BIT
// EPIC BEGIN MOD
namespace liveProcess // Duplicate definitions 
// EPIC END MOD
{
	static const void* GetLowerBoundIn4GBRange(const void* moduleBase)
	{
		// nothing can be loaded at address 0x0, 64 KB seems like a realistic minimum
		const uint64_t LOWEST_ADDRESS = 64u * 1024u;
		const uint64_t base = pointer::AsInteger<uint64_t>(moduleBase);

		// make sure we don't underflow
		if (base >= 0x80000000ull + LOWEST_ADDRESS)
		{
			return pointer::FromInteger<const void*>(base - 0x80000000ull);
		}

		// operation would underflow
		return pointer::FromInteger<const void*>(LOWEST_ADDRESS);
	}

	static const void* GetUpperBoundIn4GBRange(const void* moduleBase)
	{
		const uint64_t HighestPossibleAddress = 0x00007FFFFFFF0000ull;
		const uint64_t HighestPossibleAddressThreshold = HighestPossibleAddress - 2ull * 1024ull * 1024ull * 1024ull;

		const uint64_t base = pointer::AsInteger<uint64_t>(moduleBase);

		// make sure we don't overflow
		if (base <= HighestPossibleAddressThreshold)
		{
			return pointer::FromInteger<const void*>(base + 2ull * 1024ull * 1024ull * 1024ull);
		}

		// operation would overflow
		return pointer::FromInteger<const void*>(HighestPossibleAddress);
	}
}
#endif


LiveProcess::LiveProcess(Process::Handle processHandle, Process::Id processId, Thread::Id commandThreadId, const void* jumpToSelf, const DuplexPipe* pipe,
	const wchar_t* imagePath, const wchar_t* commandLine, const wchar_t* workingDirectory, const void* environment, size_t environmentSize)
	: m_processHandle(processHandle)
	, m_processId(processId)
	, m_commandThreadId(commandThreadId)
	, m_jumpToSelf(jumpToSelf)
	, m_pipe(pipe)
	, m_imagePath(imagePath)
	, m_commandLine(commandLine)
	, m_workingDirectory(workingDirectory)
	, m_environment(environment, environmentSize)
	, m_imagesTriedToLoad()
	, m_heartBeatDelta(0ull)
// BEGIN EPIC MOD
#if WITH_VISUALSTUDIO_DTE
// END EPIC MOD
	, m_vsDebugger(nullptr)
	, m_vsDebuggerThreads()
// BEGIN EPIC MOD
#endif
// END EPIC MOD
	, m_codeCave(nullptr)
	, m_restartState(RestartState::DEFAULT)
	, m_virtualMemoryRange(processHandle)
{
	m_imagesTriedToLoad.reserve(256u);
}


void LiveProcess::ReadHeartBeatDelta(const wchar_t* const processGroupName)
{
	HeartBeat heartBeat(processGroupName, m_processId);
	m_heartBeatDelta = heartBeat.ReadBeatDelta();
}


bool LiveProcess::MadeProgress(void) const
{
	if (m_heartBeatDelta >= 100ull * 10000ull)
	{
		// the client process hasn't stored a new heart beat in more than 100ms.
		// as long as it is running, it stores a new heart beat every 10ms, so we conclude that it
		// didn't make progress, e.g. because it is being held in the debugger.
		return false;
	}

	return true;
}


void LiveProcess::HandleDebuggingPreCompile(void)
{
// BEGIN EPIC MOD
#if WITH_VISUALSTUDIO_DTE
// END EPIC MOD
	if (!MadeProgress())
	{
		// this process did not make progress.
		// try to find a debugger that's currently debugging our process.
		m_vsDebugger = visualStudio::FindDebuggerForProcess(m_processId);
		if (m_vsDebugger)
		{
			// found a debugger.
			// enumerate all threads, freeze every thread but the command thread, and let the debugger resume.
			// this "halts" the process but lets the command thread act on commands sent by us.
			m_vsDebuggerThreads = visualStudio::EnumerateThreads(m_vsDebugger);
			if (m_vsDebuggerThreads.size() > 0u)
			{
				if (visualStudio::FreezeThreads(m_vsDebugger, m_vsDebuggerThreads))
				{
					if (visualStudio::ThawThread(m_vsDebugger, m_vsDebuggerThreads, m_commandThreadId))
					{
						if (visualStudio::Resume(m_vsDebugger))
						{
							LC_SUCCESS_USER("Automating debugger attached to process (PID: %d)", m_processId);

							return;
						}
						else
						{
							LC_LOG_USER("Failed to resume debugger (PID: %d)", m_processId);
						}
					}
					else
					{
						// BEGIN EPIC MOD
						LC_LOG_USER("Failed to thaw Live coding command thread in debugger (PID: %d)", m_processId);
						// END EPIC MOD
					}
				}
				else
				{
					LC_LOG_USER("Failed to freeze threads in debugger (PID: %d)", m_processId);
				}
			}
			else
			{
				LC_LOG_USER("Failed to enumerate threads in debugger (PID: %d)", m_processId);
			}
		}
		else
		{
			LC_LOG_USER("Failed to find debugger attached to process (PID: %d)", m_processId);
		}

		// no debugger could be found or an error occurred.
		// continue by installing a code cave.
		m_vsDebugger = nullptr;
		LC_LOG_USER("Failed to automate debugger attached to process (PID: %d), using fallback mechanism", m_processId);
		LC_SUCCESS_USER("Waiting for client process (PID: %d), hit 'Continue' (F5 in Visual Studio) if being held in the debugger", m_processId);
	}
// BEGIN EPIC MOD
#endif
// END EPIC MOD

	// this process either made progress and is not held in the debugger, or we failed automating the debugger.
	// "halt" this process by installing a code cave.
	InstallCodeCave();
}


void LiveProcess::HandleDebuggingPostCompile(void)
{
	if (m_codeCave)
	{
		// we installed a code cave previously, remove it
		UninstallCodeCave();
	}
// BEGIN EPIC MOD
#if WITH_VISUALSTUDIO_DTE
// END EPIC MOD
	else if (m_vsDebugger)
	{
		// we automated the debugger previously. break into the debugger again and resume all threads.
		// when debugging a C# project that calls into C++ code, the VS debugger sometimes creates new MTA threads in between our PreCompile and PostCompile calls.
		// try getting a new list of threads and thaw them all as well.
		if (visualStudio::Break(m_vsDebugger))
		{
			types::vector<EnvDTE::ThreadPtr> newDebuggerThreads = visualStudio::EnumerateThreads(m_vsDebugger);
			if (newDebuggerThreads.size() > 0u)
			{
				visualStudio::ThawThreads(m_vsDebugger, newDebuggerThreads);
			}

			visualStudio::ThawThreads(m_vsDebugger, m_vsDebuggerThreads);
		}
	}

	m_vsDebugger = nullptr;
// BEGIN EPIC MOD
#endif
// END EPIC MOD
}


void LiveProcess::InstallCodeCave(void)
{
	m_codeCave = new CodeCave(m_processHandle, m_processId, m_commandThreadId, m_jumpToSelf);
	m_codeCave->Install();
}


void LiveProcess::UninstallCodeCave(void)
{
	m_codeCave->Uninstall();
	delete m_codeCave;
	m_codeCave = nullptr;
}


void LiveProcess::AddLoadedImage(const executable::Header& imageHeader)
{
	m_imagesTriedToLoad.insert(imageHeader);
}


void LiveProcess::RemoveLoadedImage(const executable::Header& imageHeader)
{
	m_imagesTriedToLoad.erase(imageHeader);
}


bool LiveProcess::TriedToLoadImage(const executable::Header& imageHeader) const
{
	return (m_imagesTriedToLoad.find(imageHeader) != m_imagesTriedToLoad.end());
}


bool LiveProcess::PrepareForRestart(void)
{
	// signal to the target process that a restart for this process was requested
	Event requestRestart(primitiveNames::RequestRestart(m_processId).c_str(), Event::Type::AUTO_RESET);
	requestRestart.Signal();

	// the client code in the target is now inside the lpp::lppWantsRestart() code block.
	// wait until it calls lpp::lppRestart() after finishing custom client code.
	// give the client 10 seconds to finish up.
	Event restartPrepared(primitiveNames::PreparedRestart(m_processId).c_str(), Event::Type::AUTO_RESET);
	const bool success = restartPrepared.WaitTimeout(10u * 1000u);
	if (success)
	{
		m_restartState = RestartState::SUCCESSFUL_PREPARE;
		return true;
	}
	else
	{
		LC_ERROR_USER("Client did not respond to restart request within 10 seconds, aborting restart (PID: %d)", m_processId);
		m_restartState = RestartState::FAILED_PREPARE;
		return false;
	}
}


void LiveProcess::WaitForExitBeforeRestart(void)
{
	if (m_restartState == RestartState::SUCCESSFUL_PREPARE)
	{
		// in case PrepareForRestart was successful, the client is now waiting for the signal to restart.
		// tell the client to exit now.
		Event executeRestart(primitiveNames::Restart(m_processId).c_str(), Event::Type::AUTO_RESET);
		executeRestart.Signal();

		// wait until the client terminates
		Process::Wait(m_processHandle);

		m_restartState = RestartState::SUCCESSFUL_EXIT;
	}
}


// BEGIN EPIC MOD
void LiveProcess::Restart(void* restartJob)
// END EPIC MOD
{
	if (m_restartState == RestartState::SUCCESSFUL_EXIT)
	{
		// restart the target application
		// BEGIN EPIC MOD - Force LiveCoding to start up for child processes
		std::wstring commandLine(m_commandLine);

		const std::wstring argument(L" -LiveCoding");
		if (commandLine.length() >= argument.length() && commandLine.compare(commandLine.length() - argument.length(), argument.length(), argument) != 0)
		{
			commandLine += argument;
		}
		// END EPIC MOD
		// BEGIN EPIC MOD - Prevent orphaned console instances if processes fail to restart. Job object will be duplicated into child process.
		Process::Context* context = Process::Spawn(m_imagePath.c_str(), m_workingDirectory.c_str(), commandLine.c_str(), m_environment.GetData(), Process::SpawnFlags::SUSPENDED);
		void* TargetHandle;
		DuplicateHandle(GetCurrentProcess(), restartJob, +Process::GetHandle(context), &TargetHandle, 0, Windows::TRUE, DUPLICATE_SAME_ACCESS);
		Process::ResumeMainThread(context);
		// END EPIC MOD

		m_restartState = RestartState::SUCCESSFUL_RESTART;
	}
}


bool LiveProcess::WasSuccessfulRestart(void) const
{
	return (m_restartState == RestartState::SUCCESSFUL_RESTART);
}


void LiveProcess::ReserveVirtualMemoryPages(void* moduleBase)
{
#if LC_64_BIT
	// BEGIN EPIC MOD
	const void* lowerBound = liveProcess::GetLowerBoundIn4GBRange(moduleBase);
	const void* upperBound = liveProcess::GetUpperBoundIn4GBRange(moduleBase);
	// END EPIC MOD
	m_virtualMemoryRange.ReservePages(lowerBound, upperBound, 64u * 1024u);
#else
	LC_UNUSED(moduleBase);
#endif
}


void LiveProcess::FreeVirtualMemoryPages(void* moduleBase)
{
#if LC_64_BIT
	// BEGIN EPIC MOD
	const void* lowerBound = liveProcess::GetLowerBoundIn4GBRange(moduleBase);
	const void* upperBound = liveProcess::GetUpperBoundIn4GBRange(moduleBase);
	// END EPIC MOD
	m_virtualMemoryRange.FreePages(lowerBound, upperBound);
#else
	LC_UNUSED(moduleBase);
#endif
}

// BEGIN EPIC MOD - Allow lazy-loading modules
void LiveProcess::AddLazyLoadedModule(const std::wstring moduleName, Windows::HMODULE moduleBase)
{
	LazyLoadedModule module;
	module.m_moduleBase = moduleBase;
	module.m_loaded = false;
	m_lazyLoadedModules.insert(std::make_pair(moduleName, module));
}

void LiveProcess::SetLazyLoadedModuleAsLoaded(const std::wstring moduleName)
{
	std::unordered_map<std::wstring, LazyLoadedModule>::iterator it = m_lazyLoadedModules.find(moduleName);
	if (it != m_lazyLoadedModules.end())
	{
		it->second.m_loaded = true;
	}
}

bool LiveProcess::IsPendingLazyLoadedModule(const std::wstring& moduleName) const
{
	std::unordered_map<std::wstring, LazyLoadedModule>::const_iterator iter = m_lazyLoadedModules.find(moduleName);
	return iter != m_lazyLoadedModules.end() && !iter->second.m_loaded;
}

Windows::HMODULE LiveProcess::GetLazyLoadedModuleBase(const std::wstring& moduleName) const
{
	std::unordered_map<std::wstring, LazyLoadedModule>::const_iterator iter = m_lazyLoadedModules.find(moduleName);
	if (iter == m_lazyLoadedModules.end())
	{
		return nullptr;
	}
	else
	{
		return iter->second.m_moduleBase;
	}
}
// END EPIC MOD

#endif