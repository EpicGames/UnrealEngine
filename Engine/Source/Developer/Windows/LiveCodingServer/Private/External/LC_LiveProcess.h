// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

#include "LC_ProcessTypes.h"
#include "LC_ThreadTypes.h"
#include "LC_Executable.h"
#include "LC_MemoryBlock.h"
#include "LC_VirtualMemoryRange.h"
// BEGIN EPIC MOD
#include "LC_Types.h"
#include "VisualStudioDTE.h"
// END EPIC MOD


class DuplexPipe;
class CodeCave;

class LiveProcess
{
public:
	LiveProcess(Process::Handle processHandle, Process::Id processId, Thread::Id commandThreadId, const void* jumpToSelf, const DuplexPipe* pipe,
		const wchar_t* imagePath, const wchar_t* commandLine, const wchar_t* workingDirectory, const void* environment, size_t environmentSize);

	void ReadHeartBeatDelta(const wchar_t* const processGroupName);

	// returns whether this process made some progress, based on the heart beat received from the client
	bool MadeProgress(void) const;

	// handles any debugging mechanism that might currently debug our process before we compile changes
	void HandleDebuggingPreCompile(void);

	// handles any debugging mechanism that might currently debug our process after we compiled changes
	void HandleDebuggingPostCompile(void);

	void InstallCodeCave(void);
	void UninstallCodeCave(void);

	void AddLoadedImage(const executable::Header& imageHeader);
	void RemoveLoadedImage(const executable::Header& imageHeader);
	bool TriedToLoadImage(const executable::Header& imageHeader) const;


	bool PrepareForRestart(void);
	void WaitForExitBeforeRestart(void);
	// BEGIN EPIC MOD
	void Restart(void* restartJob);
	// END EPIC MOD
	bool WasSuccessfulRestart(void) const;


	void ReserveVirtualMemoryPages(void* moduleBase);
	void FreeVirtualMemoryPages(void* moduleBase);


	inline Process::Handle GetProcessHandle(void) const
	{
		return m_processHandle;
	}

	inline Process::Id GetProcessId(void) const
	{
		return m_processId;
	}

	inline Thread::Id GetCommandThreadId(void) const
	{
		return m_commandThreadId;
	}

	inline const void* GetJumpToSelf(void) const
	{
		return m_jumpToSelf;
	}

	inline const DuplexPipe* GetPipe(void) const
	{
		return m_pipe;
	}

	// BEGIN EPIC MOD - Add build arguments
	inline void SetBuildArguments(const wchar_t* buildArguments)
	{
		m_buildArguments = buildArguments;
	}

	inline const wchar_t* GetBuildArguments()
	{
		return m_buildArguments.c_str();
	}
	// END EPIC MOD

	// BEGIN EPIC MOD - Allow lazy-loading modules
	struct LazyLoadedModule
	{
		Windows::HMODULE m_moduleBase;
		bool m_loaded;
	};

	void AddLazyLoadedModule(const std::wstring moduleName, Windows::HMODULE moduleBase);
	void SetLazyLoadedModuleAsLoaded(const std::wstring moduleName);
	bool IsPendingLazyLoadedModule(const std::wstring& moduleName) const;
	const types::unordered_map<std::wstring, LazyLoadedModule>& GetLazyLoadedModules() const { return m_lazyLoadedModules; }
	Windows::HMODULE GetLazyLoadedModuleBase(const std::wstring& moduleName) const;
	// END EPIC MOD

	// BEGIN EPIC MOD
	void SetReinstancingFlow(bool enable)
	{
		m_reinstancingFlow = enable;
	}
	bool IsReinstancingFlowEnabled() const
	{
		return m_reinstancingFlow;
	}
	// END EPIC MOD

	// BEGIN EPIC MOD
	void DisableCompileFinishNotification()
	{
		m_disableCompileFinishNotification = true;
	}
	bool IsDisableCompileFinishNotification()
	{
		return m_disableCompileFinishNotification;
	}
	// END EPIC MOD

	// BEGIN EPIC MOD
	void AddPage(void *page)
	{
		m_virtualMemoryRange.AddPage(page);
	}
	// END EPIC MOD

private:
	Process::Handle m_processHandle;
	Process::Id m_processId;
	Thread::Id m_commandThreadId;
	const void* m_jumpToSelf;
	const DuplexPipe* m_pipe;

	std::wstring m_imagePath;
	std::wstring m_commandLine;
	std::wstring m_workingDirectory;
	MemoryBlock m_environment;

	// BEGIN EPIC MOD - Add build arguments
	std::wstring m_buildArguments;
	// END EPIC MOD

	// BEGIN EPIC MOD - Allow lazy-loading modules
	types::unordered_map<std::wstring, LazyLoadedModule> m_lazyLoadedModules;
	// END EPIC MOD

	// BEGIN EPIC MOD
	bool m_reinstancingFlow = false;
	// END EPIC MOD

	// BEGIN EPIC MOD
	bool m_disableCompileFinishNotification = false;
	// END EPIC MOD

	// loaded modules are not identified by their full path, but by their executable image header.
	// we do this to ensure that the same executable loaded from a different path is not treated as
	// a different executable.
	types::unordered_set<executable::Header> m_imagesTriedToLoad;

	uint64_t m_heartBeatDelta;

// BEGIN EPIC MOD
#if WITH_VISUALSTUDIO_DTE
// END EPIC MOD
	// for handling communication with the VS debugger
	EnvDTE::DebuggerPtr m_vsDebugger;
	types::vector<EnvDTE::ThreadPtr> m_vsDebuggerThreads;
// BEGIN EPIC MOD
#endif
// END EPIC MOD

	// fallback in case communication with the VS debugger is not possible
	CodeCave* m_codeCave;

	// restart
	struct RestartState
	{
		enum Enum
		{
			DEFAULT,
			FAILED_PREPARE,
			SUCCESSFUL_PREPARE,
			SUCCESSFUL_EXIT,
			SUCCESSFUL_RESTART
		};
	};

	RestartState::Enum m_restartState;

	VirtualMemoryRange m_virtualMemoryRange;

	LC_DISABLE_COPY(LiveProcess);
	LC_DISABLE_MOVE(LiveProcess);
	LC_DISABLE_ASSIGNMENT(LiveProcess);
	LC_DISABLE_MOVE_ASSIGNMENT(LiveProcess);
};


#endif