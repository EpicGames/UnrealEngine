// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Mallocator.h"
// BEGIN EPIC MOD
#include "LC_Assert.h"
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
#include "LC_Logging.h"
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

namespace
{
	// HeapAlloc has roughly 16 byte overhead per allocation
	static const size_t PER_ALLOCATION_OVERHEAD = 16u;
}


Mallocator::Mallocator(const char* name, size_t alignment)
	: m_heap(::GetProcessHeap())
	, m_name(name)
	, m_alignment(alignment)
	, m_stats()
{
	LC_ASSERT(alignment <= 8u, "Desired alignment is too large.");
}


void* Mallocator::Allocate(size_t size, size_t alignment)
{
	LC_ASSERT(alignment <= m_alignment, "Desired alignment is larger than initial alignment.");

	m_stats.RegisterAllocation(size + PER_ALLOCATION_OVERHEAD);
	return ::HeapAlloc(m_heap, 0u, size);
}


void Mallocator::Free(void* ptr, size_t size)
{
	if (ptr)
	{
		m_stats.UnregisterAllocation(size + PER_ALLOCATION_OVERHEAD);
		::HeapFree(m_heap, 0u, ptr);
	}
}


void Mallocator::PrintStats(void) const
{
	m_stats.Print(m_name);
}


const AllocatorStats& Mallocator::GetStats(void) const
{
	return m_stats;
}


#endif