// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "Windows/MinimalWindowsAPI.h"
// END EPIC MOD
#include "LC_AllocatorStats.h"


// thread-safe
class Mallocator
{
public:
	Mallocator(const char* name, size_t alignment);

	void* Allocate(size_t size, size_t alignment);
	void Free(void* ptr, size_t size);

	void PrintStats(void) const;

	const AllocatorStats& GetStats(void) const;

private:
	// BEGIN EPIC MOD
	Windows::HANDLE m_heap;
	// END EPIC MOD
	const char* m_name;
	size_t m_alignment;
	AllocatorStats m_stats;
};


#endif