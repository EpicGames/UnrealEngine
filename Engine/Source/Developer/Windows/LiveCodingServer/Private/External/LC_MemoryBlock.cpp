// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_MemoryBlock.h"


MemoryBlock::MemoryBlock(const void* data, size_t size)
	: m_data(malloc(size))
	, m_size(size)
{
	memcpy(m_data, data, size);
}


MemoryBlock::~MemoryBlock(void)
{
	free(m_data);
}


#endif