// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
// END EPIC MOD

class MemoryBlock
{
public:
	MemoryBlock(const void* data, size_t size);
	~MemoryBlock(void);

	inline const void* GetData(void) const
	{
		return m_data;
	}

	inline size_t GetSize(void) const
	{
		return m_size;
	}

private:
	void* m_data;
	size_t m_size;

	LC_DISABLE_COPY(MemoryBlock);
	LC_DISABLE_MOVE(MemoryBlock);
	LC_DISABLE_ASSIGNMENT(MemoryBlock);
	LC_DISABLE_MOVE_ASSIGNMENT(MemoryBlock);
};


#endif