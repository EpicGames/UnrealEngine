// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_MicroAllocator.h"
#include "LC_VirtualMemory.h"
#include "LC_BitUtil.h"
// BEGIN EPIC MOD
#include "LC_Assert.h"
#include "LC_PoolAllocator.h"
#include "LC_Platform.h"
#include "LC_Logging.h"
// END EPIC MOD


MicroAllocator::MicroAllocator(const char* name, size_t alignment)
	: m_poolAllocators()
	, m_blockAllocator(name, alignment)
	, m_name(name)
{
	// create pool allocators for each allocation size in multiples of 4
	for (size_t i = 4u; i <= POOL_ALLOCATOR_COUNT; i += 4u)
	{
		const size_t elementSize = i;

		// all pools grow such that new blocks need to be allocated for every ELEMENT_COUNT_PER_POOL_GROWTH elements
		const uint32_t pageSize = VirtualMemory::GetPageSize();
		const size_t growSize = bitUtil::RoundUpToMultiple<size_t>(elementSize * ELEMENT_COUNT_PER_POOL_GROWTH, pageSize);
		m_poolAllocators[i] = new PoolAllocator<PoolAllocatorMultiThreadPolicy>(name, elementSize, alignment, growSize);
	}

	// store all those allocators into a lookup-table for fast access during allocation
	for (size_t i = 1u; i <= POOL_ALLOCATOR_COUNT; ++i)
	{
		const size_t nextMultiple = bitUtil::RoundUpToMultiple<size_t>(i, 4u);
		m_poolAllocators[i] = m_poolAllocators[nextMultiple];
	}
}


MicroAllocator::~MicroAllocator(void)
{
	for (size_t i = 4u; i <= POOL_ALLOCATOR_COUNT; i += 4u)
	{
		delete m_poolAllocators[i];
	}
}


void* MicroAllocator::Allocate(size_t size, size_t alignment)
{
	LC_ASSERT(size > 0u, "Allocations of zero size are not allowed.");
	if (size <= POOL_ALLOCATOR_COUNT)
	{
		PoolAllocator<PoolAllocatorMultiThreadPolicy>* allocator = m_poolAllocators[size];
		return allocator->Allocate(size, alignment);
	}

	// large blocks go to the mallocator
	return m_blockAllocator.Allocate(size, alignment);
}


void MicroAllocator::Free(void* ptr, size_t size)
{
	if (size <= POOL_ALLOCATOR_COUNT)
	{
		PoolAllocator<PoolAllocatorMultiThreadPolicy>* allocator = m_poolAllocators[size];
		allocator->Free(ptr, size);
	}
	else
	{
		m_blockAllocator.Free(ptr, size);
	}
}


void MicroAllocator::PrintStats(void) const
{
	AllocatorStats stats;
	for (size_t i = 4u; i <= POOL_ALLOCATOR_COUNT; i += 4u)
	{
		const PoolAllocator<PoolAllocatorMultiThreadPolicy>* allocator = m_poolAllocators[i];
		stats.Merge(allocator->GetStats());
	}
	stats.Merge(m_blockAllocator.GetStats());
	stats.Print(m_name);
}


#endif