// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Mallocator.h"

class PoolAllocatorMultiThreadPolicy;
template <typename T> class PoolAllocator;
class Mallocator;


// thread-safe
class MicroAllocator
{
	static const size_t POOL_ALLOCATOR_COUNT = 1024u;
	static const size_t ELEMENT_COUNT_PER_POOL_GROWTH = 128u;

public:
	MicroAllocator(const char* name, size_t alignment);
	~MicroAllocator(void);

	void* Allocate(size_t size, size_t alignment);
	void Free(void* ptr, size_t size);

	void PrintStats(void) const;

private:
	PoolAllocator<PoolAllocatorMultiThreadPolicy>* m_poolAllocators[POOL_ALLOCATOR_COUNT + 1u];
	Mallocator m_blockAllocator;
	const char* m_name;
};


#endif