// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_ModuleCache.h"
#include "LC_LiveProcess.h"


ModuleCache::ModuleCache(void)
	: m_cs()
	, m_cache()
{
	m_cache.reserve(128u);
}

// BEGIN EPIC MOD
size_t ModuleCache::Insert(const symbols::SymbolDB* symbolDb, const symbols::ContributionDB* contributionDb, const symbols::CompilandDB* compilandDb, const symbols::ThunkDB* thunkDb, const symbols::ImageSectionDB* imageSectionDb, uint64_t lastModicationTime)
// END EPIC MOD
{
	CriticalSection::ScopedLock lock(&m_cs);

	const size_t token = m_cache.size();
	m_cache.push_back(Data { static_cast<uint16_t>(token), symbolDb, contributionDb, compilandDb, thunkDb, imageSectionDb, lastModicationTime, {} });

	return token;
}


void ModuleCache::RegisterProcess(size_t token, LiveProcess* liveProcess, void* moduleBase)
{
	CriticalSection::ScopedLock lock(&m_cs);

	m_cache[token].processes.emplace_back(ProcessData { liveProcess->GetProcessId(), liveProcess->GetProcessHandle(), liveProcess->GetPipe(), moduleBase });
}


void ModuleCache::UnregisterProcess(LiveProcess* liveProcess)
{
	CriticalSection::ScopedLock lock(&m_cs);

	const size_t count = m_cache.size();
	for (size_t i = 0u; i < count; ++i)
	{
		Data& data = m_cache[i];

		for (auto it = data.processes.begin(); it != data.processes.end(); /* nothing */)
		{
			const ProcessData& process = *it;
			if (process.processId == liveProcess->GetProcessId())
			{
				it = data.processes.erase(it);
			}
			else
			{
				++it;
			}
		}
	}
}


ModuleCache::FindSymbolData ModuleCache::FindSymbolByName(size_t ignoreToken, const ImmutableString& symbolName) const
{
	CriticalSection::ScopedLock lock(&m_cs);

	const size_t count = m_cache.size();
	for (size_t i = 0u; i < count; ++i)
	{
		if (i == ignoreToken)
		{
			continue;
		}

		const Data& data = m_cache[i];
		const symbols::Symbol* symbol = symbols::FindSymbolByName(data.symbolDb, symbolName);
		if (symbol)
		{
			return FindSymbolData { &data, symbol };
		}
	}

	return FindSymbolData {};
}


// BEGIN EPIC MOD
ModuleCache::FindSymbolData ModuleCache::FindSymbolByNameBackwards(size_t ignoreToken, const ImmutableString& symbolName) const
{
	CriticalSection::ScopedLock lock(&m_cs);

	const size_t count = m_cache.size();
	for (size_t i = 0u; i < count; ++i)
	{
		const size_t index = count - 1u - i;
		if (index == ignoreToken)
		{
			continue;
		}

		const Data& data = m_cache[index];
		const symbols::Symbol* symbol = symbols::FindSymbolByName(data.symbolDb, symbolName);
		if (symbol)
		{
			return FindSymbolData{ &data, symbol };
		}
	}

	return FindSymbolData{};
}
// END EPIC MOD


ModuleCache::FindHookData ModuleCache::FindHooksInSectionBackwards(size_t ignoreToken, const ImmutableString& sectionName) const
{
	CriticalSection::ScopedLock lock(&m_cs);

	const size_t count = m_cache.size();
	for (size_t i = 0u; i < count; ++i)
	{
		const size_t index = count - 1u - i;
		if (index == ignoreToken)
		{
			continue;
		}

		const Data& data = m_cache[index];
		const uint32_t firstRva = hook::FindFirstInSection(data.imageSectionDb, sectionName);
		if (firstRva != 0u)
		{
			const uint32_t lastRva = hook::FindLastInSection(data.imageSectionDb, sectionName);
			if (lastRva != 0u)
			{
				return FindHookData { &data, firstRva, lastRva };
			}
		}
	}

	return FindHookData {};
}


types::vector<void*> ModuleCache::GatherModuleBases(Process::Id processId) const
{
	types::vector<void*> result;

	CriticalSection::ScopedLock lock(&m_cs);

	const size_t count = m_cache.size();
	result.resize(count, nullptr);

	for (size_t i = 0u; i < count; ++i)
	{
		const Data& data = m_cache[i];
		for (auto it : data.processes)
		{
			if (it.processId == processId)
			{
				result[i] = it.moduleBase;
				break;
			}
		}
	}

	return result;
}


#endif