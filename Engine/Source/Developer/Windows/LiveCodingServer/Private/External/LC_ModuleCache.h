// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Symbols.h"
#include "LC_Hook.h"
#include "LC_CriticalSection.h"
#include "LC_ProcessTypes.h"


class LiveProcess;
class DuplexPipe;

// thread-safe
class ModuleCache
{
public:
	static const size_t SEARCH_ALL_MODULES = static_cast<size_t>(-1);

	struct ProcessData
	{
		// all data except moduleBase is redundant and stored per cache entry, but this doesn't
		// increase memory requirements by much. we'd rather have all information accessible fast.
		Process::Id processId;
		Process::Handle processHandle;
		const DuplexPipe* pipe;

		void* moduleBase;
	};

	struct Data
	{
		uint16_t index;						// index of the patch corresponding to the data (0 = original executable)
		const symbols::SymbolDB* symbolDb;
		const symbols::ContributionDB* contributionDb;
		const symbols::CompilandDB* compilandDb;
		const symbols::ThunkDB* thunkDb;
		const symbols::ImageSectionDB* imageSectionDb;
		// BEGIN EPIC MOD
		uint64_t lastModificationTime;
		// END EPIC MOD

		types::vector<ProcessData> processes;	// all processes that this patch is loaded into
	};

	struct FindSymbolData
	{
		const Data* data;
		const symbols::Symbol* symbol;
	};

	struct FindHookData
	{
		const Data* data;
		uint32_t firstRva;
		uint32_t lastRva;
	};

	ModuleCache(void);

	
	// adds an entry to the cache. does not take ownership of the databases.
	// returns a token for registering a process associated with this entry.
	// BEGIN EPIC MOD
	size_t Insert(const symbols::SymbolDB* symbolDb, const symbols::ContributionDB* contributionDb, const symbols::CompilandDB* compilandDb, const symbols::ThunkDB* thunkDb, const symbols::ImageSectionDB* imageSectionDb, uint64_t lastModicationTime);
	// END EPIC MOD

	// associates a process with an entry identified by a previously returned token.
	void RegisterProcess(size_t token, LiveProcess* liveProcess, void* moduleBase);

	// removes a process from all entries
	void UnregisterProcess(LiveProcess* liveProcess);


	// tries finding a symbol by name, starting from the first module, walking to the latest, excluding the module with the given token
	FindSymbolData FindSymbolByName(size_t ignoreToken, const ImmutableString& symbolName) const;

	// BEGIN EPIC MOD
	// tries finding a symbol by name, starting from the last module, walking to the first, excluding the module with the given token
	FindSymbolData FindSymbolByNameBackwards(size_t ignoreToken, const ImmutableString& symbolName) const;
	// END EPIC MOD

	// tries finding the first and last hook in a given section, starting from the newest module, walking to the first, excluding the module with the given token
	FindHookData FindHooksInSectionBackwards(size_t ignoreToken, const ImmutableString& sectionName) const;


	types::vector<void*> GatherModuleBases(Process::Id processId) const;


	inline size_t GetSize(void) const
	{
		return m_cache.size();
	}

	inline const Data& GetEntry(size_t i) const
	{
		return m_cache[i];
	}

private:
	mutable CriticalSection m_cs;
	types::vector<Data> m_cache;
};


#endif