// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_ModulePatch.h"
#include "LC_AppSettings.h"


ModulePatch::ModulePatch(const std::wstring& exePath, const std::wstring& pdbPath, size_t token)
	: m_exePath(exePath)
	, m_pdbPath(pdbPath)
	, m_token(token)
	, m_data()
{
}


void ModulePatch::RegisterEntryPointCode(const uint8_t* code)
{
	memcpy(m_data.entryPointCode, code, sizeof(m_data.entryPointCode));
}


void ModulePatch::RegisterPrePatchHooks(uint16_t moduleIndex, uint32_t firstRva, uint32_t lastRva)
{
	m_data.prePatchHookModuleIndex = moduleIndex;
	m_data.firstPrePatchHook = firstRva;
	m_data.lastPrePatchHook = lastRva;
}


void ModulePatch::RegisterPostPatchHooks(uint16_t moduleIndex, uint32_t firstRva, uint32_t lastRva)
{
	m_data.postPatchHookModuleIndex = moduleIndex;
	m_data.firstPostPatchHook = firstRva;
	m_data.lastPostPatchHook = lastRva;
}


void ModulePatch::RegisterSecurityCookie(uint32_t originalRva, uint32_t patchRva)
{
	m_data.originalCookieRva = originalRva;
	m_data.patchCookieRva = patchRva;
}


void ModulePatch::RegisterUe4NameTable(uint32_t originalRva, uint32_t patchRva)
{
	m_data.originalUe4NameTableRva = originalRva;
	m_data.patchUe4NameTableRva = patchRva;
}


void ModulePatch::RegisterUe4ObjectArray(uint32_t originalRva, uint32_t patchRva)
{
	m_data.originalUe4ObjectArrayRva = originalRva;
	m_data.patchUe4NameTableRva = patchRva;
}


void ModulePatch::RegisterDllMain(uint32_t rva)
{
	m_data.dllMainRva = rva;
}


void ModulePatch::RegisterPreEntryPointRelocation(const relocations::Record& record)
{
	// this needs additional memory, only store data when the corresponding feature is enabled
	if (!appSettings::g_installCompiledPatchesMultiProcess->GetValue())
	{
		return;
	}

	m_data.preEntryPointRelocations.emplace_back(record);
}


void ModulePatch::RegisterPostEntryPointRelocation(const relocations::Record& record)
{
	// this needs additional memory, only store data when the corresponding feature is enabled
	if (!appSettings::g_installCompiledPatchesMultiProcess->GetValue())
	{
		return;
	}

	m_data.postEntryPointRelocations.emplace_back(record);
}


void ModulePatch::RegisterPatchedDynamicInitializer(uint32_t rva)
{
	// this needs additional memory, only store data when the corresponding feature is enabled
	if (!appSettings::g_installCompiledPatchesMultiProcess->GetValue())
	{
		return;
	}

	m_data.patchedInitializers.emplace_back(rva);
}


void ModulePatch::RegisterFunctionPatch(const functions::Record& record)
{
	// this needs additional memory, only store data when the corresponding feature is enabled
	if (!appSettings::g_installCompiledPatchesMultiProcess->GetValue())
	{
		return;
	}

	m_data.functionPatches.emplace_back(record);
}


void ModulePatch::RegisterLibraryFunctionPatch(const functions::LibraryRecord& record)
{
	// this needs additional memory, only store data when the corresponding feature is enabled
	if (!appSettings::g_installCompiledPatchesMultiProcess->GetValue())
	{
		return;
	}

	m_data.libraryFunctionPatches.emplace_back(record);
}


const std::wstring& ModulePatch::GetExePath(void) const
{
	return m_exePath;
}


const std::wstring& ModulePatch::GetPdbPath(void) const
{
	return m_pdbPath;
}


size_t ModulePatch::GetToken(void) const
{
	return m_token;
}


const ModulePatch::Data& ModulePatch::GetData(void) const
{
	return m_data;
}


#endif