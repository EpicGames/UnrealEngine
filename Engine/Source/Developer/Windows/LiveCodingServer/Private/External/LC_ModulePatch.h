// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIn EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_RelocationPatcher.h"
#include "LC_FunctionPatcher.h"
#include "LC_ExecutablePatcher.h"


class ModulePatch
{
public:
	struct Data
	{
		uint8_t entryPointCode[ExecutablePatcher::INJECTED_CODE_SIZE];

		uint16_t prePatchHookModuleIndex;
		uint32_t firstPrePatchHook;
		uint32_t lastPrePatchHook;

		uint16_t postPatchHookModuleIndex;
		uint32_t firstPostPatchHook;
		uint32_t lastPostPatchHook;

		// security cookie
		uint32_t originalCookieRva;
		uint32_t patchCookieRva;

		// UE4-specific
		uint32_t originalUe4NameTableRva;
		uint32_t patchUe4NameTableRva;
		uint32_t originalUe4ObjectArrayRva;
		uint32_t patchUe4ObjectArrayRva;

		uint32_t dllMainRva;

		types::vector<relocations::Record> preEntryPointRelocations;
		types::vector<relocations::Record> postEntryPointRelocations;

		types::vector<uint32_t> patchedInitializers;

		types::vector<functions::Record> functionPatches;
		types::vector<functions::LibraryRecord> libraryFunctionPatches;
	};

	ModulePatch(const std::wstring& exePath, const std::wstring& pdbPath, size_t token);

	void RegisterEntryPointCode(const uint8_t* code);

	void RegisterPrePatchHooks(uint16_t moduleIndex, uint32_t firstRva, uint32_t lastRva);
	void RegisterPostPatchHooks(uint16_t moduleIndex, uint32_t firstRva, uint32_t lastRva);

	// Security cookie
	void RegisterSecurityCookie(uint32_t originalRva, uint32_t patchRva);

	// UE4-specific
	void RegisterUe4NameTable(uint32_t originalRva, uint32_t patchRva);
	void RegisterUe4ObjectArray(uint32_t originalRva, uint32_t patchRva);

	void RegisterDllMain(uint32_t rva);

	void RegisterPreEntryPointRelocation(const relocations::Record& record);
	void RegisterPostEntryPointRelocation(const relocations::Record& record);

	void RegisterPatchedDynamicInitializer(uint32_t rva);

	void RegisterFunctionPatch(const functions::Record& record);
	void RegisterLibraryFunctionPatch(const functions::LibraryRecord& record);

	const std::wstring& GetExePath(void) const;
	const std::wstring& GetPdbPath(void) const;
	size_t GetToken(void) const;

	const Data& GetData(void) const;

private:
	std::wstring m_exePath;
	std::wstring m_pdbPath;
	size_t m_token;

	Data m_data;
};


#endif