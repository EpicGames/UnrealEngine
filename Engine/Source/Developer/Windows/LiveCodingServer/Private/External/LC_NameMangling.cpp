// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPC MOD
#include "LC_NameMangling.h"
#include "LC_StringUtil.h"


typedef void* (*malloc_func_t)(size_t);
typedef void (*free_func_t)(void*);

// undocumented function found in MSVC CRT
extern "C" char* __cdecl __unDName(char* buffer, const char* mangled, int buflen, malloc_func_t memget, free_func_t memfree, unsigned short flags);


std::string nameMangling::UndecorateSymbol(const char* symbolName, uint16_t flags)
{
	char buffer[65536];
	__unDName(buffer, symbolName, 65536, malloc, free, flags);

	return std::string(buffer);
}


std::wstring nameMangling::UndecorateSymbolWide(const char* symbolName, uint16_t flags)
{
	char buffer[65536];
	__unDName(buffer, symbolName, 65536, malloc, free, flags);

	return string::ToWideString(buffer);
}


#endif