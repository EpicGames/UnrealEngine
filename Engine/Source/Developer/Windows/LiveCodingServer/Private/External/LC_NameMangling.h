// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include <string>
// END EPIC MOD

namespace nameMangling
{
	// these are the undocumented flags available when undecorating symbols.
	// they were discovered by running the MSVC tool 'undname' with command-line option '/show_flags'.

	// 0x0001	Remove leading underscores from Microsoft extended keywords
	// 0x0002	Disable expansion of Microsoft extended keywords
	// 0x0004	Disable expansion of return type for primary declaration
	// 0x0008	Disable expansion of the declaration model
	// 0x0010	Disable expansion of the declaration language specifier
	// 0x0060	Disable all modifiers on the 'this' type
	// 0x0080	Disable expansion of access specifiers for members
	// 0x0100	Disable expansion of 'throw-signatures' for functions and pointers to functions
	// 0x0200	Disable expansion of 'static' or 'virtual'ness of members
	// 0x0400	Disable expansion of Microsoft model for UDT returns
	// 0x0800	Undecorate 32 - bit decorated names
	// 0x1000	Crack only the name for primary declaration
	// 			return just[scope::]name. Does expand template params
	// 0x2000	Input is just a type encoding; compose an abstract declarator
	// 0x8000	Disable enum / class / struct / union prefix
	// 0x20000	Disable expansion of __ptr64 keyword -> this cannot be used because it doesn't fit into 16-bit flags
	std::string UndecorateSymbol(const char* symbolName, uint16_t flags);
	std::wstring UndecorateSymbolWide(const char* symbolName, uint16_t flags);
}


#endif