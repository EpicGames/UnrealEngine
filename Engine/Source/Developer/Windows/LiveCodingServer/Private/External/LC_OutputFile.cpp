// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_OutputFile.h"
#include "LC_Filesystem.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
// END EPIC MOD


OutputFile::OutputFile(const wchar_t* logFilePath)
	: m_logFile(nullptr)
{
	m_logFile = ::CreateFileW(logFilePath, GENERIC_WRITE, 0, nullptr, CREATE_ALWAYS, FILE_ATTRIBUTE_NORMAL, nullptr);
	if (m_logFile == INVALID_HANDLE_VALUE)
	{
		LC_ERROR_USER("Cannot obtain handle for file %S. Error: 0x%X", logFilePath, ::GetLastError());
	}
	else
	{
		LC_LOG_USER("Creating log file at %S", Filesystem::NormalizePath(logFilePath).GetString());
	}
}


OutputFile::~OutputFile(void)
{
	::CloseHandle(m_logFile);
}


void OutputFile::Log(const char* msg)
{
	WriteToFile(msg);
}


void OutputFile::WriteToFile(const char* text)
{
	const bool isValidFile = (m_logFile && (m_logFile != INVALID_HANDLE_VALUE));
	if (isValidFile)
	{
		DWORD bytesWritten = 0u;
		const size_t length = strlen(text);
		::WriteFile(m_logFile, text, static_cast<DWORD>(length), &bytesWritten, nullptr);
	}
}

#endif