// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "Windows/MinimalWindowsAPI.h"
// END EPIC MOD

class OutputFile
{
public:
	explicit OutputFile(const wchar_t* logFilePath);
	~OutputFile(void);

	void Log(const char* msg);

private:
	void WriteToFile(const char* text);

	// BEGIN EPIC MOD
	Windows::HANDLE m_logFile;
	// END EPIC MOD
};


#endif