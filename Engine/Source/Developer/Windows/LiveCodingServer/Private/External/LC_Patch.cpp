// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Patch.h"
#include "LC_Assembler.h"
#include "LC_PointerUtil.h"
#include "LC_Process.h"
// BEGIN EPIC MOD
#include "LC_Assert.h"
#include "LC_Logging.h"
// END EPIC MOD

// BEGIN EPIC MODS
#pragma warning(push)
#pragma warning(disable:4686) // possible change in behavior, change in UDT return calling convention
// END EPIC MODS

namespace
{
	void Install(Process::Handle processHandle, void* address, const Assembler::Instruction& instruction)
	{
		Process::WriteProcessMemory(processHandle, address, instruction.machineCode, instruction.size);
		Process::FlushInstructionCache(processHandle, address, instruction.size);
	}


	template <uint8_t N>
	void Install(Process::Handle processHandle, void* address, const Assembler::SizedInstruction<N>& instruction)
	{
		Process::WriteProcessMemory(processHandle, address, instruction.machineCode, N);
		Process::FlushInstructionCache(processHandle, address, N);
	}
}


void patch::InstallNOPs(Process::Handle processHandle, void* address, uint8_t size)
{
	const Assembler::Instruction& nop = Assembler::MakeNOP(size);
	Install(processHandle, address, nop);
}


void patch::InstallJumpToSelf(Process::Handle processHandle, void* address)
{
	InstallRelativeShortJump(processHandle, address, address);
}


void patch::InstallRelativeShortJump(Process::Handle processHandle, void* address, void* destination)
{
	char* oldFuncAddr = static_cast<char*>(address);
	char* newFuncAddr = static_cast<char*>(destination);

	const ptrdiff_t displacement = newFuncAddr - oldFuncAddr;
	LC_ASSERT(displacement >= std::numeric_limits<int8_t>::min(), "Displacement is out-of-range.");
	LC_ASSERT(displacement <= std::numeric_limits<int8_t>::max(), "Displacement is out-of-range.");

	Install(processHandle, address, Assembler::MakeJMP8Relative(static_cast<int8_t>(displacement)));
}


void patch::InstallRelativeNearJump(Process::Handle processHandle, void* address, void* destination)
{
	char* oldFuncAddr = static_cast<char*>(address);
	char* newFuncAddr = static_cast<char*>(destination);

	const ptrdiff_t displacement = newFuncAddr - oldFuncAddr;
	LC_ASSERT(displacement >= std::numeric_limits<int32_t>::min(), "Displacement is out-of-range.");
	LC_ASSERT(displacement <= std::numeric_limits<int32_t>::max(), "Displacement is out-of-range.");

	Install(processHandle, address, Assembler::MakeJMP32Relative(static_cast<int32_t>(displacement)));
}

// BEGIN EPIC MODS
#pragma warning(pop)
// END EPIC MODS


#endif