// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ProcessTypes.h"


namespace patch
{
	void InstallNOPs(Process::Handle processHandle, void* address, uint8_t size);
	void InstallJumpToSelf(Process::Handle processHandle, void* address);

	void InstallRelativeShortJump(Process::Handle processHandle, void* address, void* destination);
	void InstallRelativeNearJump(Process::Handle processHandle, void* address, void* destination);
}


#endif