// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_PoolAllocator.h"
#include "LC_PointerUtil.h"
#include "LC_VirtualMemory.h"
#include "LC_BitUtil.h"
// BEGIN EPIC MOD
#include "LC_Assert.h"
#include "LC_Platform.h"
#include "LC_Logging.h"
// END EPIC MOD


namespace freeList
{
	struct Slot
	{
		Slot* next;
	};

	void* Initialize(void* memoryBlock, size_t memorySize, size_t originalElementSize, size_t alignment, size_t headerSize)
	{
		union
		{
			void* as_void;
			char* as_char;
			Slot* as_slot;
		};

		// determine the proper element size for satisfying alignment restrictions.
		// no matter the element size (be it smaller or larger than the desired alignment), rounding to the next multiple of
		// the alignment will always satisfy restrictions while producing the least amount of wasted/unused memory.
		// furthermore, we must always be able to store a Freelist* in each entry.
		const size_t minimumSize = sizeof(Slot*) > originalElementSize ? sizeof(Slot*) : originalElementSize;
		const size_t elementSize = bitUtil::RoundUpToMultiple(minimumSize, alignment);

		// in order to satisfy alignment restrictions, it suffices to offset the start of the free list only
		as_void = memoryBlock;
		as_char = pointer::AlignTop<char*>(as_char + headerSize, alignment);

		// next points to the first entry in the free list
		Slot* next = as_slot;

		// determine the number of elements that fit into the given memory range
		const size_t numElements = (static_cast<char*>(memoryBlock) + memorySize - as_char) / elementSize;

		as_char += elementSize;

		// initialize the free list
		Slot* runner = next;
		for (size_t i = 1u; i < numElements; ++i)
		{
			runner->next = as_slot;
			runner = as_slot;
			as_char += elementSize;
		}

		runner->next = nullptr;

		return next;
	}
}


template <typename T>
PoolAllocator<T>::PoolAllocator(const char* name, size_t maxElementSize, size_t maxAlignment, size_t growSize)
	: m_freeList(nullptr)
	, m_threadPolicy()
	, m_maxSize(maxElementSize)
	, m_maxAlignment(maxAlignment)
	, m_growSize(growSize)
	, m_name(name)
	, m_stats()
	, m_blockList(nullptr)
{
}


template <typename T>
PoolAllocator<T>::~PoolAllocator(void)
{
	Purge();
}


template <typename T>
void* PoolAllocator<T>::Allocate(size_t size, size_t alignment)
{
	LC_ASSERT(size <= m_maxSize, "Size of allocation too large for pool allocator.");
	LC_ASSERT(alignment <= m_maxAlignment, "Alignment of allocation too large for pool allocator.");

	m_threadPolicy.Enter();

	if (!m_freeList)
	{
		// no memory left, allocate a new block
		void* block = VirtualMemory::Allocate(m_growSize);

		// initialize free list in this block of memory
		m_freeList = freeList::Initialize(block, m_growSize, m_maxSize, m_maxAlignment, sizeof(BlockHeader));

		m_stats.RegisterAllocation(m_growSize);

		// add this block to the linked list of blocks
		BlockHeader* header = static_cast<BlockHeader*>(block);
		header->next = m_blockList;
		m_blockList = header;
	}

	// obtain one element from the head of the free list
	freeList::Slot* head = static_cast<freeList::Slot*>(m_freeList);
	m_freeList = head->next;

	m_threadPolicy.Leave();

	return head;
}


template <typename T>
void PoolAllocator<T>::Free(void* ptr, size_t)
{
	if (!ptr)
	{
		return;
	}

	m_threadPolicy.Enter();

	freeList::Slot* head = static_cast<freeList::Slot*>(ptr);

	// put the returned element at the head of the free list
	head->next = static_cast<freeList::Slot*>(m_freeList);
	m_freeList = head;

	m_threadPolicy.Leave();
}


template <typename T>
void PoolAllocator<T>::Purge(void)
{
	m_threadPolicy.Enter();

	BlockHeader* header = m_blockList;
	while (header)
	{
		BlockHeader* temp = header;
		header = header->next;
		VirtualMemory::Free(temp);

		m_stats.UnregisterAllocation(m_growSize);
	}
	m_blockList = nullptr;

	m_threadPolicy.Leave();
}


template <typename T>
void PoolAllocator<T>::PrintStats(void) const
{
	m_stats.Print(m_name);
}


template <typename T>
const AllocatorStats& PoolAllocator<T>::GetStats(void) const
{
	return m_stats;
}


// explicit template instantiation
template class PoolAllocator<PoolAllocatorSingleThreadPolicy>;
template class PoolAllocator<PoolAllocatorMultiThreadPolicy>;


#endif