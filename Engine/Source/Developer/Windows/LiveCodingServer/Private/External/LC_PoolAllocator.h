// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_AllocatorStats.h"
#include "LC_CriticalSection.h"


struct PoolAllocatorSingleThreadPolicy
{
	inline void Enter(void) {}
	inline void Leave(void) {}
};

class PoolAllocatorMultiThreadPolicy
{
public:
	inline void Enter(void)
	{
		m_cs.Enter();
	}

	inline void Leave(void)
	{
		m_cs.Leave();
	}

private:
	CriticalSection m_cs;
};


// thread-safe when used with a thread-safe policy
template <typename T>
class PoolAllocator
{
public:
	PoolAllocator(const char* name, size_t maxElementSize, size_t maxAlignment, size_t growSize);
	~PoolAllocator(void);

	void* Allocate(size_t size, size_t alignment);
	void Free(void* ptr, size_t size);

	void Purge(void);

	void PrintStats(void) const;

	const AllocatorStats& GetStats(void) const;

private:
	struct BlockHeader
	{
		BlockHeader* next;
	};

	void* m_freeList;
	T m_threadPolicy;
	size_t m_maxSize;
	size_t m_maxAlignment;
	size_t m_growSize;
	const char* m_name;
	AllocatorStats m_stats;

	// intrusive linked list
	BlockHeader* m_blockList;
};


#endif