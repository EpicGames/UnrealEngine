// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_RelocationPatcher.h"
#include "LC_StringUtil.h"
#include "LC_NameMangling.h"
#include "LC_PointerUtil.h"
#include "LC_CoffDetail.h"
// BEGIN EPIC MOD
#include "LC_Process.h"
// END EPIC MOD


bool relocations::WouldPatchRelocation(const ImmutableString& dstSymbolName)
{
	if (symbols::IsExceptionRelatedSymbol(dstSymbolName))
	{
		return false;
	}
	else if (symbols::IsVTable(dstSymbolName))
	{
		return false;
	}
	else if (symbols::IsRuntimeCheckRelatedSymbol(dstSymbolName))
	{
		return false;
	}
	else if (symbols::IsImageBaseRelatedSymbol(dstSymbolName))
	{
		return false;
	}
	else if (symbols::IsTlsArrayRelatedSymbol(dstSymbolName))
	{
		return false;
	}
	else if (string::Contains(dstSymbolName.c_str(), "$__resumable"))
	{
		return false;
	}

	return true;
}


bool relocations::WouldPatchRelocation
(
	const coff::Relocation* relocation,
	const coff::CoffDB* coffDb,
	const ImmutableString& srcSymbolName,
	const ModuleCache::FindSymbolData& originalData
)
{
	const coff::Relocation::Type::Enum type = relocation->type;
	const uint32_t characteristics = coff::GetRelocationDestinationSectionCharacteristics(coffDb, relocation);

	if (coffDetail::IsReadSection(characteristics) && !coffDetail::IsWriteSection(characteristics) && !coffDetail::IsCodeSection(characteristics))
	{
		return false;
	}
	else if (symbols::IsExceptionRelatedSymbol(srcSymbolName))
	{
		return false;
	}

	switch (type)
	{
		case coff::Relocation::Type::RELATIVE:

#if LC_64_BIT
		case coff::Relocation::Type::RELATIVE_OFFSET_1:
		case coff::Relocation::Type::RELATIVE_OFFSET_2:
		case coff::Relocation::Type::RELATIVE_OFFSET_3:
		case coff::Relocation::Type::RELATIVE_OFFSET_4:
		case coff::Relocation::Type::RELATIVE_OFFSET_5:
#endif
			return true;

		case coff::Relocation::Type::SECTION_RELATIVE:
		{
			const ImmutableString& sectionName = coff::GetTlsSectionName();
			const symbols::ImageSection* imageSection = symbols::FindImageSectionByName(originalData.data->imageSectionDb, sectionName);
			if (imageSection)
			{
				return true;
			}

			return false;
		}

		case coff::Relocation::Type::VA_32:
#if LC_64_BIT
			return false;
#else
			return true;
#endif

		case coff::Relocation::Type::RVA_32:
			return true;

#if LC_64_BIT
		case coff::Relocation::Type::VA_64:
			return true;
#endif

		case coff::Relocation::Type::UNKNOWN:
		default:
			return false;
	}
}


relocations::Record relocations::PatchRelocation
(
	const coff::Relocation* relocation,
	const coff::CoffDB* coffDb,
	const types::StringSet& forceRelocationSymbols,
	const ModuleCache* moduleCache,
	const ImmutableString& srcSymbolName,
	const ImmutableString& dstSymbolName,
	const symbols::Symbol* srcSymbol,
	size_t newModuleIndex,
	void* newModuleBases[]
	// BEGIN EPIC MOD
	, bool forceBackwards
	// END EPIC MOD
)
{
	Record record = { coff::Relocation::Type::UNKNOWN, 0u, 0u, {} };

	const coff::Relocation::Type::Enum type = relocation->type;
	const uint32_t characteristics = coff::GetRelocationDestinationSectionCharacteristics(coffDb, relocation);

	const bool forceRelocation = (forceRelocationSymbols.find(dstSymbolName) != forceRelocationSymbols.end());
	if (!forceRelocation)
	{
		// ignore relocations to anything that is read-only
		if (coffDetail::IsReadSection(characteristics) && !coffDetail::IsWriteSection(characteristics) && !coffDetail::IsCodeSection(characteristics))
		{
			LC_LOG_DEV("Ignoring relocation to %s because it is read-only", dstSymbolName.c_str());
			return record;
		}

		// if the relocation comes from a symbol used for exception handling, we must never patch it to the original exe.
		// exception handling symbols store information about the type of exceptions caught (__ehfuncinfo$), the handlers themselves
		// (__ehhandler$) and unwind information as well as destructors to call (__unwindfunclet$). if we were to change any of that,
		// an .obj file could never introduce new exceptions or change code inside try/catch blocks.
		if (symbols::IsExceptionRelatedSymbol(srcSymbolName))
		{
			LC_LOG_DEV("Ignoring relocation from %s because it is exception-related", srcSymbolName.c_str());
			return record;
		}
		// similarly, relocations pointing to the SEH table must never be patched to the original exe
		if (symbols::IsExceptionRelatedSymbol(dstSymbolName))
		{
			LC_LOG_DEV("Ignoring relocation to %s because it is exception-related", dstSymbolName.c_str());
			return record;
		}

		// if the relocation points to a virtual-function table, we must never patch it to the original exe.
		// otherwise, new functions in the VTable can never be called, but code with newly created instances expects them to exist, which would lead to a crash.
		if (symbols::IsVTable(dstSymbolName))
		{
			LC_LOG_DEV("Ignoring relocation to %s because it is a vtable", dstSymbolName.c_str());
			return record;
		}

		if (symbols::IsRuntimeCheckRelatedSymbol(dstSymbolName))
		{
			// ignore anything related to runtime checks
			LC_LOG_DEV("Ignoring relocation to %s because it belongs to runtime checks", dstSymbolName.c_str());
			return record;
		}

		if (symbols::IsImageBaseRelatedSymbol(dstSymbolName))
		{
			// ignore linker-generated symbol
			LC_LOG_DEV("Ignoring relocation to %s", dstSymbolName.c_str());
			return record;
		}

		// general note regarding thread-local storage:
		// access to variables in TLS needs two things: _tls_index and the section-relative offset of the variable.
		// in debug builds, each access first sets _tls_index, then accesses the variable via the correct offset.
		// this would allow us to support even newly introduced TLS symbols by setting the _tls_index accordingly.
		// however, in release builds, _tls_index is often just set once, and then 1 or more variables are accessed using
		// their offsets. for newly introduced TLS symbols this would mean that either existing ones use the wrong _tls_index,
		// or new symbols use the wrong (old) _tls_index.
		// therefore, we don't support introducing new TLS symbols at the moment. we *could* make it work by patching each
		// access to a TLS symbol with a jump to our own little stub that first sets the correct _tls_index, and then does the
		// access.
		if (symbols::IsTlsArrayRelatedSymbol(dstSymbolName))
		{
			// ignore compiler-generated symbol for accessing thread-local storage, because
			// that address is fixed relative to a segment register anyway.
			LC_LOG_DEV("Ignoring relocation to %s", dstSymbolName.c_str());
			return record;
		}

		if (string::Contains(dstSymbolName.c_str(), "$__resumable"))
		{
			LC_LOG_DEV("Ignoring relocation to %s", dstSymbolName.c_str());
			return record;
		}
	}

	// BEGIN EPIC MOD
	bool backwards = forceBackwards || symbols::IsUEReversePatchSymbol(dstSymbolName) || symbols::IsUEReversePatchSymbol(srcSymbolName);
	// END EPIC MOD

	// find the relocation's destination symbol in the original .exe, and patch the relocation
	// to point to this symbol.
	// BEGIN EPIC MOD
	const ModuleCache::FindSymbolData originalData = backwards ? moduleCache->FindSymbolByNameBackwards(ModuleCache::SEARCH_ALL_MODULES, dstSymbolName) : moduleCache->FindSymbolByName(newModuleIndex, dstSymbolName);
	// END EPIC MOD
	const symbols::Symbol* originalSymbol = originalData.symbol;
	if (!originalSymbol)
	{
		// probably a new symbol
		return record;
	}

	// get the address of the symbol in the original module.
	// if this symbol has an incremental linking thunk, redirect the relocation to the thunk instead of to the real function.
	// this is needed because for functions that have been incrementally linked, we only patch its thunk and not the actual function.
	uint32_t originalRva = originalSymbol->rva;

	// only functions can have thunks
	if ((relocation->dstOffset == 0u) && (coff::IsFunctionSymbol(relocation->dstSymbolType)))
	{
		const types::vector<uint32_t>& thunkRvas = symbols::FindThunkTableEntriesByRVA(originalData.data->thunkDb, originalRva);
		if (thunkRvas.size() != 0u)
		{
			// it doesn't matter which thunk we choose, as long as this thunk is also patched to the new function
			originalRva = thunkRvas[0];
		}
	}

	// patch the relocation in all processes
	switch (type)
	{
		case coff::Relocation::Type::RELATIVE:

#if LC_64_BIT
		case coff::Relocation::Type::RELATIVE_OFFSET_1:
		case coff::Relocation::Type::RELATIVE_OFFSET_2:
		case coff::Relocation::Type::RELATIVE_OFFSET_3:
		case coff::Relocation::Type::RELATIVE_OFFSET_4:
		case coff::Relocation::Type::RELATIVE_OFFSET_5:
#endif
		{
			record.relocationType = relocation->type;
			record.patchIndex = static_cast<uint16_t>(originalData.data->index);
			record.newModuleRva = srcSymbol->rva + relocation->srcRva;
			record.data.relativeRelocation.originalModuleRva = originalRva + relocation->dstOffset;

			// The 32-bit relative displacement to the target, the relocation itself is 32-bit
			const uint32_t relocationSize = 4u + coff::Relocation::Type::GetByteDistance(type);

			const size_t count = originalData.data->processes.size();
			for (size_t p = 0u; p < count; ++p)
			{
				void* moduleBase = originalData.data->processes[p].moduleBase;
				Process::Handle processHandle = originalData.data->processes[p].processHandle;
				void* newModuleBase = newModuleBases[p];

				// find the address of the relocation.
				// the relocation's RVA is relative to the start of the function.
				void* relocationAddress = pointer::Offset<void*>(newModuleBase, srcSymbol->rva + relocation->srcRva);

				const void* originalAddress = pointer::Offset<const void*>(moduleBase, originalRva + relocation->dstOffset);

#if LC_64_BIT
				const void* byteFollowingRelocation = pointer::Offset<const void*>(relocationAddress, relocationSize);
				const int64_t displacement64 = pointer::Displacement<int64_t>(byteFollowingRelocation, originalAddress);

				// more than 2 GB ahead or more than 2 GB behind?
				const bool tooFarAhead = (displacement64 > 0x7FFFFFFFll);
				const bool tooFarBehind = (displacement64 < -0x7FFFFFFFll);
				if (tooFarAhead || tooFarBehind)
				{
					LC_ERROR_DEV("Unable to reach address with 32-bit relative relocation. Ignoring relocation.");
					continue;
				}

				const uint32_t displacement = static_cast<uint32_t>(displacement64);
#else
				// 32-BIT NOTE: relative addresses are signed 32-bit offsets, but addressing performed by the CPU
				// works modulo 2^32. this means that it doesn't matter whether we go forward 3GB, or back 1GB - 
				// the resulting address will be the same.
				// we therefore carry out all calculations using *unsigned* 32-bit integers, because they have
				// natural overflow/underflow behaviour, and do *not* invoke undefined behaviour like signed integers.
				const void* byteFollowingRelocation = pointer::Offset<const void*>(relocationAddress, relocationSize);
				const uint32_t displacement = pointer::Displacement<uint32_t>(byteFollowingRelocation, originalAddress);
#endif

				Process::WriteProcessMemory(processHandle, relocationAddress, displacement);

				// BEGIN EPIC MOD
				LC_LOG_DEV("Patched relocation from symbol %d:%s to %d:%s at 0x%p (0x%x + 0x%x) (relative offset)", newModuleIndex, srcSymbolName.c_str(), record.patchIndex, dstSymbolName.c_str(), record.patchIndex, newModuleBase, srcSymbol->rva, relocation->srcRva);
				// END EPIC MOD
			}
		}
		break;

		case coff::Relocation::Type::SECTION_RELATIVE:
		{
			// The 32-bit offset of the target from the beginning of its section.
			// the original symbol is relative to the section it belongs to. re-construct the section-relative
			// address to the original section, and patch the relocation to the section-relative address
			// in the new executable.
			const ImmutableString& sectionName = coff::GetTlsSectionName();
			const symbols::ImageSection* imageSection = symbols::FindImageSectionByName(originalData.data->imageSectionDb, sectionName);
			if (imageSection)
			{
				const uint32_t originalSectionRva = imageSection->rva;
				const uint32_t sectionRelativeRva = originalSymbol->rva - originalSectionRva;

				record.relocationType = relocation->type;
				record.patchIndex = static_cast<uint16_t>(originalData.data->index);
				record.newModuleRva = srcSymbol->rva + relocation->srcRva;
				record.data.sectionRelativeRelocation.sectionRelativeRva = sectionRelativeRva;

				const size_t count = originalData.data->processes.size();
				for (size_t p = 0u; p < count; ++p)
				{
					Process::Handle processHandle = originalData.data->processes[p].processHandle;
					void* newModuleBase = newModuleBases[p];

					// find the address of the relocation.
					// the relocation's RVA is relative to the start of the function.
					void* relocationAddress = pointer::Offset<void*>(newModuleBase, srcSymbol->rva + relocation->srcRva);

					Process::WriteProcessMemory(processHandle, relocationAddress, sectionRelativeRva);

					// BEGIN EPIC MOD
					LC_LOG_DEV("Patched relocation from symbol %d:%s to %d:%s at 0x%p (0x%x + 0x%x) (section relative)", newModuleIndex, srcSymbolName.c_str(), record.patchIndex, dstSymbolName.c_str(), newModuleBase, srcSymbol->rva, relocation->srcRva);
					// END EPIC MOD
				}
			}
			else
			{
				LC_ERROR_DEV("Could not patch relocation of type %s (%d) to symbol %s", coff::Relocation::Type::ToString(type), type, dstSymbolName.c_str());
				return record;
			}
		}
		break;

		case coff::Relocation::Type::VA_32:
		{
#if LC_64_BIT
			// an absolute 32-bit virtual address cannot exist in a 64-bit image, otherwise the .exe/.dll could
			// not be loaded into the upper 32-bits of the address space.
			LC_ERROR_DEV("Ignoring relocation of type %s (%d) to symbol %s", coff::Relocation::Type::ToString(type), type, dstSymbolName.c_str());
			return record;
#else
			record.relocationType = relocation->type;
			record.patchIndex = static_cast<uint16_t>(originalData.data->index);
			record.newModuleRva = srcSymbol->rva + relocation->srcRva;
			record.data.va32Relocation.originalModuleRva = originalRva + relocation->dstOffset;

			const size_t count = originalData.data->processes.size();
			for (size_t p = 0u; p < count; ++p)
			{
				void* moduleBase = originalData.data->processes[p].moduleBase;
				Process::Handle processHandle = originalData.data->processes[p].processHandle;
				void* newModuleBase = newModuleBases[p];

				// find the address of the relocation.
				// the relocation's RVA is relative to the start of the function.
				void* relocationAddress = pointer::Offset<void*>(newModuleBase, srcSymbol->rva + relocation->srcRva);

				const void* originalAddress = pointer::Offset<const void*>(moduleBase, originalRva + relocation->dstOffset);

				// The target's 32-bit VA.
				const uint32_t va = static_cast<uint32_t>(reinterpret_cast<uintptr_t>(originalAddress));

				Process::WriteProcessMemory(processHandle, relocationAddress, va);

				LC_LOG_DEV("Patched relocation from symbol %s to %s at 0x%p (0x%x + 0x%x)", srcSymbolName.c_str(), dstSymbolName.c_str(), newModuleBase, srcSymbol->rva, relocation->srcRva);
			}
#endif
		}
		break;

		case coff::Relocation::Type::RVA_32:
		{
			record.relocationType = relocation->type;
			record.patchIndex = static_cast<uint16_t>(originalData.data->index);
			record.newModuleRva = srcSymbol->rva + relocation->srcRva;
			record.data.rva32Relocation.originalModuleRva = originalRva + relocation->dstOffset;

			const size_t count = originalData.data->processes.size();
			for (size_t p = 0u; p < count; ++p)
			{
				void* moduleBase = originalData.data->processes[p].moduleBase;
				Process::Handle processHandle = originalData.data->processes[p].processHandle;
				void* newModuleBase = newModuleBases[p];

				// find the address of the relocation.
				// the relocation's RVA is relative to the start of the function.
				void* relocationAddress = pointer::Offset<void*>(newModuleBase, srcSymbol->rva + relocation->srcRva);

				const void* originalAddress = pointer::Offset<const void*>(moduleBase, originalRva + relocation->dstOffset);

				// the relocation stores the RVA of the symbol relative to the image base of the original executable.
				// we need to patch this to point to the existing symbol, but relative to the image base of the patch executable.
				// note that the displacement is signed.
				const int64_t displacementToNewBase = pointer::Displacement<int64_t>(newModuleBase, originalAddress);
				const int32_t displacement = static_cast<int32_t>(displacementToNewBase);

				Process::WriteProcessMemory(processHandle, relocationAddress, displacement);

				// BEGIN EPIC MOD
				LC_LOG_DEV("Patched relocation from symbol %d:%s to %d:%s at 0x%p (0x%x + 0x%x) (RVA_32)", newModuleIndex, srcSymbolName.c_str(), record.patchIndex, dstSymbolName.c_str(), newModuleBase, srcSymbol->rva, relocation->srcRva);
				// END EPIC MOD
			}
		}
		break;

#if LC_64_BIT
		case coff::Relocation::Type::VA_64:
		{
			record.relocationType = relocation->type;
			record.patchIndex = static_cast<uint16_t>(originalData.data->index);
			record.newModuleRva = srcSymbol->rva + relocation->srcRva;
			record.data.va64Relocation.originalModuleRva = originalRva + relocation->dstOffset;

			const size_t count = originalData.data->processes.size();
			for (size_t p = 0u; p < count; ++p)
			{
				void* moduleBase = originalData.data->processes[p].moduleBase;
				Process::Handle processHandle = originalData.data->processes[p].processHandle;
				void* newModuleBase = newModuleBases[p];

				// find the address of the relocation.
				// the relocation's RVA is relative to the start of the function.
				void* relocationAddress = pointer::Offset<void*>(newModuleBase, srcSymbol->rva + relocation->srcRva);

				const void* originalAddress = pointer::Offset<const void*>(moduleBase, originalRva + relocation->dstOffset);

				// The target's 64-bit VA.
				const uint64_t va = static_cast<uint64_t>(reinterpret_cast<uintptr_t>(originalAddress));

				Process::WriteProcessMemory(processHandle, relocationAddress, va);

				// BEGIN EPIC MOD
				LC_LOG_DEV("Patched relocation from symbol %d:%s to %d:%s at 0x%p (0x%x + 0x%x) (VA_64)", newModuleIndex, srcSymbolName.c_str(), record.patchIndex, dstSymbolName.c_str(), newModuleBase, srcSymbol->rva, relocation->srcRva);
				// END EPIC MOD
			}
		}
		break;
#endif

		case coff::Relocation::Type::UNKNOWN:
		default:
			LC_ERROR_DEV("Unknown relocation type %s (%d)", coff::Relocation::Type::ToString(type), type);
			break;
	}

	return record;
}


void relocations::PatchRelocation
(
	const Record& record,
	Process::Handle processHandle,
	void* processModuleBases[],
	void* newModuleBase
)
{
	void* moduleBase = processModuleBases[record.patchIndex];
	if (!moduleBase)
	{
		return;
	}

	void* relocationAddress = pointer::Offset<void*>(newModuleBase, record.newModuleRva);

	switch (record.relocationType)
	{
		case coff::Relocation::Type::RELATIVE:

#if LC_64_BIT
		case coff::Relocation::Type::RELATIVE_OFFSET_1:
		case coff::Relocation::Type::RELATIVE_OFFSET_2:
		case coff::Relocation::Type::RELATIVE_OFFSET_3:
		case coff::Relocation::Type::RELATIVE_OFFSET_4:
		case coff::Relocation::Type::RELATIVE_OFFSET_5:
#endif
		{
			const uint32_t relocationSize = 4u + coff::Relocation::Type::GetByteDistance(record.relocationType);
			const void* originalAddress = pointer::Offset<const void*>(moduleBase, record.data.relativeRelocation.originalModuleRva);

#if LC_64_BIT
			const void* byteFollowingRelocation = pointer::Offset<const void*>(relocationAddress, relocationSize);
			const int64_t displacement64 = pointer::Displacement<int64_t>(byteFollowingRelocation, originalAddress);
			const uint32_t displacement = static_cast<uint32_t>(displacement64);
#else
			const void* byteFollowingRelocation = pointer::Offset<const void*>(relocationAddress, relocationSize);
			const uint32_t displacement = pointer::Displacement<uint32_t>(byteFollowingRelocation, originalAddress);
#endif

			Process::WriteProcessMemory(processHandle, relocationAddress, displacement);
		}
		break;

		case coff::Relocation::Type::SECTION_RELATIVE:
		{
			Process::WriteProcessMemory(processHandle, relocationAddress, record.data.sectionRelativeRelocation.sectionRelativeRva);
		}
		break;

		case coff::Relocation::Type::VA_32:
		{
#if LC_32_BIT
			const void* originalAddress = pointer::Offset<const void*>(moduleBase, record.data.va32Relocation.originalModuleRva);
			const uint32_t va = static_cast<uint32_t>(reinterpret_cast<uintptr_t>(originalAddress));

			Process::WriteProcessMemory(processHandle, relocationAddress, va);
#endif
		}
		break;

		case coff::Relocation::Type::RVA_32:
		{
			const void* originalAddress = pointer::Offset<const void*>(moduleBase, record.data.rva32Relocation.originalModuleRva);
			const int64_t displacementToNewBase = pointer::Displacement<int64_t>(newModuleBase, originalAddress);
			const int32_t displacement = static_cast<int32_t>(displacementToNewBase);

			Process::WriteProcessMemory(processHandle, relocationAddress, displacement);
		}
		break;

#if LC_64_BIT
		case coff::Relocation::Type::VA_64:
		{
			const void* originalAddress = pointer::Offset<const void*>(moduleBase, record.data.va64Relocation.originalModuleRva);
			const uint64_t va = static_cast<uint64_t>(reinterpret_cast<uintptr_t>(originalAddress));

			Process::WriteProcessMemory(processHandle, relocationAddress, va);
		}
		break;
#endif

		case coff::Relocation::Type::UNKNOWN:
		default:
			break;
	}
}


bool relocations::IsValidRecord(const Record& record)
{
	return (record.relocationType != coff::Relocation::Type::UNKNOWN);
}


#endif