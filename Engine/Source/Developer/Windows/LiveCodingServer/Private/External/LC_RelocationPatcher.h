// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Coff.h"
#include "LC_Symbols.h"
#include "LC_ModuleCache.h"
#include "LC_ProcessTypes.h"


namespace relocations
{
	struct Record
	{
		coff::Relocation::Type::Enum relocationType;
		uint16_t patchIndex;
		uint32_t newModuleRva;

		union Data
		{
			struct RelativeRelocation
			{
				uint32_t originalModuleRva;
			} relativeRelocation;

			struct SectionRelativeRelocation
			{
				uint32_t sectionRelativeRva;
			} sectionRelativeRelocation;

			struct VA32Relocation
			{
				uint32_t originalModuleRva;
			} va32Relocation;

			struct RVA32Relocation
			{
				uint32_t originalModuleRva;
			} rva32Relocation;

			struct VA64Relocation
			{
				uint32_t originalModuleRva;
			} va64Relocation;
		} data;
	};

	bool WouldPatchRelocation(const ImmutableString& dstSymbolName);

	bool WouldPatchRelocation
	(
		const coff::Relocation* relocation,
		const coff::CoffDB* coffDb,
		const ImmutableString& srcSymbolName,
		const ModuleCache::FindSymbolData& originalData
	);

	Record PatchRelocation
	(
		const coff::Relocation* relocation,
		const coff::CoffDB* coffDb,
		const types::StringSet& forceRelocationSymbols,
		const ModuleCache* moduleCache,
		const ImmutableString& srcSymbolName,
		const ImmutableString& dstSymbolName,
		const symbols::Symbol* srcSymbol,
		size_t newModuleIndex,
		void* newModuleBases[]
// BEGIN EPIC MOD
		, bool forceBackwards
// END EPIC MOD
	);

	void PatchRelocation
	(
		const Record& record,
		Process::Handle processHandle,
		void* processModuleBases[],
		void* newModuleBase
	);

	bool IsValidRecord(const Record& record);
};


#endif