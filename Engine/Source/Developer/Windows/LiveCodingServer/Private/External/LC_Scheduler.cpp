// Copyright 2011-2019 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Scheduler.h"
#include "LC_SchedulerWorkerThread.h"
#include "LC_SchedulerQueue.h"
#include "LC_Thread.h"


// BEGIN EPIC MODS
#pragma warning(push)
#pragma warning(disable:6011) // warning C6011: Dereferencing NULL pointer 'info'. 
// END EPIC MODS

namespace
{
	scheduler::TaskQueue* g_taskQueue = nullptr;
	scheduler::WorkerThread** g_workerThreads = nullptr;

	static bool EmptyTask(void)
	{
		return true;
	}

	static unsigned int GetLogicalProcessorCount(void)
	{
		SYSTEM_INFO info = {};
		::GetSystemInfo(&info);		
		
		return info.dwNumberOfProcessors;
	}

	static unsigned int GetPhysicalProcessorCount(void)
	{
		// try getting the number of physical cores
		SYSTEM_LOGICAL_PROCESSOR_INFORMATION* buffer = nullptr;
		DWORD bytesNeeded = 0u;
		BOOL result = ::GetLogicalProcessorInformation(buffer, &bytesNeeded);
		// BEGIN EPIC MOD
		if (result == Windows::FALSE)
		// END EPIC MOD 
		{
			if (::GetLastError() == ERROR_INSUFFICIENT_BUFFER)
			{
				// function is available. allocate a buffer large enough to hold the information
				buffer = static_cast<SYSTEM_LOGICAL_PROCESSOR_INFORMATION*>(::malloc(bytesNeeded));
				result = ::GetLogicalProcessorInformation(buffer, &bytesNeeded);

				// return the number of logical processors in case anything went wrong
				// BEGIN EPIC MOD
				if ((result == Windows::FALSE) || (buffer == nullptr))
				// END EPIC MOD
				{
					::free(buffer);
					return GetLogicalProcessorCount();
				}

				SYSTEM_LOGICAL_PROCESSOR_INFORMATION* info = buffer;

				unsigned int coreCount = 0u;
				DWORD byteOffset = 0u;

				// retrieve relationship while there is still info left in the buffer
				while (byteOffset + sizeof(SYSTEM_LOGICAL_PROCESSOR_INFORMATION) <= bytesNeeded)
				{
					switch (info->Relationship)
					{
						case RelationProcessorCore:
							++coreCount;
							break;

						case RelationNumaNode:
						case RelationCache:
						case RelationProcessorPackage:
						case RelationGroup:
						case RelationAll:
						default:
							break;
					}

					byteOffset += sizeof(SYSTEM_LOGICAL_PROCESSOR_INFORMATION);
					++info;
				}

				::free(buffer);

				// return the number of logical processors in case anything went wrong
				if (coreCount == 0u)
				{
					return GetLogicalProcessorCount();
				}

				return coreCount;
			}
		}

		// if we cannot retrieve the physical processor information, at least return some
		// meaningful number.
		return GetLogicalProcessorCount();
	}
}


void scheduler::Startup(void)
{
	// first create the task queue, and then create a worker thread for each core in the system
	g_taskQueue = new scheduler::TaskQueue;

	const unsigned int coreCount = GetPhysicalProcessorCount();
	g_workerThreads = new scheduler::WorkerThread*[coreCount];
	for (unsigned int i = 0u; i < coreCount; ++i)
	{
		g_workerThreads[i] = new scheduler::WorkerThread(g_taskQueue);
	}
}


void scheduler::Shutdown(void)
{
	// we deliberately do not destroy the worker threads because we don't want them to be joined.
	// we need to exit as fast as possible.
	delete g_taskQueue;
}


scheduler::TaskBase* scheduler::CreateEmptyTask(void)
{
	return new Task<bool>(&EmptyTask);
}


void scheduler::DestroyTask(TaskBase* task)
{
	delete task;
}


void scheduler::RunTask(TaskBase* task)
{
	g_taskQueue->PushTask(task);
}


void scheduler::WaitForTask(TaskBase* task)
{
	while (!task->IsFinished())
	{
		// help with other tasks in the mean time, if possible
		TaskBase* newTask = g_taskQueue->TryPopTask();
		if (newTask)
		{
			newTask->Execute();
		}
		else
		{
			// no task available
			Thread::Current::SleepMilliSeconds(10u);
		}
	}
}

// BEGIN EPIC MODS
#pragma warning(pop)
// END EPIC MODS


#endif