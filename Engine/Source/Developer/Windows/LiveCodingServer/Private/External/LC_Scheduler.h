// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_SchedulerTask.h"


namespace scheduler
{
	void Startup(void);
	void Shutdown(void);

	// BEGIN EPIC MOD
	// creates a new task from any function
	template <typename F>
	Task<std::invoke_result_t<F>>* CreateTask(F&& function)
	{
		return new Task<std::invoke_result_t<F()>>(function);
	}

	// creates a new task from any function as child of a parent task
	template <typename F>
	Task<std::invoke_result_t<F>>* CreateTask(TaskBase* parent, F&& function)
	{
		return new Task<std::invoke_result_t<F>>(parent, function);
	}
	// END EPIC MOD

	// creates an empty task
	TaskBase* CreateEmptyTask(void);

	// destroys a task
	void DestroyTask(TaskBase* task);

	// destroys a container of tasks
	template <typename T>
	void DestroyTasks(const T& container)
	{
		const size_t count = container.size();
		for (size_t i = 0u; i < count; ++i)
		{
			DestroyTask(container[i]);
		}
	}


	void RunTask(TaskBase* task);

	void WaitForTask(TaskBase* task);
}


#endif