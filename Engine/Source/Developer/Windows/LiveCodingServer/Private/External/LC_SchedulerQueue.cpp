// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_SchedulerQueue.h"


scheduler::TaskQueue::TaskQueue(void)
	: m_tasks()
	, m_readIndex(0u)
	, m_writeIndex(0u)
	, m_producerSema(TASK_COUNT, TASK_COUNT)
	, m_consumerSema(0u, TASK_COUNT)
	, m_cs()
{
}


void scheduler::TaskQueue::PushTask(TaskBase* task)
{
	// wait until there is room for the task
	m_producerSema.Wait();

	{
		CriticalSection::ScopedLock lock(&m_cs);

		m_tasks[m_writeIndex & ACCESS_MASK] = task;
		++m_writeIndex;
	}

	// tell the consumers that there is a new task available
	m_consumerSema.Signal();
}


scheduler::TaskBase* scheduler::TaskQueue::PopTask(void)
{
	// wait for a task to become available
	m_consumerSema.Wait();

	TaskBase* task = nullptr;
	{
		CriticalSection::ScopedLock lock(&m_cs);

		task = m_tasks[m_readIndex & ACCESS_MASK];
		++m_readIndex;
	}

	// tell the producers that there is room for a new task
	m_producerSema.Signal();

	return task;
}


scheduler::TaskBase* scheduler::TaskQueue::TryPopTask(void)
{
	// check if a task is available
	const bool isAvailable = m_consumerSema.TryWait();
	if (!isAvailable)
	{
		// no task yet, bail out
		return nullptr;
	}

	TaskBase* task = nullptr;
	{
		CriticalSection::ScopedLock lock(&m_cs);

		task = m_tasks[m_readIndex & ACCESS_MASK];
		++m_readIndex;
	}

	// tell the producers that there is room for a new task
	m_producerSema.Signal();

	return task;
}


#endif