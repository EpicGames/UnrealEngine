// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Semaphore.h"
#include "LC_CriticalSection.h"
// BEGIN EPIC MOD
#include <stdint.h>
// END EPIC MOD

namespace scheduler
{
	class TaskBase;


	// simple multi-producer, multi-consumer queue
	class TaskQueue
	{
		// BEGIN EPIC MOD - Increasing task count due to hangs when enabling for all editor modules
		static const unsigned int TASK_COUNT = 4096u;// 1024u;
		// END EPIC MOD
		static const unsigned int ACCESS_MASK = TASK_COUNT - 1u;

	public:
		TaskQueue(void);

		// blocks when there is no more room for a task
		void PushTask(TaskBase* task);

		// blocks when there is no task in the queue
		TaskBase* PopTask(void);

		// does not block
		TaskBase* TryPopTask(void);

	private:
		TaskBase* m_tasks[TASK_COUNT];
		uint64_t m_readIndex;
		uint64_t m_writeIndex;
		Semaphore m_producerSema;
		Semaphore m_consumerSema;
		CriticalSection m_cs;
	};
}


#endif