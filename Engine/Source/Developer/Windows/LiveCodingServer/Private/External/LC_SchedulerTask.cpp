// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_SchedulerTask.h"


scheduler::TaskBase::TaskBase(TaskBase* parent)
	: m_parent(parent)
	, m_openTasks(1u)
{
	if (m_parent)
	{
		m_parent->OnChildAttach();
	}
}


scheduler::TaskBase::~TaskBase(void)
{
}


void scheduler::TaskBase::Execute(void)
{
	DoExecute();

	CriticalSection::ScopedLock lock(&m_taskCS);
	--m_openTasks;

	if (IsFinished())
	{
		if (m_parent)
		{
			m_parent->OnChildDetach();
		}
	}
}


bool scheduler::TaskBase::IsFinished(void) const
{
	CriticalSection::ScopedLock lock(&m_taskCS);
	return (m_openTasks == 0u);
}


void scheduler::TaskBase::OnChildAttach(void)
{
	CriticalSection::ScopedLock lock(&m_taskCS);
	++m_openTasks;
}


void scheduler::TaskBase::OnChildDetach(void)
{
	CriticalSection::ScopedLock lock(&m_taskCS);
	--m_openTasks;
}


#endif