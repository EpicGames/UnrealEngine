// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_CriticalSection.h"
// BEGIN EPIC MOD
#include "LC_Platform.h"
#include <functional>
// END EPIC MOD


namespace scheduler
{
	class TaskBase
	{
	public:
		explicit TaskBase(TaskBase* parent);
		virtual ~TaskBase(void);

		void Execute(void);
		bool IsFinished(void) const;

	private:
		virtual void DoExecute(void) = 0;

		void OnChildAttach(void);
		void OnChildDetach(void);

		TaskBase* const m_parent;
		unsigned int m_openTasks;
		mutable CriticalSection m_taskCS;

		LC_DISABLE_COPY(TaskBase);
		LC_DISABLE_ASSIGNMENT(TaskBase);
		LC_DISABLE_MOVE(TaskBase);
		LC_DISABLE_MOVE_ASSIGNMENT(TaskBase);
	};


	template <typename R>
	class Task : public TaskBase
	{
	public:
		template <typename F>
		Task(TaskBase* parent, const F& function)
			: TaskBase(parent)
			, m_function(function)
			, m_result()
		{
		}

		template <typename F>
		Task(const F& function)
			: TaskBase(nullptr)
			, m_function(function)
			, m_result()
		{
		}

		virtual ~Task(void) {}

		R GetResult(void) const
		{
			return m_result;
		}

	private:
		virtual void DoExecute(void) override
		{
			m_result = m_function();
		}

		std::function<R (void)> m_function;
		R m_result;

		LC_DISABLE_COPY(Task);
		LC_DISABLE_ASSIGNMENT(Task);
		LC_DISABLE_MOVE(Task);
		LC_DISABLE_MOVE_ASSIGNMENT(Task);
	};
}


#endif