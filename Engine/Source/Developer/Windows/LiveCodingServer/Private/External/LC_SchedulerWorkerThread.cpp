// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_SchedulerWorkerThread.h"
#include "LC_SchedulerQueue.h"
#include "LC_SchedulerTask.h"
#include "LC_Thread.h"


scheduler::WorkerThread::WorkerThread(TaskQueue* queue)
	: m_thread()
{
	// BEGIN EPIC MOD
	m_thread = Thread::CreateFromMemberFunction("Live coding worker", 128u * 1024u, this, &scheduler::WorkerThread::ThreadFunction, queue);
	// END EPIC MOD
}


scheduler::WorkerThread::~WorkerThread(void)
{
	Thread::Join(m_thread);
}


Thread::ReturnValue scheduler::WorkerThread::ThreadFunction(TaskQueue* queue)
{
	for (;;)
	{
		// get a task from the queue and execute it
		TaskBase* task = queue->PopTask();
		if (task == nullptr)
		{
			break;
		}

		task->Execute();
	}

	return Thread::ReturnValue(0u);
}


#endif