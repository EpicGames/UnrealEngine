// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ThreadTypes.h"


namespace scheduler
{
	class TaskQueue;


	class WorkerThread
	{
	public:
		explicit WorkerThread(TaskQueue* queue);
		~WorkerThread(void);

	private:
		Thread::ReturnValue ThreadFunction(TaskQueue* queue);

		Thread::Handle m_thread;
	};
}


#endif