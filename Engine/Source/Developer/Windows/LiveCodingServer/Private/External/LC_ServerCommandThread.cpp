// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_ServerCommandThread.h"
#include "LC_Commands.h"
// BEGIN EPIC MOD
//#include "LC_MainFrame.h"
// END EPIC MOD
#include "LC_Telemetry.h"
#include "LC_Symbols.h"
#include "LC_Filesystem.h"
#include "LC_Process.h"
#include "LC_Compiler.h"
#include "LC_StringUtil.h"
#include "LC_CommandMap.h"
#include "LC_FileAttributeCache.h"
// BEGIN EPIC MOD
//#include "LC_App.h"
// END EPIC MOD
#include "LC_Shortcut.h"
#include "LC_Key.h"
#include "LC_ChangeNotification.h"
#include "LC_DirectoryCache.h"
#include "LC_VirtualDrive.h"
#include "LC_LiveModule.h"
#include "LC_LiveProcess.h"
#include "LC_CodeCave.h"
// BEGIN EPIC MOD
//#include "LC_ExceptionHandlerDialog.h"
//#include "LC_Resource.h"
// END EPIC MOD
#include "LC_PrimitiveNames.h"
#include "LC_MemoryStream.h"
#include "LC_NamedSharedMemory.h"
#include "LC_VisualStudioAutomation.h"
#include "LC_Thread.h"
#include <mmsystem.h>

// BEGIN EPIC MOD
#include "LC_AppSettings.h"
#include "LC_Allocators.h"
#include "LC_DuplexPipeClient.h"
#include "LiveCodingServer.h"
#include "Containers/UnrealString.h"
// END EPIC MOD

// unreachable code
#pragma warning (disable : 4702)

// BEGIN EPIC MODS
#pragma warning(push)
#pragma warning(disable:6031) // warning C6031: Return value ignored: 'CoInitialize'.
// END EPIC MODS

namespace
{
	static telemetry::Accumulator g_loadedModuleSize("Module size");



	static void AddVirtualDrive(void)
	{
		const std::wstring virtualDriveLetter = appSettings::g_virtualDriveLetter->GetValue();
		const std::wstring virtualDrivePath = appSettings::g_virtualDrivePath->GetValue();
		if ((virtualDriveLetter.size() != 0) && (virtualDrivePath.size() != 0))
		{
			virtualDrive::Add(virtualDriveLetter.c_str(), virtualDrivePath.c_str());
		}
	}

	static void RemoveVirtualDrive(void)
	{
		const std::wstring virtualDriveLetter = appSettings::g_virtualDriveLetter->GetValue();
		const std::wstring virtualDrivePath = appSettings::g_virtualDrivePath->GetValue();
		if ((virtualDriveLetter.size() != 0) && (virtualDrivePath.size() != 0))
		{
			virtualDrive::Remove(virtualDriveLetter.c_str(), virtualDrivePath.c_str());
		}
	}

	static executable::Header GetImageHeader(const wchar_t* path)
	{
		executable::Image* image = executable::OpenImage(path, Filesystem::OpenMode::READ);
		if (!image)
		{
			return executable::Header {};
		}

		const executable::Header& imageHeader = executable::GetHeader(image);
		executable::CloseImage(image);

		return imageHeader;
	}
}


ServerCommandThread::ServerCommandThread(MainFrame* mainFrame, const wchar_t* const processGroupName, RunMode::Enum runMode)
	: m_processGroupName(processGroupName)
	, m_runMode(runMode)
	, m_mainFrame(mainFrame)
	, m_serverThread()
	, m_compileThread()
	, m_liveModules()
	, m_liveProcesses()
	, m_imageHeaderToLiveModule()
	, m_actionCS()
	, m_exceptionCS()
	, m_inExceptionHandlerEvent(nullptr, Event::Type::MANUAL_RESET)
	, m_handleCommandsEvent(nullptr, Event::Type::MANUAL_RESET)
	, m_directoryCache(new DirectoryCache(2048u))
	, m_connectionCS()
	, m_commandThreads()
	, m_manualRecompileTriggered(false)
	, m_liveModuleToModifiedOrNewObjFiles()
// BEGIN EPIC MOD
	, m_liveModuleToAdditionalLibraries()
// END EPIC MOD
	, m_restartCS()
	, m_restartJob(nullptr)
	, m_restartedProcessCount(0u)
// BEGIN EPIC MOD
#if WITH_VISUALSTUDIO_DTE
// END EPIC MOD
	, m_restartedProcessIdToDebugger()
// BEGIN EPIC MOD
#endif
// END EPIC MOD
{
// BEGIN EPIC MOD
#if WITH_VISUALSTUDIO_DTE
// END EPIC MOD
	visualStudio::Startup();
// BEGIN EPIC MOD
#endif
// END EPIC MOD

	// BEGIN EPIC MOD
	m_serverThread = Thread::CreateFromMemberFunction("Live coding server", 64u * 1024u, this, &ServerCommandThread::ServerThread);
	m_compileThread = Thread::CreateFromMemberFunction("Live coding compilation", 64u * 1024u, this, &ServerCommandThread::CompileThread);
	// END EPIC MOD

	m_liveModules.reserve(256u);
	m_liveProcesses.reserve(8u);
	m_imageHeaderToLiveModule.reserve(256u);

	m_commandThreads.reserve(8u);
}


ServerCommandThread::~ServerCommandThread(void)
{
	// note that we deliberately do almost *nothing* here.
	// this is only called when Live++ is being torn down anyway, so we leave cleanup to the OS.
	// otherwise we could run into races when trying to terminate the thread that might currently be doing
	// some intensive work.
	delete m_directoryCache;

// BEGIN EPIC MOD
#if WITH_VISUALSTUDIO_DTE
// END EPIC MOD
	visualStudio::Shutdown();
// BEGIN EPIC MOD
#endif
// END EPIC MOD
}


void ServerCommandThread::RestartTargets(void)
{
	// protect against concurrent compilation
	m_restartCS.Enter();

	// EPIC REMOVED: g_theApp.GetMainFrame()->SetBusy(true);
	// EPIC REMOVED: g_theApp.GetMainFrame()->ChangeStatusBarText(L"Restarting target applications...");

	LC_LOG_USER("---------- Restarting target applications ----------");

	// prevent current Live++ instance from shutting down by associating it with a new job object to keep it alive
	if (!m_restartJob)
	{
		m_restartJob = ::CreateJobObjectW(NULL, primitiveNames::JobGroup(m_processGroupName).c_str());
		::AssignProcessToJobObject(m_restartJob, ::GetCurrentProcess());
	}

	// protect against m_liveProcesses being accessed when processes restart and register themselves with this Live++ instance
	CriticalSection::ScopedLock lock(&m_actionCS);

	// remove processes that were successfully restarted last time
	for (auto processIt = m_liveProcesses.begin(); processIt != m_liveProcesses.end(); /* nothing */)
	{
		LiveProcess* liveProcess = *processIt;
		if (liveProcess->WasSuccessfulRestart())
		{
			Process::Handle processHandle = liveProcess->GetProcessHandle();
			Process::Close(processHandle);

			// tell live modules to remove this process
			const size_t moduleCount = m_liveModules.size();
			for (size_t j = 0u; j < moduleCount; ++j)
			{
				LiveModule* liveModule = m_liveModules[j];
				liveModule->UnregisterProcess(liveProcess);
			}

			// BEGIN EPIC MOD
			if (liveProcess->IsReinstancingFlowEnabled())
			{
				--m_reinstancingProcessCount;
			}

			if (liveProcess->IsDisableCompileFinishNotification())
			{
				--m_disableCompileFinishNotificationProcessCount;
			}
			// END EPIC MOD

			delete liveProcess;

			processIt = m_liveProcesses.erase(processIt);
		}
		else
		{
			++processIt;
		}
	}

	// try preparing all processes for a restart
	const size_t count = m_liveProcesses.size();
	for (size_t i = 0u; i < count; ++i)
	{
		LiveProcess* liveProcess = m_liveProcesses[i];
		const bool success = liveProcess->PrepareForRestart();
		if (success)
		{
			++m_restartedProcessCount;

// BEGIN EPIC MOD
#if WITH_VISUALSTUDIO_DTE
// END EPIC MOD
			// check if a VS debugger is currently attached to the process about to restart
			const Process::Id processId = liveProcess->GetProcessId();
			EnvDTE::DebuggerPtr debugger = visualStudio::FindDebuggerAttachedToProcess(processId);
			if (debugger)
			{
				m_restartedProcessIdToDebugger.emplace(+processId, debugger);
			}
// BEGIN EPIC MOD
#endif
// END EPIC MOD
		}
	}

	// exit all successfully prepared processes
	for (size_t i = 0u; i < count; ++i)
	{
		LiveProcess* liveProcess = m_liveProcesses[i];
		liveProcess->WaitForExitBeforeRestart();
	}

	// restart all successfully prepared processes
	for (size_t i = 0u; i < count; ++i)
	{
		LiveProcess* liveProcess = m_liveProcesses[i];
		// BEGIN EPIC MOD
		liveProcess->Restart(m_restartJob);
		// END EPIC MOD
	}

	// BEGIN EPIC MOD - Prevent orphaned console instances if processes fail to restart. Job object will be duplicated into child process.
	if (m_restartJob != nullptr)
	{
		CloseHandle(m_restartJob);
		m_restartJob = nullptr;
	}
	// END EPIC MOD
}


void ServerCommandThread::CompileChanges(void)
{
	// protect against accepting this command while compilation is already in progress
	CriticalSection::ScopedLock lock(&m_actionCS);
	m_manualRecompileTriggered = true;
}


std::wstring ServerCommandThread::GetProcessImagePath(void) const
{
	// there must be at least one registered process.
	// in case the EXE was erroneously started directly, no process will be registered.
	// handle this case gracefully.
	if (m_liveProcesses.size() == 0u)
	{
		return L"Unknown";
	}

	return Process::GetImagePath(m_liveProcesses[0]->GetProcessHandle()).GetString();
}


scheduler::Task<LiveModule*>* ServerCommandThread::LoadModule(Process::Id processId, void* moduleBase, const wchar_t* givenModulePath, scheduler::TaskBase* taskRoot)
{
	// note that the path we get from the client might not be normalized, depending on how the executable was launched.
	// it is crucial to normalize the path again, otherwise we could load already loaded modules into the same
	// Live++ instance, which would wreak havoc
	const std::wstring& modulePath = Filesystem::NormalizePath(givenModulePath).GetString();
	const executable::Header imageHeader = GetImageHeader(modulePath.c_str());
	if (!executable::IsValidHeader(imageHeader))
	{
		return nullptr;
	}

	LiveProcess* liveProcess = FindProcessById(processId);
	LC_ASSERT(liveProcess, "Invalid process ID.");

	if (liveProcess->TriedToLoadImage(imageHeader))
	{
		// tried loading this module into this process already
		return nullptr;
	}

	// find any other process ID that tried to load this module already
	{
		const size_t count = m_liveProcesses.size();
		for (size_t i = 0u; i < count; ++i)
		{
			LiveProcess* otherLiveProcess = m_liveProcesses[i];
			if (otherLiveProcess->TriedToLoadImage(imageHeader))
			{
				// some *other* process loaded this module already
				LC_LOG_USER("Registering module %S (PID: %d)", modulePath.c_str(), +processId);

				LiveModule* liveModule = m_imageHeaderToLiveModule[imageHeader];
				if (liveModule)
				{
					liveModule->RegisterProcess(liveProcess, moduleBase, modulePath);
					liveModule->DisableControlFlowGuard(liveProcess, moduleBase);

					const bool installedPatchesSuccessfully = liveModule->InstallCompiledPatches(liveProcess, moduleBase);
					if (!installedPatchesSuccessfully)
					{
						LC_ERROR_USER("Compiled patches could not be installed (PID: %d)", +processId);
						liveModule->UnregisterProcess(liveProcess);
					}

					liveProcess->AddLoadedImage(imageHeader);
				}

				return nullptr;
			}
		}
	}

	symbols::Provider* moduleProvider = symbols::OpenEXE(modulePath.c_str(), symbols::OpenOptions::ACCUMULATE_SIZE);
	if (!moduleProvider)
	{
		return nullptr;
	}

	liveProcess->AddLoadedImage(imageHeader);

	// accumulate module info
	{
		const Filesystem::PathAttributes attributes = Filesystem::GetAttributes(modulePath.c_str());
		const uint64_t size = Filesystem::GetSize(attributes);

		g_loadedModuleSize.Accumulate(size);
		g_loadedModuleSize.Print();
		g_loadedModuleSize.ResetCurrent();

		LC_LOG_USER("Loading module %S (%.3f MB)", modulePath.c_str(), size / 1048576.0f);
	}

	// create a task to load the module of this batch concurrently
	LiveModule* liveModule = new LiveModule(modulePath.c_str(), imageHeader, m_runMode);
	m_imageHeaderToLiveModule.emplace(imageHeader, liveModule);

	auto task = scheduler::CreateTask(taskRoot, [liveModule, liveProcess, modulePath, moduleBase, moduleProvider]()
	{
		telemetry::Scope scope("Loading module");

		symbols::DiaCompilandDB* moduleDiaCompilandDb = symbols::GatherDiaCompilands(moduleProvider);

		liveModule->Load(moduleProvider, moduleDiaCompilandDb);
		liveModule->RegisterProcess(liveProcess, moduleBase, modulePath);
		liveModule->DisableControlFlowGuard(liveProcess, moduleBase);

		symbols::DestroyDiaCompilandDB(moduleDiaCompilandDb);
		symbols::Close(moduleProvider);

		return liveModule;
	});

	scheduler::RunTask(task);

	return task;
}


bool ServerCommandThread::UnloadModule(Process::Id processId, const wchar_t* givenModulePath)
{
	// note that the path we get from the client might not be normalized, depending on how the executable was launched.
	// it is crucial to normalize the path again, otherwise we could load already loaded modules into the same
	// Live++ instance, which would wreak havoc
	const std::wstring& modulePath = Filesystem::NormalizePath(givenModulePath).GetString();
	const executable::Header imageHeader = GetImageHeader(modulePath.c_str());
	if (!executable::IsValidHeader(imageHeader))
	{
		return false;
	}

	LiveProcess* liveProcess = FindProcessById(processId);
	LC_ASSERT(liveProcess, "Invalid process ID.");

	if (!liveProcess->TriedToLoadImage(imageHeader))
	{
		// this module was never loaded
		return false;
	}

	LC_LOG_USER("Unloading module %S", modulePath.c_str());

	liveProcess->RemoveLoadedImage(imageHeader);
	m_imageHeaderToLiveModule.erase(imageHeader);

	for (auto it = m_liveModules.begin(); it != m_liveModules.end(); /* nothing */)
	{
		LiveModule* liveModule = *it;
		if (std::equal_to<executable::Header>()(liveModule->GetImageHeader(), imageHeader))
		{
			liveModule->Unload();
			delete liveModule;

			it = m_liveModules.erase(it);

			return true;
		}
		else
		{
			++it;
		}
	}

	return false;
}


void ServerCommandThread::PrewarmCompilerEnvironmentCache(void)
{
	// EPIC REMOVED: g_theApp.GetMainFrame()->ChangeStatusBarText(L"Prewarming compiler/linker environment cache...");

	telemetry::Scope scope("Prewarming compiler/linker environment cache");

	// fetch unique compiler and linker paths from all modules
	types::StringSet uniquePaths;

	// compiler and linker paths can be overridden, so we need to make sure that we pre-warm the
	// cache for all compilers and linkers involved, depending on the UI settings.
	// there are 3 options:
	// - the path is not overridden: fetch only the paths from the compilands
	// - the paths are overridden, but only used as fallback: fetch the paths from the compilands
	//   as well as the overridden ones. we might need both, depending on which file we compile
	// - the paths are overridden, and always used: fetch only the overridden paths, we're only using those

	// fetch all compiler paths involved.
	// the compiler is only used in default mode, NOT when using an external build system.
	const bool useCompilerEnvironment = appSettings::g_useCompilerEnvironment->GetValue();
	if (useCompilerEnvironment && (m_runMode == RunMode::DEFAULT))
	{
		const std::wstring overriddenPath = appSettings::GetCompilerPath();
		const bool useOverriddenPathAsFallback = appSettings::g_useCompilerOverrideAsFallback->GetValue();

		// always prewarm for overridden compiler path if it is available
		const bool prewarmOverridenPath = (overriddenPath.length() != 0u);

		const bool prewarmCompilandCompilerPath = prewarmOverridenPath
			? useOverriddenPathAsFallback			// overridden path is set. only prewarm compiland compiler paths if the override is only used as fallback
			: true;									// no override is set, always prewarm

		if (prewarmCompilandCompilerPath)
		{
			const size_t count = m_liveModules.size();
			for (size_t i = 0u; i < count; ++i)
			{
				const LiveModule* liveModule = m_liveModules[i];

				const symbols::CompilandDB* compilandDB = liveModule->GetCompilandDatabase();
				for (auto it = compilandDB->compilands.begin(); it != compilandDB->compilands.end(); ++it)
				{
					const symbols::Compiland* compiland = it->second;
					LC_ASSERT(compiland->compilerPath.c_str(), "Invalid compiler path.");

					if (compiland->compilerPath.GetLength() != 0u)
					{
						uniquePaths.insert(compiland->compilerPath);
					}
					else
					{
						LC_WARNING_USER("Not prewarming environment cache for empty compiler in module %S", liveModule->GetModuleName().c_str());
					}
				}
			}
		}

		if (prewarmOverridenPath)
		{
			uniquePaths.insert(string::ToUtf8String(overriddenPath));
		}
	}

	// fetch all linker paths involved
	const bool useLinkerEnvironment = appSettings::g_useLinkerEnvironment->GetValue();
	if (useLinkerEnvironment)
	{
		const std::wstring overriddenPath = appSettings::GetLinkerPath();
		const bool useOverriddenPathAsFallback = appSettings::g_useLinkerOverrideAsFallback->GetValue();

		// always prewarm for overridden linker path if it is available
		const bool prewarmOverridenPath = (overriddenPath.length() != 0u);

		const bool prewarmLinkerPath = prewarmOverridenPath
			? useOverriddenPathAsFallback			// overridden path is set. only prewarm linker paths if the override is only used as fallback
			: true;									// no override is set, always prewarm

		if (prewarmLinkerPath)
		{
			const size_t count = m_liveModules.size();
			for (size_t i = 0u; i < count; ++i)
			{
				const LiveModule* liveModule = m_liveModules[i];

				const symbols::LinkerDB* linkerDB = liveModule->GetLinkerDatabase();
				if (linkerDB->linkerPath.GetLength() != 0u)
				{
					uniquePaths.insert(linkerDB->linkerPath);
				}
				else
				{
					LC_WARNING_USER("Not prewarming environment cache for empty linker in module %S", liveModule->GetModuleName().c_str());
				}
			}
		}

		if (prewarmOverridenPath)
		{
			uniquePaths.insert(string::ToUtf8String(overriddenPath));
		}
	}

	// grab environment blocks for all unique compilers/linkers concurrently
	auto taskRoot = scheduler::CreateEmptyTask();

	types::vector<scheduler::TaskBase*> tasks;
	tasks.reserve(uniquePaths.size());

	for (auto it = uniquePaths.begin(); it != uniquePaths.end(); ++it)
	{
		auto task = scheduler::CreateTask(taskRoot, [it]()
		{
			const ImmutableString& path = *it;
			compiler::UpdateEnvironmentCache(string::ToWideString(path).c_str());

			return true;
		});
		scheduler::RunTask(task);

		tasks.emplace_back(task);
	}

	// wait for all tasks to end
	scheduler::RunTask(taskRoot);
	scheduler::WaitForTask(taskRoot);

	// destroy all tasks
	scheduler::DestroyTasks(tasks);
	scheduler::DestroyTask(taskRoot);

	if (uniquePaths.size() != 0u)
	{
		LC_SUCCESS_USER("Prewarmed compiler/linker environment cache (%.3fs, %zu executables)", scope.ReadSeconds(), uniquePaths.size());
	}
}


Thread::ReturnValue ServerCommandThread::ServerThread(void)
{
	// keep named shared memory alive so that restarted processes don't try spawning new Live++ instances
	Process::NamedSharedMemory* sharedMemory = Process::CreateNamedSharedMemory(primitiveNames::StartupNamedSharedMemory(m_processGroupName).c_str(), 4096u);
	Process::WriteNamedSharedMemory(sharedMemory, ::GetCurrentProcessId());

	// inter process event for telling client that server is ready
	Event serverReadyEvent(primitiveNames::ServerReadyEvent(m_processGroupName).c_str(), Event::Type::AUTO_RESET);

	// run separate pipe servers for all incoming connections
	for (;;)
	{
		CommandThreadContext* context = new CommandThreadContext;
		context->pipe.Create(primitiveNames::Pipe(m_processGroupName).c_str());
		context->exceptionPipe.Create(primitiveNames::ExceptionPipe(m_processGroupName).c_str());

		context->readyEvent = new Event(nullptr, Event::Type::AUTO_RESET);

		// tell other processes that a new server is ready
		serverReadyEvent.Signal();

		// wait until any client connects, blocking
		context->pipe.WaitForClient();
		context->exceptionPipe.WaitForClient();

		// a new client has connected, open a new thread for communication
		// BEGIN EPIC MOD
		context->commandThread = Thread::CreateFromMemberFunction("Live coding client command communication", 64u * 1024u, this, &ServerCommandThread::CommandThread, &context->pipe, context->readyEvent);
		context->exceptionCommandThread = Thread::CreateFromMemberFunction("Live coding client exception command communication", 64u * 1024u, this, &ServerCommandThread::ExceptionCommandThread, &context->exceptionPipe);
		// END EPIC MOD

		// register this connection
		{
			CriticalSection::ScopedLock lock(&m_connectionCS);
			m_commandThreads.push_back(context);
		}
	}

	Process::DestroyNamedSharedMemory(sharedMemory);

	return Thread::ReturnValue(0u);
}

// BEGIN EPIC MOD
bool ServerCommandThread::HasReinstancingProcess()
{
	return m_reinstancingProcessCount.load() != 0;
}
// END EPIC MOD

// BEGIN EPIC MOD
bool ServerCommandThread::ShowCompileFinishNotification()
{
	return m_disableCompileFinishNotificationProcessCount.load() == 0;
}
// END EPIC MOD

// BEGIN EPIC MOD - Focus application windows on patch complete
BOOL CALLBACK FocusApplicationWindows(HWND WindowHandle, LPARAM Lparam)
{
	DWORD WindowProcessId;
    GetWindowThreadProcessId(WindowHandle, &WindowProcessId);

	const types::vector<LiveProcess*>& Processes = *(const types::vector<LiveProcess*>*)Lparam;
	for (LiveProcess* Process : Processes)
	{
		if (+Process->GetProcessId() == WindowProcessId && IsWindowVisible(WindowHandle))
		{
			SetForegroundWindow(WindowHandle);
		}
	}
    return Windows::TRUE;
}
// END EPIC MOD

// BEGIN EPIC MOD - Support for lazy-loading modules
bool ServerCommandThread::actions::FinishedLazyLoadingModules::Execute(const CommandType* command, const DuplexPipe* pipe, void*, const void*, size_t)
{ 
	pipe->SendAck(); 
	return false; 
}

struct ClientProxyThread
{
	struct ProxyEnableModulesFinishedAction
	{
		typedef commands::EnableModulesFinished CommandType;

		static bool Execute(CommandType* command, const DuplexPipe* pipe, void*, const void*, size_t)
		{
			pipe->SendAck();
			return false;
		}
	};

	LiveProcess* m_process;
	DuplexPipeClient* m_pipe;
	std::vector<std::wstring> m_enableModules;
	Thread::Handle m_threadHandle;

	ClientProxyThread(LiveProcess* process, DuplexPipeClient* pipe, const std::vector<std::wstring> enableModules)
		: m_process(process)
		, m_pipe(pipe)
		, m_enableModules(enableModules)
	{
		m_threadHandle = Thread::Create(64u * 1024u, &StaticEntryPoint, this);
		Thread::Current::SetName("Live coding client proxy");
	}

	~ClientProxyThread()
	{
		Thread::Join(m_threadHandle);
		Thread::Close(m_threadHandle);
	}

	static unsigned int __stdcall StaticEntryPoint(void* context)
	{
		static_cast<ClientProxyThread*>(context)->EntryPoint();
		return 0;
	}

	void EntryPoint()
	{
		std::vector<commands::ModuleData> modules;
		modules.resize(m_enableModules.size());

		for (size_t Idx = 0; Idx < m_enableModules.size(); Idx++)
		{
			commands::ModuleData& module = modules[Idx];
			module.base = m_process->GetLazyLoadedModuleBase(m_enableModules[Idx].c_str());
			wcscpy_s(module.path, m_enableModules[Idx].c_str());
		}

		commands::EnableModules enableModulesCommand;
		enableModulesCommand.processId = m_process->GetProcessId();
		enableModulesCommand.moduleCount = m_enableModules.size();
		enableModulesCommand.token = nullptr;
		m_pipe->SendCommandAndWaitForAck(enableModulesCommand, modules.data(), modules.size() * sizeof(commands::ModuleData));

		CommandMap commandMap;
		commandMap.RegisterAction<ProxyEnableModulesFinishedAction>();
		commandMap.HandleCommands(m_pipe, m_process);

		m_pipe->SendCommandAndWaitForAck(commands::FinishedLazyLoadingModules(), nullptr, 0);
	}
};

bool ServerCommandThread::EnableRequiredModules(const TArray<FString>& RequiredModules)
{
	bool bEnabledModule = false;
	for (LiveProcess* liveProcess : m_liveProcesses)
	{
		types::vector<std::wstring> LoadModuleFileNames;
		for (const FString& RequiredModule : RequiredModules)
		{
			std::wstring ModuleFileName = Filesystem::NormalizePath(*RequiredModule).GetString();
			if (liveProcess->IsPendingLazyLoadedModule(ModuleFileName))
			{
				LoadModuleFileNames.push_back(ModuleFileName);
			}
		}
		if (LoadModuleFileNames.size() > 0)
		{
			const std::wstring PipeName = primitiveNames::Pipe(m_processGroupName + L"_ClientProxy");

			DuplexPipeServer ServerPipe;
			ServerPipe.Create(PipeName.c_str());

			DuplexPipeClient ClientPipe;
			ClientPipe.Connect(PipeName.c_str());

			ClientProxyThread ClientThread(liveProcess, &ClientPipe, LoadModuleFileNames);

			CommandMap commandMap;
			commandMap.RegisterAction<actions::EnableModules>();
			commandMap.RegisterAction<actions::FinishedLazyLoadingModules>();
			commandMap.HandleCommands(&ServerPipe, this);

			for (const std::wstring& loadModuleFileName : LoadModuleFileNames)
			{
				liveProcess->SetLazyLoadedModuleAsLoaded(loadModuleFileName);
			}

			bEnabledModule = true;
		}
	}
	return bEnabledModule;
}
// END EPIC MOD

// BEGIN EPIC MOD
void ServerCommandThread::CompileChanges(bool didAllProcessesMakeProgress, commands::PostCompileResult& postCompileResult)
// END EPIC MOD
{
	// recompile files, if any
	// EPIC REMOVED: g_theApp.GetMainFrame()->SetBusy(true);
	// EPIC REMOVED: g_theApp.GetMainFrame()->ChangeStatusBarText(L"Creating patch...");

	telemetry::Scope scope("Creating patch");

	// EPIC REMOVED: g_theApp.GetMainFrame()->OnCompilationStart();

	LC_LOG_USER("---------- Creating patch ----------");

	// BEGIN EPIC MOD - Hook for the compiler
	GLiveCodingServer->GetCompileStartedDelegate().ExecuteIfBound();

	const ILiveCodingServer::FCompileDelegate& CompileDelegate = GLiveCodingServer->GetCompileDelegate();
	if (CompileDelegate.IsBound())
	{
		// Get the list of arguments for building each target, and use the delegate to pass them to UBT
		TArray<FString> Targets;
		for (LiveProcess* liveProcess : m_liveProcesses)
		{
			Targets.AddUnique(liveProcess->GetBuildArguments());
		}

		GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Compiling changes for live coding...");

		// Keep retrying the compile until we've added all the required modules
		FModuleToModuleFiles ModuleToModuleFiles;
		ELiveCodingCompileReason CompileReason = ELiveCodingCompileReason::Initial;
		for (;;)
		{
			// Build a list of modules which are enabled for live coding
			TArray<FString> ValidModules;
			for (LiveModule* liveModule : m_liveModules)
			{
				ValidModules.Add(liveModule->GetModuleName().c_str());
			}

			// Build a list of loaded modules which are not enabled
			TSet<FString> LazyLoadModules;
			for (LiveProcess* liveProcess : m_liveProcesses)
			{
				const std::unordered_map<std::wstring, LiveProcess::LazyLoadedModule>& lazyLoadedModules = liveProcess->GetLazyLoadedModules();
				for (const std::unordered_map<std::wstring, LiveProcess::LazyLoadedModule>::value_type& kvp : lazyLoadedModules)
				{
					if (!kvp.second.m_loaded)
					{
						LazyLoadModules.Add(FString(kvp.first.c_str()));
					}
				}
			}

			// Execute the compile
			TArray<FString> RequiredModules;
			ELiveCodingCompileResult CompileResult = CompileDelegate.Execute(Targets, ValidModules, LazyLoadModules, RequiredModules, ModuleToModuleFiles, CompileReason);
			if (CompileResult == ELiveCodingCompileResult::Success)
			{
				break;
			}
			else if (CompileResult == ELiveCodingCompileResult::Canceled)
			{
				postCompileResult = commands::PostCompileResult::Cancelled;
				GLiveCodingServer->GetCompileFinishedDelegate().ExecuteIfBound(ELiveCodingResult::Error, L"Compilation canceled.");
				return;
			}
			else if (CompileResult == ELiveCodingCompileResult::Failure)
			{
				postCompileResult = commands::PostCompileResult::Failure;
				GLiveCodingServer->GetCompileFinishedDelegate().ExecuteIfBound(ELiveCodingResult::Error, L"Compilation error.");
				return;
			}
			
			// Enable any lazy-loaded modules that we need
			if (!RequiredModules.IsEmpty() && !EnableRequiredModules(RequiredModules))
			{
				postCompileResult = commands::PostCompileResult::Failure;
				GLiveCodingServer->GetCompileFinishedDelegate().ExecuteIfBound(ELiveCodingResult::Error, L"Compilation error.");
				return;
			}

			CompileReason = ELiveCodingCompileReason::Retry;
		}

		// Reset the unity file cache
		symbols::ResetCachedUnityManifests();

		// Build up a list of all the modified object files in each module
		types::unordered_map<std::wstring, const LiveModule*> EnabledModulesByName;
		for (const LiveModule* liveModule : m_liveModules)
		{
			EnabledModulesByName[liveModule->GetModuleName()] = liveModule;
		}

		for(const TPair<FString, FModuleFiles>& Pair : ModuleToModuleFiles)
		{
			const LiveModule* liveModule = nullptr;
			std::wstring ModuleFileName = Filesystem::Devirtualize(Filesystem::NormalizePath(*Pair.Key).GetString());
			types::unordered_map<std::wstring, const LiveModule*>::iterator ix = EnabledModulesByName.find(ModuleFileName);
			if (ix == EnabledModulesByName.end())
			{
				// We couldn't find this exact module filename, but this could be a staged executable. See if we can just match the name.
				std::wstring ModuleFileNameOnly = Filesystem::GetFilename(ModuleFileName.c_str()).GetString();

				for (const LiveModule* testLiveModule : m_liveModules)
				{
					if (ModuleFileNameOnly == Filesystem::GetFilename(testLiveModule->GetModuleName().c_str()).GetString())
					{
						ModuleFileName = testLiveModule->GetModuleName();
						liveModule = testLiveModule;
						break;
					}
				}

				if (liveModule == nullptr)
				{
					LC_WARNING_USER("The module '%S' has not been loaded by any process. Changes will be ignored.", ModuleFileName.c_str());
					continue;
				}
			}
			else
			{
				liveModule = ix->second;
			}

			types::vector<symbols::ModifiedObjFile> ObjectFiles;
			for (const FString& ObjectFile : Pair.Value.Objects)
			{
				std::wstring NormalizedObjectFile = Filesystem::NormalizePath(*ObjectFile).GetString();

				if (!liveModule->IsModifiedSource(Filesystem::Devirtualize(NormalizedObjectFile).c_str()))
				{
					continue;
				}

				// If this file has a .lc.obj suffix, temporarily replace the original .obj file while generating the patch.
				// It'd be nice to track this explicitly inside Live++ and just load the new file, but it requires a lot of changes and would make upgrades difficult.
				static const TCHAR Suffix[] = TEXT(".lc.obj");
				static const size_t SuffixLen = UE_ARRAY_COUNT(Suffix) - 1;
				if (NormalizedObjectFile.length() >= SuffixLen && _wcsicmp(NormalizedObjectFile.c_str() + NormalizedObjectFile.length() - SuffixLen, Suffix) == 0)
				{
					// Get the original filename
					std::wstring OriginalObjectFile(NormalizedObjectFile.c_str(), NormalizedObjectFile.c_str() + NormalizedObjectFile.length() - SuffixLen);
					OriginalObjectFile += L".obj";

					// Back up the original file, if it exists
					Filesystem::PathAttributes OriginalFileAttributes = Filesystem::GetAttributes(OriginalObjectFile.c_str());
					if (Filesystem::DoesExist(OriginalFileAttributes))
					{
						std::wstring OriginalObjectFileBackup = OriginalObjectFile + L".lctmp";
						m_restoreFiles.push_back(std::make_pair(OriginalObjectFileBackup, OriginalObjectFile));
						Filesystem::DeleteIfExists(OriginalObjectFileBackup.c_str());
						Filesystem::Move(OriginalObjectFile.c_str(), OriginalObjectFileBackup.c_str());
					}

					// Move the new file into place
					m_restoreFiles.push_back(std::make_pair(OriginalObjectFile, NormalizedObjectFile));
					Filesystem::Move(NormalizedObjectFile.c_str(), OriginalObjectFile.c_str());
					NormalizedObjectFile = OriginalObjectFile;
				}

				// Add the file to the list of modifications
				symbols::ModifiedObjFile ModifiedObjFile;
				ModifiedObjFile.objPath = NormalizedObjectFile;
				ObjectFiles.push_back(std::move(ModifiedObjFile));
			}

			m_liveModuleToModifiedOrNewObjFiles.insert(std::make_pair(ModuleFileName, std::move(ObjectFiles)));

			types::vector<std::wstring> AdditionalLibraries;
			for (const FString& ObjectFile : Pair.Value.Libraries)
			{
				AdditionalLibraries.push_back(Filesystem::NormalizePath(*ObjectFile).GetString());
			}
			m_liveModuleToAdditionalLibraries.insert(std::make_pair(ModuleFileName, std::move(AdditionalLibraries)));
		}
	}

	GLiveCodingServer->GetStatusChangeDelegate().ExecuteIfBound(L"Creating patch...");
	// END EPIC MOD

	// recompile files, if any
	const size_t count = m_liveModules.size();
	if (count == 0u)
	{
		LC_LOG_USER("No live modules enabled");
	}

	LiveModule::ErrorType::Enum updateError = LiveModule::ErrorType::NO_CHANGE;

	// check directory notifications first to prune file changes based on directories
	m_directoryCache->PrimeNotifications();

	FileAttributeCache fileCache;

	// when all processes made progress, none of them is being held in the debugger which means it is safe to
	// communicate with the client, call hooks, use synchronization points, etc.
	// however, when a process was held in the debugger and now spins inside the code cave, we are not allowed
	// to call any of these functions, because that might lead to a deadlock.
	// similarly, if we're currently handling an exception, calling any of the client-provided functions could be fatal.
	const bool inExceptionHandler = m_inExceptionHandlerEvent.WaitTimeout(0u);
	const LiveModule::UpdateType::Enum updateType = (didAllProcessesMakeProgress && !inExceptionHandler)
		? LiveModule::UpdateType::DEFAULT
		: LiveModule::UpdateType::NO_CLIENT_COMMUNICATION;

	// BEGIN EPIC MOD
	// For Epic, don't even bother calling update if we have nothing that has been reported as modified/new.
	// This prevents the automatic change detection code from running which has issues with unity file changes.
	static const types::vector<std::wstring> emptyLibFiles;
	for (size_t i = 0u; i < count; ++i)
	{
		LiveModule* liveModule = m_liveModules[i];

		// try to find the list of modified or new .objs for this module
		const auto objFilesIt = m_liveModuleToModifiedOrNewObjFiles.find(liveModule->GetModuleName());
		if (objFilesIt == m_liveModuleToModifiedOrNewObjFiles.end() || objFilesIt->second.empty())
		{
			// no .objs for this module, ignore
			continue;
		}

		const auto libFilesIt = m_liveModuleToAdditionalLibraries.find(liveModule->GetModuleName());
		const types::vector<std::wstring>& libFiles = libFilesIt != m_liveModuleToAdditionalLibraries.end() ? libFilesIt->second : emptyLibFiles;

		// build a patch with the given list of .objs for this module
		const types::vector<symbols::ModifiedObjFile>& objFiles = objFilesIt->second;
		LiveModule::ErrorType::Enum moduleUpdateError = liveModule->Update(&fileCache, m_directoryCache, updateType, objFiles, libFiles);

		// only accept new error conditions for this module if there haven't been any updates until now.
		// this ensures that error conditions are kept and can be shown when updating several modules at once.
		if (updateError == LiveModule::ErrorType::NO_CHANGE)
		{
			updateError = moduleUpdateError;
		}
	}
	// END EPIC MOD

	// restart directory notifications for next compilation
	m_directoryCache->RestartNotifications();

	//EPIC REMOVED: g_theApp.GetMainFrame()->OnCompilationEnd();

	if (updateError == LiveModule::ErrorType::SUCCESS)
	{
		// bring Live++ to front on success
		if (appSettings::g_receiveFocusOnRecompile->GetValue() == appSettings::FocusOnRecompile::ON_SUCCESS)
		{
			// BEGIN EPIC MOD
			GLiveCodingServer->GetBringToFrontDelegate().ExecuteIfBound();
			// END EPIC MOD
		}

		// play sound on success
		const std::wstring soundOnSuccess = appSettings::g_playSoundOnSuccess->GetValue();
		if (soundOnSuccess.size() != 0u)
		{
			// first finish any sound that might still be playing, then play the real sound
			::PlaySoundW(NULL, NULL, 0u);
			::PlaySoundW(soundOnSuccess.c_str(), NULL, SND_FILENAME | SND_ASYNC | SND_NODEFAULT);
		}
	}

	if ((updateError == LiveModule::ErrorType::COMPILE_ERROR) ||
		(updateError == LiveModule::ErrorType::LINK_ERROR) ||
		(updateError == LiveModule::ErrorType::LOAD_PATCH_ERROR) ||
		(updateError == LiveModule::ErrorType::ACTIVATE_PATCH_ERROR))
	{
		// bring Live++ to front on failure
		if (appSettings::g_receiveFocusOnRecompile->GetValue() == appSettings::FocusOnRecompile::ON_ERROR)
		{
			// BEGIN EPIC MOD
			GLiveCodingServer->GetBringToFrontDelegate().ExecuteIfBound();
			// END EPIC MOD
		}

		// play sound on error
		const std::wstring soundOnError = appSettings::g_playSoundOnError->GetValue();
		if (soundOnError.size() != 0u)
		{
			// first finish any sound that might still be playing, then play the real sound
			::PlaySoundW(NULL, NULL, 0u);
			::PlaySoundW(soundOnError.c_str(), NULL, SND_FILENAME | SND_ASYNC | SND_NODEFAULT);
		}
	}

	// BEGIN EPIC MOD - Custom hooks for finishing compile
	bool setFocus = false;
	switch (updateError)
	{
		case LiveModule::ErrorType::NO_CHANGE:
			postCompileResult = commands::PostCompileResult::NoChanges;
			GLiveCodingServer->GetCompileFinishedDelegate().ExecuteIfBound(ELiveCodingResult::Success, L"No changes detected.");
			setFocus = !ShowCompileFinishNotification();
			break;

		case LiveModule::ErrorType::COMPILE_ERROR:
			postCompileResult = commands::PostCompileResult::Failure;
			GLiveCodingServer->GetCompileFinishedDelegate().ExecuteIfBound(ELiveCodingResult::Error, L"Compilation error.");
			break;

		case LiveModule::ErrorType::LINK_ERROR:
			postCompileResult = commands::PostCompileResult::Failure;
			GLiveCodingServer->GetCompileFinishedDelegate().ExecuteIfBound(ELiveCodingResult::Error, L"Linker error.");
			break;

		case LiveModule::ErrorType::LOAD_PATCH_ERROR:
			postCompileResult = commands::PostCompileResult::Failure;
			GLiveCodingServer->GetCompileFinishedDelegate().ExecuteIfBound(ELiveCodingResult::Error, L"Could not load patch image.");
			break;

		case LiveModule::ErrorType::ACTIVATE_PATCH_ERROR:
			postCompileResult = commands::PostCompileResult::Failure;
			GLiveCodingServer->GetCompileFinishedDelegate().ExecuteIfBound(ELiveCodingResult::Error, L"Could not activate patch.");
			break;

		case LiveModule::ErrorType::SUCCESS:
			postCompileResult = commands::PostCompileResult::Success;
			GLiveCodingServer->GetCompileFinishedDelegate().ExecuteIfBound(ELiveCodingResult::Success, L"Patch creation successful.");
			setFocus = true;
			break;

		default:
			postCompileResult = commands::PostCompileResult::Success;
			GLiveCodingServer->GetCompileFinishedDelegate().ExecuteIfBound(ELiveCodingResult::Success, L"Finished.");
			break;
	}

	if (setFocus)
	{
		EnumWindows(FocusApplicationWindows, (LPARAM)&m_liveProcesses);
	}
	// END EPIC MOD
	
	LC_LOG_USER("---------- Finished (%.3fs) ----------", scope.ReadSeconds());

	// EPIC REMOVED: g_theApp.GetMainFrame()->ResetStatusBarText();
	// EPIC REMOVED: g_theApp.GetMainFrame()->SetBusy(false);
}

// BEGIN EPIC MOD - Add the ability for pre and post compile notifications
void ServerCommandThread::CallPrecompileHooks(bool didAllProcessesMakeProgress)
{
	// when all processes made progress, none of them is being held in the debugger which means it is safe to
	// communicate with the client, call hooks, use synchronization points, etc.
	// however, when a process was held in the debugger and now spins inside the code cave, we are not allowed
	// to call any of these functions, because that might lead to a deadlock.
	// similarly, if we're currently handling an exception, calling any of the client-provided functions could be fatal.
	const bool inExceptionHandler = m_inExceptionHandlerEvent.WaitTimeout(0u);
	const LiveModule::UpdateType::Enum updateType = (didAllProcessesMakeProgress && !inExceptionHandler)
		? LiveModule::UpdateType::DEFAULT
		: LiveModule::UpdateType::NO_CLIENT_COMMUNICATION;

	if (updateType == LiveModule::UpdateType::NO_CLIENT_COMMUNICATION)
	{
		return;
	}

	const size_t count = m_liveProcesses.size();
	for (size_t i = 0u; i < count; ++i)
	{
		LiveProcess* liveProcess = m_liveProcesses[i];
		liveProcess->GetPipe()->SendCommandAndWaitForAck(commands::PreCompile{}, nullptr, 0u);
	}
}

void ServerCommandThread::CallPostcompileHooks(bool didAllProcessesMakeProgress, commands::PostCompileResult postCompileResult)
{
	// when all processes made progress, none of them is being held in the debugger which means it is safe to
	// communicate with the client, call hooks, use synchronization points, etc.
	// however, when a process was held in the debugger and now spins inside the code cave, we are not allowed
	// to call any of these functions, because that might lead to a deadlock.
	// similarly, if we're currently handling an exception, calling any of the client-provided functions could be fatal.
	const bool inExceptionHandler = m_inExceptionHandlerEvent.WaitTimeout(0u);
	const LiveModule::UpdateType::Enum updateType = (didAllProcessesMakeProgress && !inExceptionHandler)
		? LiveModule::UpdateType::DEFAULT
		: LiveModule::UpdateType::NO_CLIENT_COMMUNICATION;

	if (updateType == LiveModule::UpdateType::NO_CLIENT_COMMUNICATION)
	{
		return;
	}

	const size_t count = m_liveProcesses.size();
	for (size_t i = 0u; i < count; ++i)
	{
		LiveProcess* liveProcess = m_liveProcesses[i];
		liveProcess->GetPipe()->SendCommandAndWaitForAck(commands::PostCompile{ postCompileResult }, nullptr, 0u);
	}
}
// END EPIC MOD

Thread::ReturnValue ServerCommandThread::CompileThread(void)
{
	input::Key keyControl(VK_CONTROL);
	input::Key keyAlt(VK_MENU);
	input::Key keyShift(VK_SHIFT);
	input::Key keyShortcut(VK_F11);

	Event compilationEvent(primitiveNames::CompilationEvent(m_processGroupName).c_str(), Event::Type::MANUAL_RESET);

	ChangeNotification changeNotification;

	if (appSettings::g_continuousCompilationEnabled->GetValue())
	{
		changeNotification.Create(appSettings::g_continuousCompilationPath->GetValue());
	}

	for (;;)
	{
		// protect against concurrent restarts
		m_restartCS.Enter();

		const int shortcutValue = appSettings::g_compileShortcut->GetValue();
		keyShortcut.AssignCode(shortcut::GetVirtualKeyCode(shortcutValue));

		keyControl.Clear();
		keyAlt.Clear();
		keyShift.Clear();
		keyShortcut.Clear();

		keyControl.Update();
		keyAlt.Update();
		keyShift.Update();
		keyShortcut.Update();

		// BEGIN EPIC MOD - Adding SetActive command
		if(!m_active)
		{
			keyShortcut.Clear();
		}
		// END EPIC MOD

		const bool control = shortcut::ContainsControl(shortcutValue) ? keyControl.IsPressed() : !keyControl.IsPressed();
		const bool alt = shortcut::ContainsAlt(shortcutValue) ? keyAlt.IsPressed() : !keyAlt.IsPressed();
		const bool shift = shortcut::ContainsShift(shortcutValue) ? keyShift.IsPressed() : !keyShift.IsPressed();
		const bool isShortcutPressed = (control && alt && shift && keyShortcut.WentDown());

		// did anything change in the watched directory?
		const unsigned int changeNotificationTimeout = static_cast<unsigned int>(appSettings::g_continuousCompilationTimeout->GetValue());

		const bool foundAnyModification = changeNotification.CheckOnce();
		if (isShortcutPressed || foundAnyModification || m_manualRecompileTriggered)
		{
			// clear the log if desired by the user
			if (appSettings::g_clearLogOnRecompile->GetValue())
			{
				// BEGIN EPIC MOD
				GLiveCodingServer->GetClearOutputDelegate().ExecuteIfBound();
				// END EPIC MOD
			}

			if (foundAnyModification)
			{
				LC_SUCCESS_USER("Detected file modification, re-checking until timeout (%d ms)", changeNotificationTimeout);
				changeNotification.CheckNext(changeNotificationTimeout);
			}
			else if (isShortcutPressed)
			{
				// BEGIN EPIC MOD
				LC_SUCCESS_USER("Accepted Live coding shortcut");
				// END EPIC MDO
			}
			else if (m_manualRecompileTriggered)
			{
				LC_SUCCESS_USER("Manual recompile triggered");
			}
		}

		if (isShortcutPressed || foundAnyModification || m_manualRecompileTriggered)
		{
			// forbid command thread to handle commands through the pipe
			m_handleCommandsEvent.Reset();

			// tell clients that we're about to compile.
			// clients will send a command to say that they're ready. this command will let the command thread
			// rest until we signal the event again.
			compilationEvent.Signal();

			// remove inactive/disconnected processes
			{
				for (auto processIt = m_liveProcesses.begin(); processIt != m_liveProcesses.end(); /* nothing */)
				{
					LiveProcess* liveProcess = *processIt;
					Process::Handle processHandle = liveProcess->GetProcessHandle();
					if (!Process::IsActive(processHandle))
					{
						LC_WARNING_USER("Process %d is no longer valid, disconnecting", liveProcess->GetProcessId());

						Process::Close(processHandle);

						// tell live modules to remove this process
						const size_t moduleCount = m_liveModules.size();
						for (size_t j = 0u; j < moduleCount; ++j)
						{
							LiveModule* liveModule = m_liveModules[j];
							liveModule->UnregisterProcess(liveProcess);
						}

						// BEGIN EPIC MOD
						if (liveProcess->IsReinstancingFlowEnabled())
						{
							--m_reinstancingProcessCount;
						}

						if (liveProcess->IsDisableCompileFinishNotification())
						{
							--m_disableCompileFinishNotificationProcessCount;
						}
						// END EPIC MOD

						delete liveProcess;

						processIt = m_liveProcesses.erase(processIt);
					}
					else
					{
						// update process heart beats to know whether it made some progress
						liveProcess->ReadHeartBeatDelta(m_processGroupName.c_str());

						++processIt;
					}
				}
			}

			bool didAllProcessesMakeProgress = true;
			{
				const size_t processCount = m_liveProcesses.size();
				for (size_t i = 0u; i < processCount; ++i)
				{
					LiveProcess* liveProcess = m_liveProcesses[i];
					didAllProcessesMakeProgress &= liveProcess->MadeProgress();
				}
			}

			if (!didAllProcessesMakeProgress)
			{
				// BEGIN EPIC MOD
				LC_SUCCESS_USER("Possible debugger detected, can take a while to freeze threads");
				LC_SUCCESS_USER("Do not continue the execution of the process until the threads have been thawed");
				// END EPIC MOD

				// not all processes made progress.
				// this usually means that at least one of them is currently being debugged.
				// let each process handle this.
				const size_t processCount = m_liveProcesses.size();
				for (size_t i = 0u; i < processCount; ++i)
				{
					LiveProcess* liveProcess = m_liveProcesses[i];
					liveProcess->HandleDebuggingPreCompile();
				}

				// don't allow the exception handler dialog to be shown when continuing in the debugger with F5
				m_exceptionCS.Enter();
			}

			// wait until all command threads/clients are ready to go. we might not be getting commands
			// from a client because it is being held in the debugger.
			{
				CriticalSection::ScopedLock lock(&m_connectionCS);

				const size_t count = m_commandThreads.size();
				for (size_t i = 0u; i < count; ++i)
				{
					CommandThreadContext* threadContext = m_commandThreads[i];
					threadContext->readyEvent->Wait();
				}
			}

			// do not let other processes register new modules during compilation
			CriticalSection::ScopedLock actionLock(&m_actionCS);

			// setup the same virtual drive we had when loading the project
			AddVirtualDrive();

			// bring Live++ to front on shortcut trigger
			if (appSettings::g_receiveFocusOnRecompile->GetValue() == appSettings::FocusOnRecompile::ON_SHORTCUT)
			{
				// BEGIN EPIC MOD
				GLiveCodingServer->GetBringToFrontDelegate().ExecuteIfBound();
				// END EPIC MOD
			}

			// BEGIN EPIC MOD - Add the ability for pre and post compile notifications
			CallPrecompileHooks(didAllProcessesMakeProgress);
			// END EPIC MOD

			// BEGIN EPIC MOD 
			commands::PostCompileResult postCompileResult = commands::PostCompileResult::Success;
			CompileChanges(didAllProcessesMakeProgress, postCompileResult);
			// END EPIC MOD

			// BEGIN EPIC MOD - Add the ability for pre and post compile notifications
			CallPostcompileHooks(didAllProcessesMakeProgress, postCompileResult);
			// END EPIC MOD


			// BEGIN EPIC MOD - Non-destructive compile
			for (std::vector<std::pair<std::wstring, std::wstring>>::reverse_iterator it = m_restoreFiles.rbegin(); it != m_restoreFiles.rend(); it++)
			{
				Filesystem::DeleteIfExists(it->second.c_str());
				Filesystem::Move(it->first.c_str(), it->second.c_str());
			}
			m_restoreFiles.clear();
			// END EPIC MOD

			RemoveVirtualDrive();

			if (!didAllProcessesMakeProgress)
			{
				// BEGIN EPIC MOD
				LC_SUCCESS_USER("Possible debugger detected, can take a while to thaw threads");
				// END EPIC MOD

				const size_t processCount = m_liveProcesses.size();
				for (size_t i = 0u; i < processCount; ++i)
				{
					LiveProcess* liveProcess = m_liveProcesses[i];
					liveProcess->HandleDebuggingPostCompile();
				}

				// remove the lock on the exception handler dialog
				m_exceptionCS.Leave();

				// BEGIN EPIC MOD
				LC_SUCCESS_USER("Thawing process has completed. You may now resume process execution");
				// END EPIC MOD
			}

			compilationEvent.Reset();

			m_handleCommandsEvent.Signal();

			// clear change notifications that might have happened while compiling
			changeNotification.Check(0u);

			// clear API recompiles
			m_manualRecompileTriggered = false;
			m_liveModuleToModifiedOrNewObjFiles.clear();
			// BEGIN EPIC MOD
			m_liveModuleToAdditionalLibraries.clear();
			// END EPIC MOD
		}

		m_restartCS.Leave();

		Thread::Current::SleepMilliSeconds(10u);
	}

	return Thread::ReturnValue(0u);
}


Thread::ReturnValue ServerCommandThread::CommandThread(DuplexPipeServer* pipe, Event* readyEvent)
{
	// handle incoming commands
	CommandMap commandMap;
	commandMap.RegisterAction<actions::TriggerRecompile>();
	commandMap.RegisterAction<actions::TriggerRestart>();
	commandMap.RegisterAction<actions::LogMessage>();
	commandMap.RegisterAction<actions::BuildPatch>();
	commandMap.RegisterAction<actions::ReadyForCompilation>();
	commandMap.RegisterAction<actions::DisconnectClient>();
	commandMap.RegisterAction<actions::RegisterProcess>();
	commandMap.RegisterAction<actions::EnableModules>();
	commandMap.RegisterAction<actions::DisableModules>();
	commandMap.RegisterAction<actions::ApplySettingBool>();
	commandMap.RegisterAction<actions::ApplySettingInt>();
	commandMap.RegisterAction<actions::ApplySettingString>();
	// BEGIN EPIC MOD - Adding ShowConsole command
	commandMap.RegisterAction<actions::ShowConsole>();
	// END EPIC MOD
	// BEGIN EPIC MOD - Adding SetVisible command
	commandMap.RegisterAction<actions::SetVisible>();
	// END EPIC MOD
	// BEGIN EPIC MOD - Adding SetActive command
	commandMap.RegisterAction<actions::SetActive>();
	// END EPIC MOD
	// BEGIN EPIC MOD - Adding SetBuildArguments command
	commandMap.RegisterAction<actions::SetBuildArguments>();
	// END EPIC MOD
	// BEGIN EPIC MOD - Support for lazy-loading modules
	commandMap.RegisterAction<actions::EnableLazyLoadedModule>();
	// END EPIC MOD
	// BEGIN EPIC MOD
	commandMap.RegisterAction<actions::SetReinstancingFlow>();
	commandMap.RegisterAction<actions::DisableCompileFinishNotification>();
	// END EPIC MOD
	// BEGIN EPIC MOD
	commandMap.RegisterAction<actions::EnableModulesEx>();
	// END EPIC MOD

	for (;;)
	{
		const bool success = commandMap.HandleCommands(pipe, this);

		// we must have received a ReadyForCompilation command to get here, or the pipe is broken.
		// in any case, let the main server thread responsible for compilation know that this client is ready.
		// this is needed to always let the compilation thread advance, even when a client might have disconnected.
		readyEvent->Signal();

		if ((!success) || (!pipe->IsValid()))
		{
			// pipe was closed or is broken, bail out.
			// remove ourselves from the array of threads first.
			RemoveCommandThread(pipe);
			return Thread::ReturnValue(1u);
		}

		// wait until we're allowed to handle commands again
		m_handleCommandsEvent.Wait();

		// tell client that compilation has finished
		pipe->SendCommandAndWaitForAck(commands::CompilationFinished {}, nullptr, 0u);
	}

	RemoveCommandThread(pipe);
	return Thread::ReturnValue(0u);
}


Thread::ReturnValue ServerCommandThread::ExceptionCommandThread(DuplexPipeServer* exceptionPipe)
{
	// handle incoming exception commands
	CommandMap commandMap;
	commandMap.RegisterAction<actions::HandleException>();

	for (;;)
	{
		const bool success = commandMap.HandleCommands(exceptionPipe, this);
		if ((!success) || (!exceptionPipe->IsValid()))
		{
			// pipe was closed or is broken, bail out
			return Thread::ReturnValue(1u);
		}
	}

	return Thread::ReturnValue(0u);
}


void ServerCommandThread::RemoveCommandThread(const DuplexPipe* pipe)
{
	CriticalSection::ScopedLock lock(&m_connectionCS);

	const size_t count = m_commandThreads.size();
	for (size_t i = 0u; i < count; ++i)
	{
		CommandThreadContext* threadContext = m_commandThreads[i];
		if (&threadContext->pipe == pipe)
		{
			// don't bother cleaning up the context, just remove it
			auto it = m_commandThreads.begin();
			std::advance(it, i);
			m_commandThreads.erase(it);

			return;
		}
	}
}


LiveProcess* ServerCommandThread::FindProcessById(Process::Id processId)
{
	const size_t count = m_liveProcesses.size();
	for (size_t i = 0u; i < count; ++i)
	{
		LiveProcess* process = m_liveProcesses[i];
		if (process->GetProcessId() == processId)
		{
			return process;
		}
	}

	return nullptr;
}


bool ServerCommandThread::actions::TriggerRecompile::Execute(const CommandType*, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	ServerCommandThread* commandThread = static_cast<ServerCommandThread*>(context);

	// protect against accepting this command while compilation is already in progress
	CriticalSection::ScopedLock lock(&commandThread->m_actionCS);

	pipe->SendAck();

	commandThread->m_manualRecompileTriggered = true;

	return true;
}


bool ServerCommandThread::actions::TriggerRestart::Execute(const CommandType*, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	ServerCommandThread* commandThread = static_cast<ServerCommandThread*>(context);

	// protect against accepting this command while compilation is already in progress
	CriticalSection::ScopedLock lock(&commandThread->m_actionCS);

	pipe->SendAck();

	commandThread->RestartTargets();

	return true;
}


bool ServerCommandThread::actions::LogMessage::Execute(const CommandType*, const DuplexPipe* pipe, void*, const void* payload, size_t)
{
	LC_LOG_USER("%S", static_cast<const wchar_t*>(payload));

	pipe->SendAck();

	return true;
}


bool ServerCommandThread::actions::BuildPatch::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void* payload, size_t payloadSize)
{
	pipe->SendAck();

	ServerCommandThread* commandThread = static_cast<ServerCommandThread*>(context);

	// protect against accepting this command while compilation is already in progress
	CriticalSection::ScopedLock lock(&commandThread->m_actionCS);

	memoryStream::Reader payloadStream(payload, payloadSize);
	for (unsigned int i = 0u; i < command->fileCount; ++i)
	{
		const commands::BuildPatch::PatchData patchData = payloadStream.Read<commands::BuildPatch::PatchData>();
		const symbols::ModifiedObjFile modifiedObjFile = { patchData.objPath, patchData.amalgamatedObjPath };

		commandThread->m_liveModuleToModifiedOrNewObjFiles[patchData.moduleName].push_back(modifiedObjFile);
	}

	commandThread->m_manualRecompileTriggered = true;

	return true;
}


bool ServerCommandThread::actions::HandleException::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	pipe->SendAck();

	// BEGIN EPIC MOD - Using internal CrashReporter
#if 0
	ServerCommandThread* commandThread = static_cast<ServerCommandThread*>(context);

	// protect against several processes showing a dialog at the same time.
	// protect against showing the exception handler dialog while compilation is already in progress.
	CriticalSection::ScopedLock lock(&commandThread->m_exceptionCS);

	LiveProcess* liveProcess = commandThread->FindProcessById(command->processId);
	if (!liveProcess)
	{
		// signal client we did not handle the exception
		pipe->SendCommandAndWaitForAck(commands::HandleExceptionFinished { nullptr, nullptr, nullptr, false }, nullptr, 0u);
		return true;
	}

	// let the compile thread know that we're currently handling an exception.
	// this is needed to ensure that no hooks or synchronization points are called during compilation.
	commandThread->m_inExceptionHandlerEvent.Signal();

	// hold all processes in place
	const size_t processCount = commandThread->m_liveProcesses.size();
	for (size_t i = 0u; i < processCount; ++i)
	{
		commandThread->m_liveProcesses[i]->InstallCodeCave();
	}

	ExceptionHandlerDialog dialog(commandThread->m_processGroupName, liveProcess, command->threadId, command->exception, command->context, command->clientContextPtr);
	const INT_PTR result = dialog.DoModal();

	// release processes from the cave
	for (size_t i = 0u; i < processCount; ++i)
	{
		commandThread->m_liveProcesses[i]->UninstallCodeCave();
	}

	// remove our signal saying that we're handling an exception
	commandThread->m_inExceptionHandlerEvent.Reset();

	if (result == IDC_EXCEPTION_HANDLER_LEAVE)
	{
		// tell the client that it needs to unwind its stack and continue at the return address
		const ExceptionHandlerDialog::ParentFrameData& frameData = dialog.GetParentFrameData();
		pipe->SendCommandAndWaitForAck(commands::HandleExceptionFinished { frameData.returnAddress, frameData.framePointer, frameData.stackPointer, true }, nullptr, 0u);
		return true;
	}
	else if (result == IDC_EXCEPTION_HANDLER_IGNORE)
	{
		// tell the client that we ignored the exception
		pipe->SendCommandAndWaitForAck(commands::HandleExceptionFinished { nullptr, nullptr, nullptr, false }, nullptr, 0u);
		return true;
	}

	// signal client that we handled the exception and there's nothing left to do
	pipe->SendCommandAndWaitForAck(commands::HandleExceptionFinished { nullptr, nullptr, nullptr, true }, nullptr, 0u);
#endif
	// END EPIC MOD

	return true;
}


bool ServerCommandThread::actions::ReadyForCompilation::Execute(const CommandType*, const DuplexPipe* pipe, void*, const void*, size_t)
{
	pipe->SendAck();

	// don't continue execution
	return false;
}


bool ServerCommandThread::actions::DisconnectClient::Execute(const CommandType*, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	ServerCommandThread* instance = static_cast<ServerCommandThread*>(context);

	// unregister this connection
	{
		instance->RemoveCommandThread(pipe);

		CriticalSection::ScopedLock lock(&instance->m_connectionCS);
		if (instance->m_commandThreads.size() == 0u)
		{
			// BEGIN EPIC MOD - No built-in UI
			// // this was the last client to disconnect, remove the system tray
			// g_theApp.GetMainFrame()->GetSystemTray()->Destroy();
			// END EPIC MOD
		}
	}

	pipe->SendAck();

	return true;
}

// BEGIN EPIC MOD - Adding ShowConsole command
bool ServerCommandThread::actions::ShowConsole::Execute(const CommandType*, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	pipe->SendAck();

	GLiveCodingServer->GetShowConsoleDelegate().ExecuteIfBound();

	return true;
}
// END EPIC MOD

// BEGIN EPIC MOD - Adding SetVisible command
bool ServerCommandThread::actions::SetVisible::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	pipe->SendAck();

	GLiveCodingServer->GetSetVisibleDelegate().ExecuteIfBound(command->visible);

	return true;
}
// END EPIC MOD

// BEGIN EPIC MOD - Adding SetActive command
bool ServerCommandThread::actions::SetActive::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	ServerCommandThread* commandThread = static_cast<ServerCommandThread*>(context);

	// protect against accepting this command while compilation is already in progress
	CriticalSection::ScopedLock lock(&commandThread->m_actionCS);

	pipe->SendAck();

	commandThread->m_active = command->active;

	return true;
}
// END EPIC MOD

// BEGIN EPIC MOD - Adding SetBuildArguments command
bool ServerCommandThread::actions::SetBuildArguments::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	ServerCommandThread* commandThread = static_cast<ServerCommandThread*>(context);

	// protect against accepting this command while compilation is already in progress
	CriticalSection::ScopedLock lock(&commandThread->m_actionCS);

	for (LiveProcess* process : commandThread->m_liveProcesses)
	{
		if (process->GetProcessId() == command->processId)
		{
			process->SetBuildArguments(command->arguments);
		}
	}

	pipe->SendAck();

	return true;
}
// END EPIC MOD

// BEGIN EPIC MOD - Support for lazy-loading modules
bool ServerCommandThread::actions::EnableLazyLoadedModule::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	ServerCommandThread* commandThread = static_cast<ServerCommandThread*>(context);

	// protect against accepting this command while compilation is already in progress
	CriticalSection::ScopedLock lock(&commandThread->m_actionCS);

	// Check if this module is already enabled - it may have been lazy-loaded, then fully loaded, by a restarted process. If so, translate this into a call to EnableModules.
	const std::wstring modulePath = Filesystem::NormalizePath(command->fileName).GetString();
	for (LiveModule* module : commandThread->m_liveModules)
	{
		if(module->GetModuleName() == modulePath)
		{
			EnableModules::CommandType EnableCmd = { };
			EnableCmd.moduleCount = 1;
			EnableCmd.processId = command->processId;
			EnableCmd.token = command->token;

			commands::ModuleData Module;
			Module.base = command->moduleBase;
			wcscpy_s(Module.path, command->fileName);

			return EnableModules::Execute(&EnableCmd, pipe, context, &Module, sizeof(Module));
		}
	}

	// Acknowledge the command
	pipe->SendAck();

	// Register the module for lazy loading
	for (LiveProcess* process : commandThread->m_liveProcesses)
	{
		if (process->GetProcessId() == command->processId)
		{
			process->AddLazyLoadedModule(modulePath, command->moduleBase);
			LC_LOG_DEV("Registered module %S for lazy-loading", modulePath.c_str());
		}
	}

	// Tell the client we're done
	pipe->SendCommandAndWaitForAck(commands::EnableModulesFinished { command->token }, nullptr, 0u);
	return true;
}
// END EPIC MOD

// BEGIN EPIC MOD
bool ServerCommandThread::actions::SetReinstancingFlow::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	ServerCommandThread* commandThread = static_cast<ServerCommandThread*>(context);

	// protect against accepting this command while compilation is already in progress
	CriticalSection::ScopedLock lock(&commandThread->m_actionCS);

	for (LiveProcess* process : commandThread->m_liveProcesses)
	{
		if (process->GetProcessId() == command->processId)
		{
			if (command->enable)
			{
				if (!process->IsReinstancingFlowEnabled())
				{
					++commandThread->m_reinstancingProcessCount;
				}
			}
			else
			{
				if (process->IsReinstancingFlowEnabled())
				{
					--commandThread->m_reinstancingProcessCount;
				}
			}
			process->SetReinstancingFlow(command->enable);
		}
	}

	pipe->SendAck();

	return true;
}
// END EPIC MOD

// BEGIN EPIC MOD
bool ServerCommandThread::actions::DisableCompileFinishNotification::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void*, size_t)
{
	ServerCommandThread* commandThread = static_cast<ServerCommandThread*>(context);

	// protect against accepting this command while compilation is already in progress
	CriticalSection::ScopedLock lock(&commandThread->m_actionCS);

	for (LiveProcess* process : commandThread->m_liveProcesses)
	{
		if (process->GetProcessId() == command->processId)
		{
			++commandThread->m_disableCompileFinishNotificationProcessCount;
			process->DisableCompileFinishNotification();
		}
	}

	pipe->SendAck();

	return true;
}
// END EPIC MOD

bool ServerCommandThread::actions::RegisterProcess::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void* payload, size_t)
{
	pipe->SendAck();

	ServerCommandThread* commandThread = static_cast<ServerCommandThread*>(context);

	// protect against several client DLLs calling into this action at the same time
	CriticalSection::ScopedLock lock(&commandThread->m_actionCS);

	Process::Handle processHandle = Process::Open(command->processId);

	// check if any live module in this process group has patches installed already
	{
		const std::wstring& processPath = Process::GetImagePath(processHandle).GetString();

		bool registeredSuccessfully = true;
		if (!appSettings::g_installCompiledPatchesMultiProcess->GetValue())
		{
			// we are not allowed to install any compiled patches when a new executable is spawned
			bool processGroupHasPatches = false;
			const size_t count = commandThread->m_liveModules.size();
			for (size_t i = 0u; i < count; ++i)
			{
				LiveModule* liveModule = commandThread->m_liveModules[i];
				if (liveModule->HasInstalledPatches())
				{
					// BEGIN EPIC MOD
					std::wstring caption(L"Live coding - Registering process ");
					// END EPIC MOD
					caption += Filesystem::GetFilename(processPath.c_str()).GetString();

					processGroupHasPatches = true;
					// BEGIN EPIC MOD - Using non-modal error dialog
					GLiveCodingServer->GetLogOutputDelegate().ExecuteIfBound(ELiveCodingLogVerbosity::Failure, L"This process cannot be added to the existing process group, because at least one module already has installed patches. Live coding is disabled for this process.");
					// END EPIC MD
					break;
				}
			}

			registeredSuccessfully = !processGroupHasPatches;
		}

		if (registeredSuccessfully)
		{
			const wchar_t* imagePath = pointer::Offset<const wchar_t*>(payload, 0u);
			const wchar_t* commandLine = pointer::Offset<const wchar_t*>(imagePath, command->imagePathSize);
			const wchar_t* workingDirectory = pointer::Offset<const wchar_t*>(commandLine, command->commandLineSize);
			const void* environment = pointer::Offset<const wchar_t*>(workingDirectory, command->workingDirectorySize);

			LiveProcess* liveProcess = new LiveProcess(processHandle, command->processId, command->threadId, command->jumpToSelf, pipe, imagePath, commandLine, workingDirectory, environment, command->environmentSize);
			commandThread->m_liveProcesses.push_back(liveProcess);
			// BEGIN EPIC MOD - No built-in UI
			// commandThread->m_mainFrame->UpdateWindowTitle();
			// END EPIC MOD

			if (command->restartedProcessId == Process::Id(0u))
			{
				// this is a new process
				LC_SUCCESS_USER("Registered process %S (PID: %d)", processPath.c_str(), +command->processId);
			}
			else
			{
				// this process was restarted
				LC_SUCCESS_USER("Registered restarted process %S (PID: %d, previous PID: %d)", processPath.c_str(), +command->processId, command->restartedProcessId);

// BEGIN EPIC MOD
#if WITH_VISUALSTUDIO_DTE
// END EPIC MOD
				// reattach the debugger in case the previous process had a debugger attached
				{
					auto it = commandThread->m_restartedProcessIdToDebugger.find(+command->restartedProcessId);
					if (it != commandThread->m_restartedProcessIdToDebugger.end())
					{
						LC_LOG_USER("Reattaching debugger to PID %d", +command->processId);

						const EnvDTE::DebuggerPtr& debugger = it->second;
						const bool success = visualStudio::AttachToProcess(debugger, command->processId);
						if (!success)
						{
							LC_ERROR_USER("Failed to reattach debugger to PID %d", +command->processId);
						}

						commandThread->m_restartedProcessIdToDebugger.erase(it);
					}
				}
// BEGIN EPIC MOD
#endif
// END EPIC MOD

				--commandThread->m_restartedProcessCount;
				if (commandThread->m_restartedProcessCount == 0u)
				{
					// finished restarting, remove the job that kept this instance alive
					if (commandThread->m_restartJob)
					{
						::CloseHandle(commandThread->m_restartJob);
						commandThread->m_restartJob = nullptr;

						commandThread->m_restartCS.Leave();

						LC_LOG_USER("---------- Restarting finished ----------");

						// EPIC REMOVED: g_theApp.GetMainFrame()->ResetStatusBarText();
						// EPIC REMOVED: g_theApp.GetMainFrame()->SetBusy(false);
					}
					// BEGIN EPIC MOD - Prevent orphaned console instances if processes fail to restart. Job object will be duplicated into child process.
					else
					{
						commandThread->m_restartCS.Leave();
						LC_LOG_USER("---------- Restarting finished ----------");
					}
					// END EPIC MOD
				}
			}
		}

		// tell client we are finished
		pipe->SendCommandAndWaitForAck(commands::RegisterProcessFinished { registeredSuccessfully }, nullptr, 0u);
	}

	return true;
}


bool ServerCommandThread::actions::EnableModules::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void* payload, size_t payloadSize)
{
	pipe->SendAck();

	// EPIC REMOVED: g_theApp.GetMainFrame()->ChangeStatusBarText(L"Loading modules...");

	ServerCommandThread* commandThread = static_cast<ServerCommandThread*>(context);

	// protect against several client DLLs calling into this action at the same time.
	// this ensures that all modules are loaded serialized per process.
	CriticalSection::ScopedLock lock(&commandThread->m_actionCS);

	telemetry::Scope moduleLoadingScope("Module loading");

	// set up virtual drives before loading anything, otherwise files won't be detected and therefore discarded
	AddVirtualDrive();

	scheduler::TaskBase* rootTask = scheduler::CreateEmptyTask();
	types::vector<scheduler::Task<LiveModule*>*> loadModuleTasks;

	const unsigned int moduleCount = command->moduleCount;
	loadModuleTasks.reserve(moduleCount);

	memoryStream::Reader payloadStream(payload, payloadSize);
	for (unsigned int i = 0u; i < moduleCount; ++i)
	{
		const commands::ModuleData moduleData = payloadStream.Read<commands::ModuleData>();
		scheduler::Task<LiveModule*>* task = commandThread->LoadModule(command->processId, moduleData.base, moduleData.path, rootTask);

		// the module could have failed to load
		if (task)
		{
			loadModuleTasks.push_back(task);
		}
	}

	// wait for all tasks to finish
	scheduler::RunTask(rootTask);
	scheduler::WaitForTask(rootTask);

	const size_t loadModuleTaskCount = loadModuleTasks.size();
	commandThread->m_liveModules.reserve(loadModuleTaskCount);

	size_t loadedTranslationUnits = 0u;

	// update all live modules loaded by the tasks
	for (size_t i = 0u; i < loadModuleTaskCount; ++i)
	{
		scheduler::Task<LiveModule*>* task = loadModuleTasks[i];
		LiveModule* liveModule = task->GetResult();

		commandThread->m_liveModules.push_back(liveModule);

		// update directory cache for this live module
		liveModule->UpdateDirectoryCache(commandThread->m_directoryCache);

		// update the number of loaded translation units
		loadedTranslationUnits += liveModule->GetCompilandDatabase()->compilands.size();
	}

	scheduler::DestroyTasks(loadModuleTasks);
	scheduler::DestroyTask(rootTask);

	// dump memory statistics
	{
		LC_LOG_INDENT_TELEMETRY;
		g_symbolAllocator.PrintStats();
		g_immutableStringAllocator.PrintStats();
		g_contributionAllocator.PrintStats();
		g_compilandAllocator.PrintStats();
		g_dependencyAllocator.PrintStats();
	}

	// BEGIN EPIC MOD - Suppress output when lazy loading modules
	if (+commandThread->m_compileThread == nullptr || Thread::Current::GetId() != Thread::GetId(commandThread->m_compileThread))
	{
		if (loadModuleTaskCount > 0u)
		{
			LC_SUCCESS_USER("Loaded %zu module(s) (%.3fs, %zu translation units)", loadModuleTaskCount, moduleLoadingScope.ReadSeconds(), loadedTranslationUnits);
		}

		// EPIC REMOVED commandThread->PrewarmCompilerEnvironmentCache();

		// tell user we are ready, but only once to not clutter the log
		{
			static bool showedOnce = false;
			if (!showedOnce)
			{
				showedOnce = true;
				const int shortcut = appSettings::g_compileShortcut->GetValue();
				const std::wstring& shortcutText = shortcut::ConvertShortcutToText(shortcut);
				LC_SUCCESS_USER("Live coding ready - Save changes and press %S to re-compile code", shortcutText.c_str());
			}
		}
	}
	// END EPIC MOD

	// remove virtual drives once we're finished
	RemoveVirtualDrive();

	// tell server we are finished
	pipe->SendCommandAndWaitForAck(commands::EnableModulesFinished { command->token }, nullptr, 0u);

	// EPIC REMOVED: g_theApp.GetMainFrame()->ResetStatusBarText();

	return true;
}

// BEGIN EPIC MOD
bool ServerCommandThread::actions::EnableModulesEx::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void* payload, size_t payloadSize)
{
	pipe->SendAck();

	ServerCommandThread* commandThread = static_cast<ServerCommandThread*>(context);

	// protect against several client DLLs calling into this action at the same time.
	// this ensures that all modules are loaded serialized per process.
	CriticalSection::ScopedLock lock(&commandThread->m_actionCS);

	telemetry::Scope moduleLoadingScope("Module loading ex");

	// Locate the process
	LiveProcess* liveProcess = nullptr;
	for (LiveProcess* process : commandThread->m_liveProcesses)
	{
		if (process->GetProcessId() == command->processId)
		{
			liveProcess = process;
			break;
		}
	}

	// set up virtual drives before loading anything, otherwise files won't be detected and therefore discarded
	AddVirtualDrive();

	scheduler::TaskBase* rootTask = scheduler::CreateEmptyTask();
	types::vector<scheduler::Task<LiveModule*>*> loadModuleTasks;

	const unsigned int moduleCount = command->moduleCount;
	loadModuleTasks.reserve(moduleCount);

	memoryStream::Reader payloadStream(payload, payloadSize);
	for (unsigned int i = 0u; i < moduleCount; ++i)
	{
		const commands::ModuleData moduleData = payloadStream.Read<commands::ModuleData>();
		scheduler::Task<LiveModule*>* task = commandThread->LoadModule(command->processId, moduleData.base, moduleData.path, rootTask);

		// the module could have failed to load
		if (task)
		{
			loadModuleTasks.push_back(task);
		}
	}

	const unsigned int lazyModuleCount = command->lazyLoadModuleCount;
	unsigned int loadedLazyModules = 0;
	for (unsigned int i = 0u; i < lazyModuleCount; ++i)
	{
		const commands::ModuleData moduleData = payloadStream.Read<commands::ModuleData>();

		// Check if this module is already enabled - it may have been lazy-loaded, then fully loaded, by a restarted process. If so, translate this into a call to EnableModules.
		bool isEnabled = false;
		const std::wstring modulePath = Filesystem::NormalizePath(moduleData.path).GetString();
		for (LiveModule* module : commandThread->m_liveModules)
		{
			if (module->GetModuleName() == modulePath)
			{
				scheduler::Task<LiveModule*>* task = commandThread->LoadModule(command->processId, moduleData.base, moduleData.path, rootTask);
				if (task)
				{
					loadModuleTasks.push_back(task);
				}
				isEnabled = true;
				break;
			}
		}
		
		if (!isEnabled && liveProcess != nullptr)
		{
			liveProcess->AddLazyLoadedModule(modulePath, static_cast<Windows::HMODULE>(moduleData.base));
			LC_LOG_DEV("Registered module %S for lazy-loading", modulePath.c_str());
			++loadedLazyModules;
		}
	}

	const unsigned int reservedPagesCount = command->reservedPagesCount;
	for (unsigned int i = 0u; i < reservedPagesCount; ++i)
	{
		const uintptr_t page = payloadStream.Read<uintptr_t>();
		if (liveProcess != nullptr)
		{
			liveProcess->AddPage(reinterpret_cast<void*>(page));
		}
	}

	// wait for all tasks to finish
	scheduler::RunTask(rootTask);
	scheduler::WaitForTask(rootTask);

	const size_t loadModuleTaskCount = loadModuleTasks.size();
	commandThread->m_liveModules.reserve(loadModuleTaskCount);

	size_t loadedTranslationUnits = 0u;

	// update all live modules loaded by the tasks
	for (size_t i = 0u; i < loadModuleTaskCount; ++i)
	{
		scheduler::Task<LiveModule*>* task = loadModuleTasks[i];
		LiveModule* liveModule = task->GetResult();

		commandThread->m_liveModules.push_back(liveModule);

		// update directory cache for this live module
		liveModule->UpdateDirectoryCache(commandThread->m_directoryCache);

		// update the number of loaded translation units
		loadedTranslationUnits += liveModule->GetCompilandDatabase()->compilands.size();
	}

	scheduler::DestroyTasks(loadModuleTasks);
	scheduler::DestroyTask(rootTask);

	// dump memory statistics
	{
		LC_LOG_INDENT_TELEMETRY;
		g_symbolAllocator.PrintStats();
		g_immutableStringAllocator.PrintStats();
		g_contributionAllocator.PrintStats();
		g_compilandAllocator.PrintStats();
		g_dependencyAllocator.PrintStats();
	}

	if (+commandThread->m_compileThread == nullptr || Thread::Current::GetId() != Thread::GetId(commandThread->m_compileThread))
	{
		if (loadModuleTaskCount > 0u)
		{
			LC_SUCCESS_USER("Loaded %zu module(s), %u lazy load module(s), and %u reserved page ranges (%.3fs, %zu translation units)", 
				loadModuleTaskCount, loadedLazyModules, reservedPagesCount, moduleLoadingScope.ReadSeconds(), loadedTranslationUnits);
		}

		// tell user we are ready, but only once to not clutter the log
		{
			static bool showedOnce = false;
			if (!showedOnce)
			{
				showedOnce = true;
				const int shortcut = appSettings::g_compileShortcut->GetValue();
				const std::wstring& shortcutText = shortcut::ConvertShortcutToText(shortcut);
				LC_SUCCESS_USER("Live coding ready - Save changes and press %S to re-compile code", shortcutText.c_str());
			}
		}
	}

	// remove virtual drives once we're finished
	RemoveVirtualDrive();

	// tell server we are finished
	pipe->SendCommandAndWaitForAck(commands::EnableModulesFinished{ command->token }, nullptr, 0u);

	return true;
}
// END EPIC MOD

bool ServerCommandThread::actions::DisableModules::Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void* payload, size_t payloadSize)
{
	pipe->SendAck();

	// EPIC REMOVED: g_theApp.GetMainFrame()->ChangeStatusBarText(L"Unloading modules...");

	ServerCommandThread* commandThread = static_cast<ServerCommandThread*>(context);

	// protect against several client DLLs calling into this action at the same time.
	// this ensures that all modules are loaded serialized per process.
	CriticalSection::ScopedLock lock(&commandThread->m_actionCS);

	telemetry::Scope moduleUnloadingScope("Module unloading");

	unsigned int unloadedModules = 0u;
	const unsigned int moduleCount = command->moduleCount;
	memoryStream::Reader payloadStream(payload, payloadSize);
	for (unsigned int i = 0u; i < moduleCount; ++i)
	{
		const commands::ModuleData moduleData = payloadStream.Read<commands::ModuleData>();
		const bool success = commandThread->UnloadModule(command->processId, moduleData.path);
		if (success)
		{
			++unloadedModules;
		}
	}

	if (unloadedModules > 0u)
	{
		LC_SUCCESS_USER("Unloaded %u module(s) (%.3fs)", unloadedModules, moduleUnloadingScope.ReadSeconds());
	}

	// tell server we are finished
	pipe->SendCommandAndWaitForAck(commands::DisableModulesFinished { command->token }, nullptr, 0u);

	// EPIC REMOVED: g_theApp.GetMainFrame()->ResetStatusBarText();

	return true;
}


bool ServerCommandThread::actions::ApplySettingBool::Execute(const CommandType* command, const DuplexPipe* pipe, void*, const void*, size_t)
{
	pipe->SendAck();

	appSettings::ApplySettingBool(command->settingName, (command->settingValue == 0) ? false : true);

	return true;
}


bool ServerCommandThread::actions::ApplySettingInt::Execute(const CommandType* command, const DuplexPipe* pipe, void*, const void*, size_t)
{
	pipe->SendAck();

	appSettings::ApplySettingInt(command->settingName, command->settingValue);

	return true;
}


bool ServerCommandThread::actions::ApplySettingString::Execute(const CommandType* command, const DuplexPipe* pipe, void*, const void*, size_t)
{
	pipe->SendAck();

	appSettings::ApplySettingString(command->settingName, command->settingValue);

	return true;
}

// BEGIN EPIC MODS
#pragma warning(pop)
// END EPIC MODS


#endif