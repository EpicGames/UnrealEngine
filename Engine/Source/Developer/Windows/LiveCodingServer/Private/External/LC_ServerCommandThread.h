// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

#include "LC_ThreadTypes.h"
#include "LC_ProcessTypes.h"
#include "LC_Commands.h"
#include "LC_Telemetry.h"
#include "LC_DuplexPipeServer.h"
#include "LC_CriticalSection.h"
#include "LC_Event.h"
#include "LC_Scheduler.h"
#include "LC_Executable.h"
#include "LC_RunMode.h"
#include "LC_LiveModule.h"
// BEGIN EPIC MOD
#include "LC_Types.h"
#include "VisualStudioDTE.h"
#include <atomic>
// END EPIC MOD

class MainFrame;
class DirectoryCache;
class LiveModule;
class LiveModuleOrbis;
class LiveProcess;

class ServerCommandThread
{
public:
	ServerCommandThread(MainFrame* mainFrame, const wchar_t* const processGroupName, RunMode::Enum runMode);
	~ServerCommandThread(void);

	void RestartTargets(void);
	void CompileChanges(void);

	std::wstring GetProcessImagePath(void) const;

	// BEGIN EPIC MOD
	bool HasReinstancingProcess();
	// END EPIC MOD

	// BEGIN EPIC MOD
	bool ShowCompileFinishNotification();
	// END EPIC MOD

private:
	scheduler::Task<LiveModule*>* LoadModule(Process::Id processId, void* moduleBase, const wchar_t* modulePath, scheduler::TaskBase* taskRoot);
	bool UnloadModule(Process::Id processId, const wchar_t* modulePath);

	void PrewarmCompilerEnvironmentCache(void);

	Thread::ReturnValue ServerThread(void);
	Thread::ReturnValue CompileThread(void);

	// BEGIN EPIC MOD - Add the ability for pre and post compile notifications
	void CallPrecompileHooks(bool didAllProcessesMakeProgress);
	void CallPostcompileHooks(bool didAllProcessesMakeProgress, commands::PostCompileResult postCompileResult);
	// END EPIC MOD

	struct CommandThreadContext
	{
		DuplexPipeServer pipe;
		Event* readyEvent;
		Thread::Handle commandThread;

		DuplexPipeServer exceptionPipe;
		Thread::Handle exceptionCommandThread;
	};

	Thread::ReturnValue CommandThread(DuplexPipeServer* pipe, Event* readyEvent);
	Thread::ReturnValue ExceptionCommandThread(DuplexPipeServer* exceptionPipe);

	void RemoveCommandThread(const DuplexPipe* pipe);

	LiveProcess* FindProcessById(Process::Id processId);

	// BEGIN EPIC MOD
	void CompileChanges(bool didAllProcessesMakeProgress, commands::PostCompileResult& postCompileResult);
	// END EPIC MOD

	// actions
	struct actions
	{
		#define DECLARE_ACTION(_name)																													\
			struct _name																																\
			{																																			\
				typedef ::commands::_name CommandType;																									\
				static bool Execute(const CommandType* command, const DuplexPipe* pipe, void* context, const void* payload, size_t payloadSize);		\
			}

		DECLARE_ACTION(TriggerRecompile);
		DECLARE_ACTION(TriggerRestart);
		DECLARE_ACTION(LogMessage);
		DECLARE_ACTION(BuildPatch);
		DECLARE_ACTION(HandleException);
		DECLARE_ACTION(ReadyForCompilation);
		DECLARE_ACTION(DisconnectClient);
		DECLARE_ACTION(RegisterProcess);

		DECLARE_ACTION(EnableModules);
		DECLARE_ACTION(DisableModules);

		DECLARE_ACTION(ApplySettingBool);
		DECLARE_ACTION(ApplySettingInt);
		DECLARE_ACTION(ApplySettingString);

		// BEGIN EPIC MOD - Adding ShowConsole command
		DECLARE_ACTION(ShowConsole);
		// END EPIC MOD

		// BEGIN EPIC MOD - Adding ShowConsole command
		DECLARE_ACTION(SetVisible);
		// END EPIC MOD

		// BEGIN EPIC MOD - Adding SetActive command
		DECLARE_ACTION(SetActive);
		// END EPIC MOD

		// BEGIN EPIC MOD - Adding SetBuildArguments command
		DECLARE_ACTION(SetBuildArguments);
		// END EPIC MOD

		// BEGIN EPIC MOD - Adding support for lazy-loading modules
		DECLARE_ACTION(EnableLazyLoadedModule);
		DECLARE_ACTION(FinishedLazyLoadingModules);
		// END EPIC MOD

		// BEGIN EPIC MOD
		DECLARE_ACTION(SetReinstancingFlow);
		// END EPIC MOD

		// BEGIN EPIC MOD
		DECLARE_ACTION(DisableCompileFinishNotification);
		// END EPIC MOD

		// BEGIN EPIC MOD
		DECLARE_ACTION(EnableModulesEx);
		// END EPIC MOD

		#undef DECLARE_ACTION
	};


	std::wstring m_processGroupName;
	RunMode::Enum m_runMode;

	MainFrame* m_mainFrame;
	Thread::Handle m_serverThread;
	Thread::Handle m_compileThread;

	types::vector<LiveModule*> m_liveModules;
	types::vector<LiveProcess*> m_liveProcesses;
	types::unordered_map<executable::Header, LiveModule*> m_imageHeaderToLiveModule;

	CriticalSection m_actionCS;
	CriticalSection m_exceptionCS;
	Event m_inExceptionHandlerEvent;
	Event m_handleCommandsEvent;

	// directory cache for all modules combined
	DirectoryCache* m_directoryCache;

	// keeping track of the client connections
	CriticalSection m_connectionCS;
	types::vector<CommandThreadContext*> m_commandThreads;

	// BEGIN EPIC MOD
	std::atomic<unsigned int> m_reinstancingProcessCount = 0;
	// END EPIC MOD

	// BEGIN EPIC MOD
	std::atomic<unsigned int> m_disableCompileFinishNotificationProcessCount = 0;
	// END EPIC MOD

	// BEGIN EPIC MOD - Non-destructive compile
	std::vector<std::pair<std::wstring, std::wstring>> m_restoreFiles;
	// END EPIC MOD

	// BEGIN EPIC MOD - Adding SetActive command
	bool m_active = true;
	// END EPIC MOD

	// BEGIN EPIC MOD - Lazy loading modules
	bool EnableRequiredModules(const TArray<FString>& RequiredModules);
	// END EPIC MOD

	// for triggering recompiles using the API
	bool m_manualRecompileTriggered;
	types::unordered_map<std::wstring, types::vector<symbols::ModifiedObjFile>> m_liveModuleToModifiedOrNewObjFiles;

	// BEGIN EPIC MOD
	types::unordered_map<std::wstring, types::vector<std::wstring>> m_liveModuleToAdditionalLibraries;
	// END EPIC MOD

	// restart mechanism
	CriticalSection m_restartCS;
	void* m_restartJob;
	unsigned int m_restartedProcessCount;
// BEGIN EPIC MOD
#if WITH_VISUALSTUDIO_DTE
// END EPIC MOD
	types::unordered_map<unsigned int, EnvDTE::DebuggerPtr> m_restartedProcessIdToDebugger;
// BEGIN EPIC MOD
#endif
// END EPIC MOD
};


#endif