// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Settings.h"
#include "LC_AppSettings.h"
#include "LC_Filesystem.h"
#include "LC_StringUtil.h"


namespace
{
	static unsigned int g_userSettingsLoaded = 0u;
	static unsigned int g_projectSettingsLoaded = 0u;


	static UINT LoadSetting(const wchar_t* group, const wchar_t* name, int initialValue)
	{
		// BEGIN EPIC MOD - Just use default Epic settings
#if 1
		return initialValue;
#else
		// first try loading the setting from the project settings file
		const std::wstring& projectSettingsPath = appSettings::GetProjectSettingsPath();
		const Filesystem::PathAttributes& attributes = Filesystem::GetAttributes(projectSettingsPath.c_str());
		if (Filesystem::DoesExist(attributes))
		{
			// a file is there, so try loading the setting
			const UINT ILLEGAL_VALUE = static_cast<UINT>(-1);
			const UINT value = ::GetPrivateProfileIntW(group, name, ILLEGAL_VALUE, projectSettingsPath.c_str());
			if (value != ILLEGAL_VALUE)
			{
				++g_projectSettingsLoaded;

				// a value was found, use this one
				return value;
			}
		}

		++g_userSettingsLoaded;

		// either the value was not found, or the project settings file does not exist.
		// load the value from the user settings file instead.
		const std::wstring& userSettingsPath = appSettings::GetUserSettingsPath();
		const UINT value = ::GetPrivateProfileIntW(group, name, initialValue, userSettingsPath.c_str());

		return value;
#endif
		// END EPIC MOD
	}


	static std::wstring LoadSetting(const wchar_t* group, const wchar_t* name, const wchar_t* initialValue)
	{
		// BEGIN EPIC MOD - Just use default Epic settings
#if 1
		return initialValue;
#else
		// first try loading the setting from the project settings file
		const std::wstring& projectSettingsPath = appSettings::GetProjectSettingsPath();
		const Filesystem::PathAttributes& attributes = Filesystem::GetAttributes(projectSettingsPath.c_str());
		if (Filesystem::DoesExist(attributes))
		{
			// a file is there, so try loading the setting
			const wchar_t* ILLEGAL_VALUE = L"__ILLEGAL_STRING__";

			wchar_t value[MAX_PATH] = {};
			::GetPrivateProfileStringW(group, name, ILLEGAL_VALUE, value, MAX_PATH, projectSettingsPath.c_str());
			if (!string::Matches(value, ILLEGAL_VALUE))
			{
				++g_projectSettingsLoaded;

				// a value was found, use this one
				return std::wstring(value);
			}
		}

		++g_userSettingsLoaded;

		// either the value was not found, or the project settings file does not exist.
		// load the value from the user settings file instead.
		const std::wstring& userSettingsPath = appSettings::GetUserSettingsPath();
		wchar_t value[MAX_PATH] = {};
		::GetPrivateProfileStringW(group, name, initialValue, value, MAX_PATH, userSettingsPath.c_str());

		return std::wstring(value);
#endif
		// END EPIC MOD
	}
}


unsigned int settings::GetLoadedUserSettingsCount(void)
{
	return g_userSettingsLoaded;
}


unsigned int settings::GetLoadedProjectSettingsCount(void)
{
	return g_projectSettingsLoaded;
}


Setting::Setting(Type::Enum type)
	: m_type(type)
{
}


Setting::Type::Enum Setting::GetType(void) const
{
	return m_type;
}


SettingBool::SettingBool(const wchar_t* group, const wchar_t* name, const wchar_t* shortDescription, const wchar_t* description, bool initialValue)
	: Setting(Type::BOOLEAN)
	, m_group(group)
	, m_name(name)
	, m_shortDescription(shortDescription)
	, m_description(description)
	, m_value(initialValue)
	, m_previousValue(initialValue)
	, m_initialValue(initialValue)
{
	const UINT value = LoadSetting(m_group, m_name, initialValue);

	// set value so that non-existent values will immediately be saved to the .ini file
	SetValue(value != 0u);

	// set once now that the value is loaded, never changed afterwards
	m_initialValue = m_value;
}


void SettingBool::SetValue(bool value)
{
	SetValueWithoutSaving(value);

	// store setting in user settings file
	const std::wstring& iniPath = appSettings::GetUserSettingsPath();
	::WritePrivateProfileStringW(m_group, m_name, std::to_wstring(m_value).c_str(), iniPath.c_str());
}


void SettingBool::SetValueWithoutSaving(bool value)
{
	m_previousValue = m_value;
	m_value = value;
}


const wchar_t* SettingBool::GetName(void) const
{
	return m_name;
}


const wchar_t* SettingBool::GetShortDescription(void) const
{
	return m_shortDescription;
}


const wchar_t* SettingBool::GetDescription(void) const
{
	return m_description;
}


bool SettingBool::GetValue(void) const
{
	return m_value;
}


bool SettingBool::GetPreviousValue(void) const
{
	return m_previousValue;
}


bool SettingBool::GetInitialValue(void) const
{
	return m_initialValue;
}


SettingInt::SettingInt(const wchar_t* group, const wchar_t* name, const wchar_t* shortDescription, const wchar_t* description, int initialValue)
	: Setting(Type::INTEGER)
	, m_group(group)
	, m_name(name)
	, m_shortDescription(shortDescription)
	, m_description(description)
	, m_value(initialValue)
	, m_previousValue(initialValue)
	, m_initialValue(initialValue)
{
	const UINT value = LoadSetting(m_group, m_name, initialValue);

	// set value so that non-existent values will immediately be saved to the .ini file
	SetValue(static_cast<int>(value));

	// set once now that the value is loaded, never changed afterwards
	m_initialValue = m_value;
}


void SettingInt::SetValue(int value)
{
	SetValueWithoutSaving(value);

	// store setting in user settings file
	const std::wstring& iniPath = appSettings::GetUserSettingsPath();
	::WritePrivateProfileStringW(m_group, m_name, std::to_wstring(m_value).c_str(), iniPath.c_str());
}


void SettingInt::SetValueWithoutSaving(int value)
{
	m_previousValue = m_value;
	m_value = value;
}


const wchar_t* SettingInt::GetName(void) const
{
	return m_name;
}


const wchar_t* SettingInt::GetShortDescription(void) const
{
	return m_shortDescription;
}


const wchar_t* SettingInt::GetDescription(void) const
{
	return m_description;
}


int SettingInt::GetValue(void) const
{
	return m_value;
}


int SettingInt::GetPreviousValue(void) const
{
	return m_previousValue;
}


int SettingInt::GetInitialValue(void) const
{
	return m_initialValue;
}


SettingString::SettingString(const wchar_t* group, const wchar_t* name, const wchar_t* shortDescription, const wchar_t* description, const wchar_t* initialValue)
	: Setting(Type::STRING)
	, m_group(group)
	, m_name(name)
	, m_shortDescription(shortDescription)
	, m_description(description)
	, m_value(initialValue)
	, m_previousValue(initialValue)
{
	const std::wstring value = LoadSetting(m_group, m_name, initialValue);

	// set value so that non-existent values will immediately be saved to the .ini file
	SetValue(value.c_str());
}


void SettingString::SetValue(const wchar_t* value)
{
	SetValueWithoutSaving(value);

	// store setting in user settings file
	const std::wstring& iniPath = appSettings::GetUserSettingsPath();
	::WritePrivateProfileStringW(m_group, m_name, value, iniPath.c_str());
}


void SettingString::SetValueWithoutSaving(const wchar_t* value)
{
	m_previousValue = m_value;
	m_value = value;
}


const wchar_t* SettingString::GetName(void) const
{
	return m_name;
}


const wchar_t* SettingString::GetShortDescription(void) const
{
	return m_shortDescription;
}


const wchar_t* SettingString::GetDescription(void) const
{
	return m_description;
}


const wchar_t* SettingString::GetValue(void) const
{
	return m_value.c_str();
}


const wchar_t* SettingString::GetPreviousValue(void) const
{
	return m_previousValue.c_str();
}


SettingIntProxy::SettingIntProxy(SettingInt* setting)
	: Setting(Type::INTEGER_PROXY)
	, m_setting(setting)
{
}


SettingIntProxy& SettingIntProxy::AddMapping(const wchar_t* str, int value)
{
	m_mappings.emplace_back(Mapping { str, value });
	return *this;
}


const wchar_t* SettingIntProxy::MapIntToString(int value) const
{
	const size_t count = m_mappings.size();
	for (size_t i = 0u; i < count; ++i)
	{
		if (m_mappings[i].value == value)
		{
			return m_mappings[i].str.c_str();
		}
	}

	return L"";
}


int SettingIntProxy::MapStringToInt(const wchar_t* str) const
{
	const size_t count = m_mappings.size();
	for (size_t i = 0u; i < count; ++i)
	{
		if (wcscmp(m_mappings[i].str.c_str(), str) == 0)
		{
			return m_mappings[i].value;
		}
	}

	return -1;
}


SettingInt* SettingIntProxy::GetSetting(void)
{
	return m_setting;
}


size_t SettingIntProxy::GetMappingCount(void) const
{
	return m_mappings.size();
}


const wchar_t* SettingIntProxy::GetMappingString(size_t index) const
{
	return m_mappings[index].str.c_str();
}


int SettingIntProxy::GetMappingInt(size_t index) const
{
	return m_mappings[index].value;
}


SettingShortcut::SettingShortcut(const wchar_t* group, const wchar_t* name, const wchar_t* shortDescription, const wchar_t* description, int initialValue)
	: Setting(Type::SHORTCUT)
	, m_group(group)
	, m_name(name)
	, m_shortDescription(shortDescription)
	, m_description(description)
	, m_value(initialValue)
{
	const UINT value = LoadSetting(m_group, m_name, initialValue);

	// set value so that non-existent values will immediately be saved to the .ini file
	SetValue(static_cast<int>(value));
}


void SettingShortcut::SetValue(int value)
{
	SetValueWithoutSaving(value);

	// store setting in user settings file
	const std::wstring& iniPath = appSettings::GetUserSettingsPath();
	::WritePrivateProfileStringW(m_group, m_name, std::to_wstring(m_value).c_str(), iniPath.c_str());
}


void SettingShortcut::SetValueWithoutSaving(int value)
{
	m_value = value;
}


const wchar_t* SettingShortcut::GetName(void) const
{
	return m_name;
}


const wchar_t* SettingShortcut::GetShortDescription(void) const
{
	return m_shortDescription;
}


const wchar_t* SettingShortcut::GetDescription(void) const
{
	return m_description;
}


int SettingShortcut::GetValue(void) const
{
	return m_value;
}


#endif