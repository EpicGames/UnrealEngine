// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "LC_Types.h"
#include "LC_Platform.h"
// END EPIC MOD

namespace settings
{
	unsigned int GetLoadedUserSettingsCount(void);
	unsigned int GetLoadedProjectSettingsCount(void);
}


// base
class Setting
{
public:
	struct Type
	{
		enum Enum
		{
			BOOLEAN,
			INTEGER,
			INTEGER_PROXY,
			STRING,
			SHORTCUT
		};
	};

	explicit Setting(Type::Enum type);
	LC_DISABLE_COPY(Setting);
	LC_DISABLE_ASSIGNMENT(Setting);

	Type::Enum GetType(void) const;

private:
	const Type::Enum m_type;
};


// real settings
class SettingBool : public Setting
{
public:
	SettingBool(const wchar_t* group, const wchar_t* name, const wchar_t* shortDescription, const wchar_t* description, bool initialValue);
	LC_DISABLE_COPY(SettingBool);
	LC_DISABLE_ASSIGNMENT(SettingBool);

	void SetValue(bool value);
	void SetValueWithoutSaving(bool value);

	const wchar_t* GetName(void) const;
	const wchar_t* GetShortDescription(void) const;
	const wchar_t* GetDescription(void) const;
	bool GetValue(void) const;
	bool GetPreviousValue(void) const;
	bool GetInitialValue(void) const;

private:
	const wchar_t* m_group;
	const wchar_t* m_name;
	const wchar_t* m_shortDescription;
	const wchar_t* m_description;
	bool m_value;
	bool m_previousValue;
	bool m_initialValue;
};


class SettingInt : public Setting
{
public:
	SettingInt(const wchar_t* group, const wchar_t* name, const wchar_t* shortDescription, const wchar_t* description, int initialValue);
	LC_DISABLE_COPY(SettingInt);
	LC_DISABLE_ASSIGNMENT(SettingInt);

	void SetValue(int value);
	void SetValueWithoutSaving(int value);

	const wchar_t* GetName(void) const;
	const wchar_t* GetShortDescription(void) const;
	const wchar_t* GetDescription(void) const;
	int GetValue(void) const;
	int GetPreviousValue(void) const;
	int GetInitialValue(void) const;

private:
	const wchar_t* m_group;
	const wchar_t* m_name;
	const wchar_t* m_shortDescription;
	const wchar_t* m_description;
	int m_value;
	int m_previousValue;
	int m_initialValue;
};


class SettingString : public Setting
{
public:
	SettingString(const wchar_t* group, const wchar_t* name, const wchar_t* shortDescription, const wchar_t* description, const wchar_t* initialValue);
	LC_DISABLE_COPY(SettingString);
	LC_DISABLE_ASSIGNMENT(SettingString);

	void SetValue(const wchar_t* value);
	void SetValueWithoutSaving(const wchar_t* value);

	const wchar_t* GetName(void) const;
	const wchar_t* GetShortDescription(void) const;
	const wchar_t* GetDescription(void) const;
	const wchar_t* GetValue(void) const;
	const wchar_t* GetPreviousValue(void) const;

private:
	const wchar_t* m_group;
	const wchar_t* m_name;
	const wchar_t* m_shortDescription;
	const wchar_t* m_description;
	std::wstring m_value;
	std::wstring m_previousValue;
};


// proxy settings
class SettingIntProxy : public Setting
{
	struct Mapping
	{
		std::wstring str;
		int value;
	};

public:
	explicit SettingIntProxy(SettingInt* setting);
	LC_DISABLE_COPY(SettingIntProxy);
	LC_DISABLE_ASSIGNMENT(SettingIntProxy);

	SettingIntProxy& AddMapping(const wchar_t* str, int value);
	const wchar_t* MapIntToString(int value) const;
	int MapStringToInt(const wchar_t* str) const;

	SettingInt* GetSetting(void);

	size_t GetMappingCount(void) const;
	const wchar_t* GetMappingString(size_t index) const;
	int GetMappingInt(size_t index) const;

private:
	SettingInt* m_setting;
	types::vector<Mapping> m_mappings;
};


class SettingShortcut : public Setting
{
public:
	SettingShortcut(const wchar_t* group, const wchar_t* name, const wchar_t* shortDescription, const wchar_t* description, int initialValue);
	LC_DISABLE_COPY(SettingShortcut);
	LC_DISABLE_ASSIGNMENT(SettingShortcut);

	void SetValue(int value);
	void SetValueWithoutSaving(int value);

	const wchar_t* GetName(void) const;
	const wchar_t* GetShortDescription(void) const;
	const wchar_t* GetDescription(void) const;
	int GetValue(void) const;

private:
	const wchar_t* m_group;
	const wchar_t* m_name;
	const wchar_t* m_shortDescription;
	const wchar_t* m_description;
	int m_value;
};


#endif