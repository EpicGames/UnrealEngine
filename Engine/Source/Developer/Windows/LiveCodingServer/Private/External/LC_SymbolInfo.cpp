// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_SymbolInfo.h"
// BEGIN EPIC MOD
#include <stdio.h>
// END EPIC MOD

SymbolInfo::SymbolInfo(const char* const function, const char* const filename, unsigned int line)
	: m_function()
	, m_filename()
	, m_line(line)
{
	strcpy_s(m_function, function);
	strcpy_s(m_filename, filename);
}


#endif