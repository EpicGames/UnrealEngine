// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "LC_Platform.h"
#include "LC_Foundation_Windows.h"
// END EPIC MOD

class SymbolInfo
{
public:
	SymbolInfo(const char* const function, const char* const filename, unsigned int line);

	inline const char* GetFunction(void) const
	{
		return m_function;
	}

	inline const char* GetFilename(void) const
	{
		return m_filename;
	}

	inline unsigned int GetLine(void) const
	{
		return m_line;
	}

private:
	LC_DISABLE_ASSIGNMENT(SymbolInfo);

	char m_function[512u];
	// BEGIN EPIC MOD
	char m_filename[WINDOWS_MAX_PATH];
	// END EPIC MOD
	unsigned int m_line;
};


#endif