// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_SymbolPatterns.h"


namespace symbolPatterns
{
	// highly compiler-specific
	// https://en.wikiversity.org/wiki/Visual_C%2B%2B_name_mangling
	extern const char* const PCH_SYMBOL_PATTERNS[1] =
	{
		// in newer versions of Visual Studio, translation units using a precompiled header file will
		// emit a corresponding directive to make the linker force-include the PCH's symbol, e.g.
		// -INCLUDE:___@@_PchSym_@00@UwvevolknvmgUkilqvxghUorevxlwvUxlwvUgvnkUdrmDCUwvyftUvcvwbmznrxifmgrnvUkxsOlyq@FC6294CA356B5C81
		"@_PchSym_@"
	};

	extern const char* const VTABLE_PATTERNS[3] =
	{
		// in undecorated form, "`vftable'" denotes a virtual function table.
		// in decorated form, this is denoted by "??_7".
		"??_7",

		// additionally, there is a thing known as a local virtual function table or "local vftable".
		// see https://groups.google.com/forum/#!msg/microsoft.public.vc.language/atSh_2VSc2w/EgJ3r_7OzVUJ
		// this is denoted by "??_S" in decorated form.
		"??_S",

		// in undecorated form, "`vbtable'" denotes a virtual base class table, used with multiple virtual inheritance.
		// in decorated form, this is denoted by "??_8".
		"??_8"
	};

	extern const char* const RTTI_OBJECT_LOCATOR_PATTERNS[1] =
	{
		// in undecorated form, "const Foo::`RTTI Complete Object Locator'" denotes an RTTI object locator.
		// in decorated form, this is denoted by "??_R4".
		"??_R4"
	};

	extern const char* const DYNAMIC_INITIALIZER_PATTERNS[1] =
	{
		// a dynamic initializer is a piece of code for constructing e.g. static/global instances.
		// in its relocations, it mostly refers to global/static data (the thing being constructed) and constructor(s).

		// in undecorated form, "`dynamic initializer'" denotes a dynamic initializer used for constructing global instances.
		// in decorated form, this is denoted by "??__E".
		"??__E"
	};

	extern const char* const DYNAMIC_ATEXIT_DESTRUCTORS[1] =
	{
		// a dynamic atexit destructor is a piece of code for destructing e.g. static/global instances.
		// in its relocations, it mostly refers to global/static data (the thing being destructed) and destructor(s).

		// in undecorated form, "`dynamic atexit destructor'" denotes a dynamic atexit destructor used for destructing global instances.
		// in decorated form, this is denoted by "??__F".
		"??__F"
	};

	extern const char* const POINTER_TO_DYNAMIC_INITIALIZER_PATTERNS[1] =
	{
		// pointers to dynamic initializers always have $initializer$ in their name and reside in the .CRT$XCU section.
		// in its relocations, it only refers to dynamic initializers.
		"$initializer$"
	};

	extern const char* const STRING_LITERAL_PATTERNS[2] =
	{
		// in decorated form, a string literal is denoted by "??_C@_".
		// in COFF files, string literals are sometimes named "$SG", depending on compiler settings.
		"??_C@_",
		"$SG"
	};

	extern const char* const LINE_NUMBER_PATTERNS[1] =
	{
		// line numbers are named "$LN????", e.g. "$LN11"
		"$LN"
	};

	extern const char* const FLOATING_POINT_CONSTANT_PATTERNS[4] =
	{
		// NOTE: both 32-bit and 64-bit constants have the same mangled name (two leading underscores)

		// compiler-specific, floating-point values
		"__real@",

		// compiler-specific, __m128 (SSE <-> SSE 4.2)
		"__xmm@",

		// compiler-specific, __m256 (AVX)
		"__ymm@",

		// compiler-specific, __m512 (AVX512)
		"__zmm@"
	};

#if LC_64_BIT
	extern const char* const EXCEPTION_RELATED_PATTERNS[17] =
#else
	extern const char* const EXCEPTION_RELATED_PATTERNS[11] =
#endif
	{
		// used for C++ exception handling
		// http://www.openrce.org/articles/full_view/21

		// function symbols
#if LC_64_BIT
		"?dtor$",
		"?catch$",
		"?fin$",
		"?filt$",
		"__catch$",	// CL
		"?catch$",	// Clang
		"_CxxThrowException",
		"__CxxFrameHandler",
		"__GSHandlerCheck",
#else
		"__ehhandler$",
		"__unwindfunclet$",
		"__catch$",	// CL
		"?catch$",	// Clang
		"__except_handler3",
		"__except_handler4",
#endif

		// data symbols
#if LC_64_BIT
		"$unwind$",
		"$chain$",
		"$pdata$",
		"$cppxdata$",
		"$stateUnwindMap$",
		"$tryMap$",
		"$handlerMap$",
		"$ip2state$"
#else
		"__ehfuncinfo$",
		"__catchsym$",
		"__unwindtable$",
		"__tryblocktable$",
		"__sehtable$"
#endif
	};

	extern const char* const EXCEPTION_UNWIND_PATTERNS[1] =
	{
#if LC_64_BIT
		"?dtor$",
#else
		"__unwindfunclet$",
#endif
	};

	extern const char* const EXCEPTION_CLAUSE_PATTERNS[2] =
	{
		"__catch$",	// CL
		"?catch$",	// Clang
	};

	extern const char* const RTC_PATTERNS[8] =
	{
		"@_RTC_Check",			// @_RTC_Check_4_to_1@4 and @_RTC_CheckStackVars@8
		LC_IDENTIFIER("_RTC_CheckEsp"),
		LC_IDENTIFIER("_RTC_InitBase"),
		LC_IDENTIFIER("_RTC_Shutdown"),
		".rtc$",				// _RTC_InitBase.rtc$ and _RTC_Shutdown.rtc$ and _RTC_CheckStackVars.rtc$
		"$rtcName$",			// 64-bit runtime-check data, referenced by frame data, read-only
		"$rtcVarDesc",			// 64-bit runtime-check data, referenced by frame data, read-only
		"$rtcFrameData"			// 64-bit runtime-check data, read-only
	};

	extern const char* const SDL_CHECK_PATTERNS[2] =
	{
		LC_IDENTIFIER("__security_cookie"),
		"__security_check_cookie"
	};

	extern const char* const CFG_PATTERNS[1] =
	{
		// NOTE: both 32-bit and 64-bit constants have the same mangled name (two leading underscores)
		"__guard_fids"		// control flow guard function identifiers
	};

	extern const char* const IMAGE_BASE_PATTERNS[1] =
	{
		// NOTE: both 32-bit and 64-bit constants have the same mangled name (two leading underscores)
		"__ImageBase"
	};

	extern const char* const TLS_ARRAY_PATTERNS[1] =
	{
		// 64-bit: a hard-coded placeholder for gs:0x58, often not even emitted as symbol
		// 32-bit: a hard-coded placeholder for fs:0x2C
		LC_IDENTIFIER("_tls_array")
	};

	extern const char* const TLS_INDEX_PATTERNS[1] =
	{
		LC_IDENTIFIER("_tls_index")
	};
	
	extern const char* const TLS_INIT_PATTERNS[3] =
	{
		LC_IDENTIFIER("_Init_thread_epoch"),
		LC_IDENTIFIER("_Init_thread_header"),
		LC_IDENTIFIER("_Init_thread_footer")
	};

	extern const char* const TLS_STATICS_PATTERNS[1] =
	{
		"?$TSS"
	};

	extern const char* const ANONYMOUS_NAMESPACE_PATTERN =
	{
		"@?A0x"
	};

	// BEGIN EPIC MOD
	extern const char* const UE_STATICS_BLOCK_PATTERNS[7] =
	{
		"*Z_Construct_UClass_*_Statics*",
		"*Z_Construct_UEnum_*_Statics*",
		"*Z_Construct_UFunction_*_Statics*",
		"*Z_Construct_UDelegateFunction_*_Statics*",
		"*Z_Construct_UScriptStruct_*_Statics*",
		"*Z_Construct_UPackage_*_Statics*",
		"*Z_CompiledInDeferFile_*_Statics*",
	};

	extern const char* const UE_REGISTER_PATTERNS[6] =
	{
		"?AutoInitialize",
		"?Z_CompiledInDeferEnum_UEnum_",
		"?Z_CompiledInDeferStruct_UScriptStruct_",
		"?Z_CompiledInDeferCppStructOps_UScriptStruct_",
		"?Z_CompiledInDeferPackage_UPackage_",
		"?Z_CompiledInDeferFile_",
	};
	// END EPIC MOD
}


#endif