// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
#include "LC_Platform.h"
// END EPIC MOD

namespace symbolPatterns
{
	extern const char* const PCH_SYMBOL_PATTERNS[1];

	extern const char* const VTABLE_PATTERNS[3];
	extern const char* const RTTI_OBJECT_LOCATOR_PATTERNS[1];
	extern const char* const DYNAMIC_INITIALIZER_PATTERNS[1];
	extern const char* const DYNAMIC_ATEXIT_DESTRUCTORS[1];
	extern const char* const POINTER_TO_DYNAMIC_INITIALIZER_PATTERNS[1];
	extern const char* const STRING_LITERAL_PATTERNS[2];
	extern const char* const LINE_NUMBER_PATTERNS[1];
	extern const char* const FLOATING_POINT_CONSTANT_PATTERNS[4];

#if LC_64_BIT
	extern const char* const EXCEPTION_RELATED_PATTERNS[17];
#else
	extern const char* const EXCEPTION_RELATED_PATTERNS[11];
#endif
	extern const char* const EXCEPTION_UNWIND_PATTERNS[1];

	extern const char* const EXCEPTION_CLAUSE_PATTERNS[2];

	extern const char* const RTC_PATTERNS[8];
	extern const char* const SDL_CHECK_PATTERNS[2];
	extern const char* const CFG_PATTERNS[1];

	extern const char* const IMAGE_BASE_PATTERNS[1];

	extern const char* const TLS_ARRAY_PATTERNS[1];
	extern const char* const TLS_INDEX_PATTERNS[1];
	extern const char* const TLS_INIT_PATTERNS[3];
	extern const char* const TLS_STATICS_PATTERNS[1];

	extern const char* const ANONYMOUS_NAMESPACE_PATTERN;

	// BEGIN EPIC MOD
	extern const char* const UE_STATICS_BLOCK_PATTERNS[7];
	extern const char* const UE_REGISTER_PATTERNS[6];
	// END EPIC MOD
}


#endif