// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_SymbolReconstruction.h"

#include "LC_StringUtil.h"
#include "LC_PointerUtil.h"
#include "LC_NameMangling.h"
#include "LC_DiaUtil.h"


namespace
{
	static inline bool HasLowerRVA(const symbols::Contribution* lhs, uint32_t rva)
	{
		return lhs->rva < rva;
	}
}


void symbols::ReconstructFromExecutableCoff
(
	const symbols::Provider* provider,
	const executable::Image* image,
	const executable::ImageSectionDB* imageSections,
	const coff::CoffDB* coffDb,
	const types::StringSet& strippedSymbols, 
	const symbols::ObjPath& objPath,
	const symbols::ContributionDB* contributionDb,
	const symbols::ThunkDB* thunkDb,
	const symbols::ImageSectionDB* imageSectionDb,
	symbols::SymbolDB* symbolDB
)
{
	const executable::PreferredBase imageBase = executable::GetPreferredBase(image);
	const uint32_t imageSize = executable::GetSize(image);

	LC_LOG_DEV("Gathering symbols from COFF file %s", objPath.c_str());
	LC_LOG_INDENT_DEV;

	LC_LOG_DEV("Symbols in COFF: %d", coffDb->symbols.size());
	LC_LOG_DEV("Symbols stripped: %d", strippedSymbols.size());

	// gather symbols by following relocation "paths", backtracking from the location in the executable
	// to the symbol's origin RVA. our starting entry paths are the functions and data of which we already
	// know the name and RVA.
	size_t unknownSymbolsToFind = 0u;
	types::vector<const coff::Symbol*> openSymbols;
	openSymbols.reserve(coffDb->symbols.size());
	{
		const size_t count = coff::GetIndexCount(coffDb);
		for (size_t i = 0u; i < count; ++i)
		{
			// do we have a symbol at that index?
			const coff::Symbol* symbol = coff::GetSymbolByIndex(coffDb, i);
			if (symbol)
			{
				// yes, so check whether this symbol is known already
				const ImmutableString& symbolName = coff::GetSymbolName(coffDb, symbol);
				const symbols::Symbol* srcSymbol = symbols::FindSymbolByName(symbolDB, symbolName);
				if (srcSymbol)
				{
					LC_LOG_DEV("Known symbol %s at 0x%X", symbolName.c_str(), srcSymbol->rva);
					openSymbols.push_back(symbol);
				}
				else if (strippedSymbols.find(symbolName) != strippedSymbols.end())
				{
					LC_LOG_DEV("Stripped symbol %s", symbolName.c_str());
				}
				else
				{
					const coff::Section& coffSection = coffDb->sections[symbol->sectionIndex];
					if (coff::IsMSVCJustMyCodeSection(coffSection.name.c_str()))
					{
						LC_LOG_DEV("JustMyCode symbol %s", symbolName.c_str());
					}
					else
					{
						LC_LOG_DEV("Unknown symbol %s", symbolName.c_str());
						++unknownSymbolsToFind;
					}
				}
			}
			else
			{
				// we do not have a symbol stored in the COFF, because it might be external/unresolved.
				// if so, chances are very high that this symbol is already known publicly.
				const ImmutableString& symbolName = coff::GetUnresolvedSymbolName(coffDb, i);
				if (symbolName.GetLength() == 0u)
				{
					continue;
				}

				if (symbols::FindSymbolByName(symbolDB, symbolName))
				{
					LC_LOG_DEV("Publicly known symbol %s", symbolName.c_str());
				}
				else if (!coff::IsInterestingSymbol(symbolName))
				{
					// relocations to those symbols are not stored in the COFF, hence we
					// can not reconstruct these anyway
					LC_LOG_DEV("Non-interesting symbol %s", symbolName.c_str());
				}
				else if (symbols::IsImageBaseRelatedSymbol(symbolName))
				{
					LC_LOG_DEV("Linker-generated image base symbol %s", symbolName.c_str());
				}
				else if (symbols::IsTlsArrayRelatedSymbol(symbolName))
				{
					LC_LOG_DEV("Compiler-generated symbol %s", symbolName.c_str());
				}
				else if (symbols::IsSectionSymbol(symbolName))
				{
					LC_LOG_DEV("Section symbol %s", symbolName.c_str());
				}
				else if (strippedSymbols.find(symbolName) != strippedSymbols.end())
				{
					LC_LOG_DEV("Stripped symbol %s", symbolName.c_str());
				}
				else if (symbolDB->symbolsWithoutRva.find(symbolName) != symbolDB->symbolsWithoutRva.end())
				{
					// ignore symbols without an RVA. those are often generated by the compiler or linker,
					// are being relocated to, but store absolute values encoded in their offset in the PDB.
					LC_LOG_DEV("Compiler- or linker-generated symbol %s without an RVA", symbolName.c_str());
				}
				else if (string::Contains(symbolName.c_str(), "$__resumable"))
				{
					LC_LOG_DEV("Coroutine symbol %s", symbolName.c_str());
				}
				else
				{
					LC_LOG_DEV("Unknown unresolved symbol %s", symbolName.c_str());
					++unknownSymbolsToFind;
				}
			}
		}
	}

	LC_LOG_DEV("Unknown symbols left to find: %d", unknownSymbolsToFind);

	// do we already know all symbols?
	if (unknownSymbolsToFind == 0u)
	{
		LC_LOG_DEV("Know all symbols already, nothing to do");
		return;
	}

	// keep walking relocations of all open symbols to determine the RVA of symbols contained in this .obj
	types::unordered_set<const coff::Symbol*> walkedAlready;
	types::unordered_set<const coff::Symbol*> triedReconstructingAlready;

	unsigned int pass = 0u;

walkOpenSymbols:
	while (openSymbols.size() > 0u)
	{
		const coff::Symbol* symbol = openSymbols.back();
		openSymbols.pop_back();

		// check whether we walked this symbol already
		const auto it = walkedAlready.find(symbol);
		if (it != walkedAlready.end())
		{
			// handled already, nothing more to do
			continue;
		}

		// check whether the symbol is actually the one that contributed its code.
		// in case of COMDATs available in both executable and static libraries, this might not
		// be true and would lead to completely wrong symbols being reconstructed.
		const ImmutableString& srcSymbolName = coff::GetSymbolName(coffDb, symbol);
		const symbols::Symbol* srcSymbol = symbols::FindSymbolByName(symbolDB, srcSymbolName);
		if (srcSymbol)
		{
			const symbols::Contribution* symbolContribution = symbols::FindContributionByRVA(contributionDb, srcSymbol->rva);
			if (symbolContribution)
			{
				const ImmutableString& contributingCompiland = symbols::GetContributionCompilandName(contributionDb, symbolContribution);
				if (contributingCompiland != objPath)
				{
					LC_LOG_DEV("Not walking symbol %s from contribution in different file %s", srcSymbolName.c_str(), contributingCompiland.c_str());
					continue;
				}
			}
		}

		LC_LOG_DEV("Walking relocations of symbol %s", srcSymbolName.c_str());
		LC_LOG_INDENT_DEV;

		const size_t relocationCount = symbol->relocations.size();
		for (size_t i = 0u; i < relocationCount; ++i)
		{
			const coff::Relocation* relocation = symbol->relocations[i];

			// ignore relocations to symbols in .msvcjmc (MSVC JustMyCode) sections
			if (relocation->dstSectionIndex >= 0)
			{
				const uint32_t index = static_cast<uint32_t>(relocation->dstSectionIndex);
				const coff::Section& section = coffDb->sections[index];
				if (coff::IsMSVCJustMyCodeSection(section.name.c_str()))
				{
					LC_LOG_DEV("Ignoring relocation to symbol in section %s", section.name.c_str());
					continue;
				}
			}

			const ImmutableString& dstSymbolName = coff::GetRelocationDstSymbolName(coffDb, relocation);

			// the symbol we are looking for might already be in the database because of the public symbols gathered from the PDB
			if (symbols::FindSymbolByName(symbolDB, dstSymbolName))
			{
				LC_LOG_DEV("Publicly known symbol %s", dstSymbolName.c_str());

				// we know this symbol already, but we might not have walked its relocations yet.
				// add it to the list and continue.
				const coff::Symbol* nextSymbol = coff::GetSymbolByIndex(coffDb, relocation->dstSymbolNameIndex);
				if (nextSymbol)
				{
					openSymbols.push_back(nextSymbol);
				}

				continue;
			}
			else if (strippedSymbols.find(dstSymbolName) != strippedSymbols.end())
			{
				// the relocation points to a symbol we should ignore
				LC_LOG_DEV("Ignoring stripped symbol \"%s\"", dstSymbolName.c_str());
				continue;
			}
			else if (symbols::IsImageBaseRelatedSymbol(dstSymbolName))
			{
				// the linker-generated __ImageBase always sits at RVA zero, and relocations should never be patched
				LC_LOG_DEV("Ignoring destination symbol \"%s\"", dstSymbolName.c_str());
				continue;
			}
			else if (symbols::IsTlsArrayRelatedSymbol(dstSymbolName))
			{
				// compiler-generated symbols such as __tls_array don't have any RVA, because they always reside at
				// the same address, e.g. relative to a segment register.
				// one such example would be how thread-local storage variables are accessed:
				//   the generated code always fetches the flat address of the thread-local storage array from the TEB (https://en.wikipedia.org/wiki/Win32_Thread_Information_Block).
				//   the TEB itself can be accessed using segment register FS on x86, and GS on x64, so one of the first instructions of thread-local storage access is always going to
				//   access the member at 0x2C/0x58 relative to FS/GS, e.g.:
				//     mov eax, dword ptr fs:0x2C (x86)
				//     mov rax, qword ptr gs:0x58 (x64)
				//	 therefore, the "RVA" of __tls_array is 0x2C (x86) or 0x58 (x64).
				// see http://www.nynaeve.net/?p=180 for more in-depth information about thread-local storage on Windows.
				// NOTE: we do need the RVA of __tls_index because that is used to set the data segment register to the
				// table used for accessing TLS variables.
				LC_LOG_DEV("Ignoring destination symbol \"%s\"", dstSymbolName.c_str());
				continue;
			}
			else if (symbols::IsSectionSymbol(dstSymbolName))
			{
				LC_LOG_DEV("Ignoring section symbol \"%s\"", dstSymbolName.c_str());
				continue;
			}
			else if (string::Contains(dstSymbolName.c_str(), "$__resumable"))
			{
				LC_LOG_DEV("Ignoring Coroutine symbol %s", dstSymbolName.c_str());
				continue;
			}

			if (!srcSymbol)
			{
				LC_ERROR_DEV("Cannot find source symbol %s (%s)",
					srcSymbolName.c_str(),
					nameMangling::UndecorateSymbol(srcSymbolName.c_str(), 0u).c_str());
				continue;
			}

			const coff::Relocation::Type::Enum type = relocation->type;

			// the relocation's RVA is relative to the start of the function, and the executable already has all relocations
			// resolved. hence we can backtrack the RVA of the destination symbol by peeking into the executable's code
			// at the address of the relocation.
			const uint32_t relocationRva = srcSymbol->rva + relocation->srcRva;

			// check for invalid RVAs before trying to reconstruct the symbol.
			// these can occur when a COMDAT gets stripped in an .obj, but is needed by an .obj coming from a library.
			// the COMDAT will then be stripped from the executable, so we shouldn't try reconstructing it.			
			{
#if LC_64_BIT
				if (type == coff::Relocation::Type::VA_64)
				{
					const uint64_t rvaInCode = executable::ReadFromImage<uint64_t>(image, imageSections, relocationRva);
					if (rvaInCode == 0u)
					{
						continue;
					}
				}
				else
#endif
				{
					const uint32_t rvaInCode = executable::ReadFromImage<uint32_t>(image, imageSections, relocationRva);
					if (rvaInCode == 0u)
					{
						continue;
					}
				}
			}

			// even though the final RVA can only be 32-bit because no image can ever be larger than 4GB, intermediate results
			// can point to addresses in the full 64-bit address space.

#if LC_64_BIT
			uint64_t dstRva = 0u;
#else
			uint32_t dstRva = 0u;
#endif

			// backtrack to the real RVA of the destination symbol depending on the type of relocation.
			// 32-BIT NOTE: relative addresses are signed 32-bit offsets, but addressing performed by the CPU
			// works modulo 2^32. this means that it doesn't matter whether we go forward 3GB, or back 1GB - 
			// the resulting address will be the same.
			// we therefore carry out all calculations using *unsigned* 32-bit integers, because they have
			// natural overflow/underflow behaviour, and do *not* invoke undefined behaviour like signed integers.
			switch (type)
			{
				case coff::Relocation::Type::RELATIVE:

#if LC_64_BIT
				case coff::Relocation::Type::RELATIVE_OFFSET_1:
				case coff::Relocation::Type::RELATIVE_OFFSET_2:
				case coff::Relocation::Type::RELATIVE_OFFSET_3:
				case coff::Relocation::Type::RELATIVE_OFFSET_4:
				case coff::Relocation::Type::RELATIVE_OFFSET_5:
#endif
				{
					// relative relocations are used for e.g. JMP and CALL instructions and are relative to the address
					// of the next instruction.
					// example:
					//   00015DAA E8 1E B8 FF FF       call        _printf(0115CDh)
					// the CALL instruction sits at 0x00015DAA and calls printf at 0x0115CD, but this is *not* the address
					// encoded in the CALL instruction. the encoded relative address is 0xFFFFB81E, which is -18402.
					// adding 0xFFFFB81E to 0x00015DAA + 5 (the address of the next instruction!) yields 0x0115CD.
					// NOTE: the relocation points to the address of the *relocation*, not the beginning of
					// the *instruction* (hence we add 4, not 5).

					const uint32_t rva = executable::ReadFromImage<uint32_t>(image, imageSections, relocationRva);
					dstRva = relocationRva + rva + 4ull + coff::Relocation::Type::GetByteDistance(type);
				}
				break;

				case coff::Relocation::Type::SECTION_RELATIVE:
				{
					// section-relative relocations are used for thread-local storage, e.g. accessing __declspec(thread)
					// variables.
					// example:
					//	00016845 A1 14 35 02 00       mov         eax, dword ptr[_tls_index(023514h)]
					//	0001684A 64 8B 0D 2C 00 00 00 mov         ecx, dword ptr fs:[2Ch]
					//	00016851 8B 14 81             mov         edx, dword ptr[ecx + eax*4]
					//	00016854 8B 82 04 01 00 00    mov         eax, dword ptr[edx + 104h]
					// the code accesses a global variable in thread-local storage, which happens relative to the
					// .tls section. the section-relative offset of the variable in question is 0x104, and the relocation
					// directly stores this offset (0x00000104 in the last line).

					// grab RVA of the symbol's section
					const ImmutableString& sectionName = coff::GetTlsSectionName();
					const symbols::ImageSection* section = symbols::FindImageSectionByName(imageSectionDb, sectionName);
					if (!section)
					{
						LC_ERROR_DEV("Cannot find section %s in image", sectionName.c_str());
						continue;
					}

					// the relocation itself is 32-bit, always positive
					dstRva = executable::ReadFromImage<uint32_t>(image, imageSections, relocationRva) + section->rva;
				}
				break;

				case coff::Relocation::Type::VA_32:
				{
#if LC_64_BIT
					// an absolute 32-bit virtual address cannot exist in a 64-bit image, otherwise the .exe/.dll could
					// not be loaded into the upper 32-bits of the address space.
					LC_ERROR_DEV("Ignoring relocation of type %s (%d)", coff::Relocation::Type::ToString(type), type);
					continue;
#else
					// direct virtual addresses are used for accessing e.g. global symbols, string literals.
					// the instruction directly stores the absolute address of the symbol in question.
					// example:
					//	00015DA5 68 9C 11 02 00       push        2119Ch
					// this pushes the absolute address of a string literal to the stack. the address encoded
					// in the opcode is 0x0002119C, which is the direct address of the string literal in memory.
					dstRva = executable::ReadFromImage<uint32_t>(image, imageSections, relocationRva) - imageBase;
#endif
				}
				break;

				case coff::Relocation::Type::RVA_32:
				{
					// in 32-bit, this type of relocation is only used for .debug and .rsrc (resource) sections.
					// the latter are only needed by the linker in order to know where to place resources in the executable.

					// in 64-bit, this type of relocation is used for addressing exception-relevant functions and data,
					// and seldomly for accessing data at an absolute offset to the image base, e.g.
					//   mov rcx,qword ptr [r8+rcx*8+1771060h]
					// r8 stores the image base, 1771060h is the value of the RVA_32 relocation.
					dstRva = executable::ReadFromImage<uint32_t>(image, imageSections, relocationRva);
				}
				break;

#if LC_64_BIT
				case coff::Relocation::Type::VA_64:
				{
					// direct virtual addresses are used for accessing e.g. global symbols, same as on 32-bit
					dstRva = executable::ReadFromImage<uint64_t>(image, imageSections, relocationRva) - imageBase;
				}
				break;
#endif

				case coff::Relocation::Type::UNKNOWN:
				default:
					LC_ERROR_DEV("Unknown relocation type %s (%d)", coff::Relocation::Type::ToString(type), type);
					break;
			}

			// the original relocation might have been applied to the symbol at a certain offset.
			// subtract that offset (if any) to arrive at the symbol's original RVA.
			dstRva -= relocation->dstOffset;

			if (dstRva == 0u)
			{
				// this was reconstructed from a stripped COMDAT symbol that is referenced by an .obj where it
				// wasn't stripped (e.g. an .obj contained in a .lib).
				continue;
			}

			if (dstRva > imageSize)
			{
				// the RVA underflowed somewhere (the unsigned int would then surely be larger than 2 GB),
				// or the RVA lies outside the module.
				LC_ERROR_DEV("Detected wrong RVA 0x%X: Relocation %s (%d) from %s to %s in file %s", 
					dstRva,
					coff::Relocation::Type::ToString(type), type,
					srcSymbolName.c_str(), dstSymbolName.c_str(), objPath.c_str());
				LC_ERROR_DEV("Source symbol at 0x%X", srcSymbol->rva);
				LC_ERROR_DEV("Relocation srcRva: 0x%X, dstOffset: 0x%X", relocation->srcRva, relocation->dstOffset);
				continue;
			}

			// at this point, the RVA itself must fit into 32-bit, even in 64-bit
			uint32_t dstRva32 = static_cast<uint32_t>(dstRva);

			// when incremental linking is enabled, the linker links function calls against "@ILT+offset" thunks rather
			// than the real function address. we can follow these thunks and get the function's real RVA.
			const uint32_t thunkTarget = symbols::FindThunkTargetByRVA(thunkDb, dstRva32);
			if (thunkTarget != 0u)
			{
				// the real destination RVA is at the thunk's target
				dstRva32 = thunkTarget;
			}

			// we found a new symbol, add it to the database
			LC_LOG_DEV("Found new symbol %s at RVA 0x%X", dstSymbolName.c_str(), dstRva32);
			symbols::CreateNewSymbol(dstSymbolName, dstRva32, symbolDB);

			// walk the relocations of the new symbol as well
			const coff::Symbol* nextSymbol = coff::GetSymbolByIndex(coffDb, relocation->dstSymbolNameIndex);
			if (nextSymbol)
			{
				openSymbols.push_back(nextSymbol);
			}

			--unknownSymbolsToFind;

			// did we already find all symbols?
			if (unknownSymbolsToFind == 0u)
			{
				LC_LOG_DEV("All symbols known, exiting");
				return;
			}
		}

		walkedAlready.insert(symbol);
	}

	// there are no more symbols to walk, but we haven't found all of them yet.

	// we can try finding the remaining symbols by matching their sections to sections in the PE image.
	// sections with the same name across several .obj files get merged into one section in the image, which makes it
	// a bit harder to find the address of an .obj's section in the image.
	// in order to do this, we find the section in question inside the image, and gather all different contributions
	// to this section. for each contribution, we then check whether its size matches the one in the .obj, and whether it
	// originated from the .obj in question.
	// if both match, we can finally check the symbol's names to ensure that we found the correct contribution.
	// from there, we can calculate the symbol's section-relative offset and reconstruct its RVA.

	// start by gathering all static functions and symbols which haven't been found already
	LC_LOG_DEV("Reconstructing symbol RVAs from executable contributions");
	LC_LOG_INDENT_DEV;

	// fetch all contributions for the .obj we're trying to reconstruct
	const symbols::ContributionDB::ContributionsPerCompiland* contributionsForThisCompiland = symbols::GetContributionsForCompilandName(contributionDb, objPath);
	if (!contributionsForThisCompiland)
	{
		LC_ERROR_DEV("Cannot find contributions for compiland %s", objPath.c_str());
		return;
	}

	types::vector<const coff::Symbol*> missingSymbols;
	missingSymbols.reserve(unknownSymbolsToFind);
	{
		const size_t count = coffDb->symbols.size();
		for (size_t i = 0u; i < count; ++i)
		{
			const coff::Symbol* symbol = coffDb->symbols[i];

			// if we are in our second pass (or later), check whether we tried reconstructing this symbol already
			if (pass > 0u)
			{
				const auto it = triedReconstructingAlready.find(symbol);
				if (it != triedReconstructingAlready.end())
				{
					// tried already
					continue;
				}
			}
			triedReconstructingAlready.insert(symbol);

			const ImmutableString& symbolName = coff::GetSymbolName(coffDb, symbol);
			if (strippedSymbols.find(symbolName) != strippedSymbols.end())
			{
				// the missing symbol is one we stripped
				continue;
			}

			// only static symbols can be missing, all others need to be known already
			if ((symbol->type == coff::SymbolType::STATIC_FUNCTION) ||
				(symbol->type == coff::SymbolType::STATIC_DATA))
			{
				const symbols::Symbol* srcSymbol = symbols::FindSymbolByName(symbolDB, symbolName);
				if (srcSymbol)
				{
					// found already, nothing more to do
					LC_LOG_DEV("Ignoring known symbol \"%s\"", symbolName.c_str());
					continue;
				}
				else if (symbols::IsRuntimeCheckRelatedSymbol(symbolName))
				{
					// code for runtime checks is always compiled into an .obj and doesn't need to be patched, and therefore
					// there's no need to find all the symbols
					LC_LOG_DEV("Ignoring runtime-check-related symbol \"%s\"", symbolName.c_str());
					continue;
				}
				else if (symbols::IsControlFlowGuardRelatedSymbol(symbolName))
				{
					// control flow guard stores function identifiers in separate symbols in .gfids$y section, which is not
					// an explicit section in the executable, and therefore cannot be found.
					// this is of no interest to us anyway, because we disable CFG.
					LC_LOG_DEV("Ignoring control flow guard-related symbol \"%s\"", symbolName.c_str());
					continue;
				}
				else if (symbols::IsExceptionRelatedSymbol(symbolName))
				{
					// even though exception-related symbols such as unwind tables and handlers are never patched or relocated
					// by us, catch clauses will refer to function and data symbols, and some of them could be stripped by us.
					// we therefore need to reconstruct these symbols as well.
					// we could also try reconstructing all exception-related symbols, but that has a serious impact on
					// performance!
					if (!symbols::IsExceptionClauseSymbol(symbolName))
					{
						// no exception clause, hence we're really not interested
						continue;
					}
				}

				missingSymbols.push_back(symbol);
			}
			else
			{
				// externally visible COMDAT symbols might not be known at this point, but will be found in one of
				// the OBJ files eventually. this is not an error.
				// ??$__vcrt_va_start_verify_argument_type@QBD@@YAXXZ is probably the most prominent example of where
				// this happens all the time.
			}
		}
	}

	// next try finding the missing symbols.
	// NOTE: this is carefully constructed to only run into O(N^2) in rare edge cases, because the original O(N^2) algorithm
	// caused a 25-30s slowdown for some users.
	const size_t missingSymbolCount = missingSymbols.size();

	// TODO: we use uint64_t to store the RVA and whether the missing symbol is an exception clause.
	// once we have our own PDB loading in place, we don't need this anymore and can use a set of uint32_t.
	types::unordered_set<uint64_t> potentialContributionRVAsAcrossAllMissingSymbols;
	potentialContributionRVAsAcrossAllMissingSymbols.reserve(contributionsForThisCompiland->size());

	for (size_t i = 0u; i < missingSymbolCount; ++i)
	{
		const coff::Symbol* symbol = missingSymbols[i];

		const ImmutableString& missingSymbolName = coff::GetSymbolName(coffDb, symbol);
		const uint64_t isExceptionClauseSymbol = symbols::IsExceptionClauseSymbol(missingSymbolName) ? (1ull << 32ull) : 0ull;

		const coff::Section& coffSection = coffDb->sections[symbol->sectionIndex];
		if (coff::IsMSVCJustMyCodeSection(coffSection.name.c_str()))
		{
			LC_LOG_DEV("Ignoring JustMyCode symbol %s in section %s", missingSymbolName.c_str(), coffSection.name.c_str());
			continue;
		}

		LC_LOG_DEV("Trying to find RVA for static symbol %s in section %s", missingSymbolName.c_str(), coffSection.name.c_str());
		LC_LOG_INDENT_DEV;

		// the address of the symbol relative to the COFF section it's defined in, e.g.:
		// .bss at COFF RVA 1000
		// symbol0 at COFF RVA 1000, at section relative addr. 0
		// symbol1 at COFF RVA 1004, at section relative addr. 4
		// symbol2 at COFF RVA 1008, at section relative addr. 8
		const uint32_t sectionRelativeAddress = symbol->rva - coffSection.rawDataRva;

		// find this section in the image
		const symbols::ImageSection* imageSection = symbols::FindImageSectionByName(imageSectionDb, coffSection.name);
		if (!imageSection)
		{
			LC_ERROR_DEV("Cannot find image section %s", coffSection.name.c_str());
			continue;
		}

		const uint32_t startOfImageSection = imageSection->rva;
		const uint32_t endOfImageSection = startOfImageSection + imageSection->size;

		// walk all contributions that are part of the image section and discard the ones that cannot match the symbol in question
		auto contributionIt = std::lower_bound(contributionsForThisCompiland->begin(), contributionsForThisCompiland->end(), startOfImageSection, &HasLowerRVA);
		while (contributionIt != contributionsForThisCompiland->end())
		{
			const symbols::Contribution* contribution = *contributionIt;
			++contributionIt;

			if (contribution->rva >= endOfImageSection)
			{
				// no more contributions that belong to this section
				break;
			}

			if (contribution->size != coffSection.rawDataSize)
			{
				// section size does not match
				continue;
			}
			else if (sectionRelativeAddress >= contribution->size)
			{
				// the symbol cannot be part of this contributing section because it is not large enough
				continue;
			}
			else
			{
				// this is a potential contribution, store it for now
				const uint32_t rva = contribution->rva + sectionRelativeAddress;
				potentialContributionRVAsAcrossAllMissingSymbols.insert(isExceptionClauseSymbol | rva);
			}
		}
	}

	// populate a cache of all DIA names for all potential contributions once
	// BEGIN EPIC MOD
	struct DiaRvaData
	{
		uint32_t rva;
		bool valid;
	};
	types::StringMap<DiaRvaData> diaNameToRva;
	// END EPIC MOD
	diaNameToRva.reserve(potentialContributionRVAsAcrossAllMissingSymbols.size());

	types::unordered_map<uint32_t, IDiaSymbol*> rvaToDiaSymbol;
	rvaToDiaSymbol.reserve(potentialContributionRVAsAcrossAllMissingSymbols.size());

	for (auto potentialContributionsIt : potentialContributionRVAsAcrossAllMissingSymbols)
	{
		const uint64_t setData = potentialContributionsIt;
		const uint32_t rva = setData & 0x00000000FFFFFFFFull;
		const bool isExceptionClauseSymbol = (setData & 0xFFFFFFFF00000000ull) != 0ull;

		// TODO: no longer needs to be special-cased once our own loading of PDB files is in place
		// exception clauses are labels stored as children of functions, so they need to be special-cased
		IDiaSymbol* diaSymbol = isExceptionClauseSymbol
			? dia::FindLabelByRva(provider->diaSession, rva)
			: dia::FindSymbolByRVA(provider->diaSession, rva);

		if (diaSymbol)
		{
			const std::wstring& diaName = dia::GetSymbolName(diaSymbol).GetString();
			const ImmutableString& name = string::ToUtf8String(diaName);

			// BEGIN EPIC MOD
			auto ixb = diaNameToRva.insert(std::make_pair(name, DiaRvaData{ rva, true }));
			if (!ixb.second && ixb.first->second.rva != rva && ixb.first->second.valid)
			{
				LC_LOG_DEV("Ignoring Dia symbol %s for fast path because multiple RVAs claim the symbol", name.c_str());
				ixb.first->second.valid = false;
			}
			// END EPIC MOD
			rvaToDiaSymbol.insert(std::make_pair(rva, diaSymbol));
		}
	}

	// perform the actual lookup using the cache we just built
	for (size_t i = 0u; i < missingSymbolCount; ++i)
	{
		const coff::Symbol* symbol = missingSymbols[i];

		const ImmutableString& missingSymbolName = coff::GetSymbolName(coffDb, symbol);
		const coff::Section& coffSection = coffDb->sections[symbol->sectionIndex];
		if (coff::IsMSVCJustMyCodeSection(coffSection.name.c_str()))
		{
			LC_LOG_DEV("Ignoring JustMyCode symbol %s in section %s", missingSymbolName.c_str(), coffSection.name.c_str());
			continue;
		}

		const std::string& coffUndecoratedName = symbols::UndecorateSymbolName(missingSymbolName);

		auto diaNameIt = diaNameToRva.find(ImmutableString(coffUndecoratedName.c_str()));
		// BEGIN EPIC MOD
		if (diaNameIt != diaNameToRva.end() && diaNameIt->second.valid)
		// END EPIC MOD
		{
			// fast path.
			// there is a symbol that matches the exact name of the symbol in the .obj file
			// BEGIN EPIC MOD
			const uint32_t rva = diaNameIt->second.rva;
			// END EPIC MOD

			LC_LOG_DEV("Fast path, found symbol %s at 0x%X", missingSymbolName.c_str(), rva);

			symbols::CreateNewSymbol(missingSymbolName, rva, symbolDB);

			openSymbols.push_back(symbol);

			--unknownSymbolsToFind;

			// did we already find all symbols?
			if (unknownSymbolsToFind == 0u)
			{
				LC_LOG_DEV("All symbols known, exiting");
				return;
			}
		}
		else
		{
			// slow path.
			// unfortunately, there is no exact match, but there might be several symbols/contributions with
			// a name that partially matches that of the symbol in the .obj file.
			// in that case, we check all contributions for this symbol, check whether its name is contained in that of
			// the .obj file, and check all its parents and their names as well.
			// if we find a symbol that matches all of the above, we have a worthy candidate. we can only accept this
			// symbol if it's the *only* candidate though. in case of several ambiguous contributions, we'd rather not
			// make a wrong guess.
			const std::wstring& wideCoffUndecoratedName = string::ToWideString(coffUndecoratedName);

			types::unordered_set<const symbols::Contribution*> potentialContributions;
			potentialContributions.reserve(contributionsForThisCompiland->size());

			const uint32_t sectionRelativeAddress = symbol->rva - coffSection.rawDataRva;

			// find this section in the image
			const symbols::ImageSection* imageSection = symbols::FindImageSectionByName(imageSectionDb, coffSection.name);
			if (!imageSection)
			{
				// BEGIN EPIC MOD
				if (diaNameIt != diaNameToRva.end())
				{
					// fast path.
					// there is a symbol that matches the exact name of the symbol in the .obj file
					const uint32_t rva = diaNameIt->second.rva;

					LC_LOG_DEV("Fast path, found symbol %s at 0x%X", missingSymbolName.c_str(), rva);

					symbols::CreateNewSymbol(missingSymbolName, rva, symbolDB);

					openSymbols.push_back(symbol);

					--unknownSymbolsToFind;

					// did we already find all symbols?
					if (unknownSymbolsToFind == 0u)
					{
						LC_LOG_DEV("All symbols known, exiting");
						return;
					}
				}
				else
				{
					LC_ERROR_DEV("Cannot find image section %s", coffSection.name.c_str());
				}
				// END EPIC MOD
				continue;
			}

			const uint32_t startOfImageSection = imageSection->rva;
			const uint32_t endOfImageSection = startOfImageSection + imageSection->size;

			// walk all contributions that are part of the image section and discard the ones that cannot match the symbol in question
			auto contributionIt = std::lower_bound(contributionsForThisCompiland->begin(), contributionsForThisCompiland->end(), startOfImageSection, &HasLowerRVA);
			while (contributionIt != contributionsForThisCompiland->end())
			{
				const symbols::Contribution* contribution = *contributionIt;
				++contributionIt;

				if (contribution->rva >= endOfImageSection)
				{
					// no more contributions that belong to this section
					break;
				}

				if (contribution->size != coffSection.rawDataSize)
				{
					// section size does not match
					continue;
				}
				else if (sectionRelativeAddress >= contribution->size)
				{
					// the symbol cannot be part of this contributing section because it is not large enough
					continue;
				}
				else
				{
					// this is a potential contribution, store it for now
					potentialContributions.emplace(contribution);
				}
			}

			types::unordered_set<uint32_t> worthyCandidates;
			worthyCandidates.reserve(potentialContributions.size());

			for (auto it = potentialContributions.begin(); it != potentialContributions.end(); ++it)
			{
				const symbols::Contribution* contribution = *it;
				const uint32_t rva = contribution->rva + sectionRelativeAddress;

				// get the symbol name at the potential RVA from the DIA cache
				{
					auto cacheIt = rvaToDiaSymbol.find(rva);
					if (cacheIt != rvaToDiaSymbol.end())
					{
						IDiaSymbol* diaSymbol = cacheIt->second;
						const std::wstring& diaName = dia::GetSymbolName(diaSymbol).GetString();

						if (string::Contains(wideCoffUndecoratedName.c_str(), diaName.c_str()))
						{
							// the name partially matches, now check all its parents
							bool doAllParentsMatch = true;
							IDiaSymbol* parent = dia::GetParent(diaSymbol);
							while (parent)
							{
								// we are only interested in parents which are functions
								if (!dia::IsFunction(parent))
								{
									break;
								}

								const std::wstring& parentName = dia::GetSymbolName(parent).GetString();
								if (string::Contains(wideCoffUndecoratedName.c_str(), parentName.c_str()))
								{
									parent = dia::GetParent(parent);
								}
								else
								{
									doAllParentsMatch = false;
									break;
								}
							}

							if (doAllParentsMatch)
							{
								worthyCandidates.emplace(rva);
							}
						}
					}
				}
			}

			if (worthyCandidates.size() == 1u)
			{
				// there was only one worthy candidate
				const uint32_t rva = *worthyCandidates.begin();

				LC_LOG_DEV("Slow path, found symbol %s at 0x%X", missingSymbolName.c_str(), rva);

				CreateNewSymbol(missingSymbolName, rva, symbolDB);

				openSymbols.push_back(symbol);

				--unknownSymbolsToFind;

				// did we already find all symbols?
				if (unknownSymbolsToFind == 0u)
				{
					LC_LOG_DEV("All symbols known, exiting");
					return;
				}
			}
			else if (worthyCandidates.size() == 0u)
			{
				// if we had potential candidates but could not find a symbol, there is still a possibility that the
				// symbol has been stripped by the linker due to the /Gw option that puts data symbols into separate
				// sections. this happens in ComplexClassGlobal.cpp in our test cases as well.
				LC_WARNING_DEV("Could not find symbol %s in compiland %s, possibly stripped by linker",
					coff::GetSymbolName(coffDb, symbol).c_str(),
					objPath.c_str());
			}
			else
			{
				LC_ERROR_DEV("Contributions for symbol %s are ambiguous", missingSymbolName.c_str());
			}
		}
	}

	if (openSymbols.size() != 0u)
	{
		// we found new symbols to walk, so do another pass
		LC_LOG_DEV("Doing another pass");
		++pass;
		goto walkOpenSymbols;
	}
}


#endif