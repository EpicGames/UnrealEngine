// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_Coff.h"
#include "LC_Symbols.h"
#include "LC_Executable.h"


namespace symbols
{
	void ReconstructFromExecutableCoff
	(
		const symbols::Provider* provider,
		const executable::Image* image,
		const executable::ImageSectionDB* imageSections,
		const coff::CoffDB* coffDb,
		const types::StringSet& strippedSymbols,
		const symbols::ObjPath& objPath,
		const symbols::ContributionDB* contributionDb,
		const symbols::ThunkDB* thunkDb,
		const symbols::ImageSectionDB* imageSectionDb,
		symbols::SymbolDB* symbolDB
	);
}


#endif