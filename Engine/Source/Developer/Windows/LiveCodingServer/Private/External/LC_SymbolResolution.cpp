// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_SymbolResolution.h"
#include "LC_PointerUtil.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

// BEGIN EPIC MOD
#pragma push_macro("OPTIONAL")
#ifndef OPTIONAL
#define OPTIONAL
#endif
#include <dbghelp.h>
#pragma pop_macro("OPTIONAL")
// END EPIC MOD

namespace symbolResolution
{
	// loads symbol information for this process in the constructor,
	// and unloads it in the destructor.
	// makes it easier to only load information when it is needed.
	struct SymbolHelper
	{
		SymbolHelper(void)
		{
			::SymSetOptions(SYMOPT_DEFERRED_LOADS | SYMOPT_LOAD_LINES | SYMOPT_UNDNAME | SYMOPT_INCLUDE_32BIT_MODULES);
			const BOOL success = ::SymInitialize(::GetCurrentProcess(), "", true);
			// BEGIN EPIC MOD
			if (success == Windows::FALSE)
			// END EPIC MOD
			{
				const DWORD error = ::GetLastError();
				LC_ERROR_DEV("SymInitialize failed with error 0x%X", error);
			}
		}

		~SymbolHelper(void)
		{
			const BOOL success = ::SymCleanup(::GetCurrentProcess());
			// BEGIN EPIC MOD
			if (success == Windows::FALSE)
			// END EPIC MOD
			{
				const DWORD error = ::GetLastError();
				LC_ERROR_DEV("SymCleanup failed with error 0x%X", error);
			}
		}
	};

	SymbolInfo ResolveSymbolsForAddress(const void* const address)
	{
		// prepare symbol information
		static SymbolHelper s_symbolHelper;

		// retrieve function
		DWORD64 displacement64 = 0;
		ULONG64 buffer[(sizeof(SYMBOL_INFO) + MAX_SYM_NAME * sizeof(TCHAR) + sizeof(ULONG64) - 1) / sizeof(ULONG64)] = {};
		SYMBOL_INFO* symbolInfo = reinterpret_cast<SYMBOL_INFO*>(buffer);
		symbolInfo->SizeOfStruct = sizeof(SYMBOL_INFO);
		symbolInfo->MaxNameLen = MAX_SYM_NAME;

		const HANDLE process = ::GetCurrentProcess();
		const DWORD64 intAddress = pointer::AsInteger<DWORD64>(address);

		if (::SymFromAddr(process, intAddress, &displacement64, symbolInfo))
		{
			// retrieve filename and line number
			DWORD displacement = 0;
			IMAGEHLP_LINE64 line = {};
			line.SizeOfStruct = sizeof(IMAGEHLP_LINE64);

			if (::SymGetLineFromAddr64(process, intAddress, &displacement, &line))
				return SymbolInfo(symbolInfo->Name, line.FileName, line.LineNumber);
		}

		return SymbolInfo("unknown", "unknown", 0);
	}
}


#endif