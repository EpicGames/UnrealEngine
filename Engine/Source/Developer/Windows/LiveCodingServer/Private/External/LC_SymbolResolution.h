// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

#include "LC_SymbolInfo.h"


namespace symbolResolution
{
	// Resolves the symbol information for a given address
	SymbolInfo ResolveSymbolsForAddress(const void* const address);
}


#endif