// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Symbols.h"
#include "LC_Filesystem.h"
#include "LC_StringUtil.h"
#include "LC_Process.h"
#include "LC_Telemetry.h"
#include "LC_ImmutableString.h"
#include "LC_SymbolPatterns.h"
#include "LC_DiaUtil.h"
#include "LC_Memory.h"
#include "LC_FileAttributeCache.h"
#include "LC_PointerUtil.h"
#include "LC_Coff.h"
#include "LC_CoffCache.h"
#include "LC_CriticalSection.h"
#include "LC_NameMangling.h"
#include "LC_UniqueId.h"
#include "LC_Amalgamation.h"
#include "LC_CompilerOptions.h"
#include "LC_AppSettings.h"
// BEGIN EPIC MOD
#include "LC_Allocators.h"
#include <diacreate.h>
#include <algorithm>
#include <process.h>

#include "Misc/FileHelper.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Dom/JsonObject.h"

#include "Windows/AllowWindowsPlatformAtomics.h"
// END EPIC MOD

namespace
{
	static telemetry::Accumulator g_loadedPdbSize("PDB size");


	static inline bool SortContributionByAscendingRVA(const symbols::Contribution* lhs, const symbols::Contribution* rhs)
	{
		return lhs->rva < rhs->rva;
	}


	static inline bool SortImageSectionByAscendingRVA(const symbols::ImageSection& lhs, const symbols::ImageSection& rhs)
	{
		return lhs.rva < rhs.rva;
	}


	static inline bool ImageSectionHasLowerRVA(uint32_t rva, const symbols::ImageSection& rhs)
	{
		return rva < rhs.rva;
	}


	static inline bool ContributionHasLowerRvaUpperBound(uint32_t rva, const symbols::Contribution* rhs)
	{
		return rva < rhs->rva;
	}


	static inline bool ContributionHasLowerRvaLowerBound(const symbols::Contribution* lhs, uint32_t rva)
	{
		return lhs->rva < rva;
	}


	class LoadCallback : public IDiaLoadCallback
	{
	public:
		explicit LoadCallback(uint32_t openOptions)
			: m_openOptions(openOptions)
		{
		}

		virtual ~LoadCallback(void)
		{
		}

		virtual HRESULT STDMETHODCALLTYPE QueryInterface(
			/* [in] */ REFIID riid,
			/* [iid_is][out] */ _COM_Outptr_ void __RPC_FAR *__RPC_FAR *ppvObject)
		{
			// always set out parameter to NULL, validating it first
			if (!ppvObject)
			{
				return E_INVALIDARG;
			}

			*ppvObject = NULL;
			if (riid == IID_IUnknown || riid == IID_IDiaLoadCallback)
			{
				// increment the reference count and return the pointer
				*ppvObject = this;
				AddRef();

				return NOERROR;
			}

			return E_NOINTERFACE;
		}

		virtual ULONG STDMETHODCALLTYPE AddRef(void)
		{
			::InterlockedIncrement(&m_refCount);
			return m_refCount;
		}

		virtual ULONG STDMETHODCALLTYPE Release(void)
		{
			// decrement the object's internal counter and delete the interface if zero
			ULONG refCount = ::InterlockedDecrement(&m_refCount);
			if (0 == m_refCount)
			{
				delete this;
			}

			return refCount;
		}

		virtual HRESULT STDMETHODCALLTYPE NotifyDebugDir(
			/* [in] */ BOOL fExecutable,
			/* [in] */ DWORD cbData,
			/* [size_is][in] */ BYTE *pbData)
		{
			LC_UNUSED(fExecutable);
			LC_UNUSED(cbData);
			LC_UNUSED(pbData);

			return S_OK;
		}

		virtual HRESULT STDMETHODCALLTYPE NotifyOpenDBG(
			/* [in] */ LPCOLESTR dbgPath,
			/* [in] */ HRESULT resultCode)
		{
			LC_UNUSED(dbgPath);
			LC_UNUSED(resultCode);

			return S_OK;
		}

		virtual HRESULT STDMETHODCALLTYPE NotifyOpenPDB(
			/* [in] */ LPCOLESTR pdbPath,
			/* [in] */ HRESULT resultCode)
		{
			if (resultCode == S_OK)
			{
				// the PDB was successfully loaded from this path
				const Filesystem::PathAttributes attributes = Filesystem::GetAttributes(pdbPath);
				const uint64_t size = Filesystem::GetSize(attributes);

				if (m_openOptions & symbols::OpenOptions::ACCUMULATE_SIZE)
				{
					LC_LOG_DEV("Loading PDB %S", pdbPath);

					g_loadedPdbSize.Accumulate(size);
					g_loadedPdbSize.Print();
					g_loadedPdbSize.ResetCurrent();
				}
			}

			return S_OK;
		}

		virtual HRESULT STDMETHODCALLTYPE RestrictRegistryAccess(void)
		{
			return S_OK;
		}

		virtual HRESULT STDMETHODCALLTYPE RestrictSymbolServerAccess(void)
		{
			return S_OK;
		}

	private:
		volatile ULONG m_refCount = 0ul;
		uint32_t m_openOptions;
	};


	static symbols::Provider* CreateProvider(const wchar_t* filename, uint32_t openOptions)
	{
		// create a IDiaDataSource manually from the msdia140.dll that we ship with
		std::wstring msdiaPath = Process::Current::GetImagePath().GetString();
		msdiaPath = Filesystem::GetDirectory(msdiaPath.c_str()).GetString();
		msdiaPath += L"\\";
		msdiaPath += L"msdia140.dll";

		HMODULE diaModule = ::LoadLibraryW(msdiaPath.c_str());
		if (diaModule == NULL)
		{
			LC_WARNING_USER("msdia140.dll seems to be missing in the Live++ directory at path %S (module: %S)", msdiaPath.c_str(), filename);

			// let the OS find any msdia140.dll in the system
			diaModule = ::LoadLibraryW(L"msdia140.dll");
			if (diaModule == NULL)
			{
				LC_ERROR_USER("No msdia140.dll found by the operating system (module: %S)", filename);
				return nullptr;
			}
		}

		typedef decltype(&DllGetClassObject) DllGetClassObjectFunction;

		DllGetClassObjectFunction getClassObjectFunction = reinterpret_cast<DllGetClassObjectFunction>(reinterpret_cast<uintptr_t>(::GetProcAddress(diaModule, "DllGetClassObject")));
		if (!getClassObjectFunction)
		{
			LC_ERROR_USER("Could not find DllGetClassObject function in DIA DLL (module: %S)", filename);
			return nullptr;
		}

		IClassFactory* classFactory = nullptr;
		{
			const HRESULT hr = getClassObjectFunction(__uuidof(DiaSource), IID_IClassFactory, reinterpret_cast<void**>(&classFactory));
			if (FAILED(hr))
			{
				LC_ERROR_USER("Cannot create IClassFactory instance (module: %S, error: 0x%X)", filename, hr);
				return nullptr;
			}

			if (!classFactory)
			{
				LC_ERROR_USER("IClassFactory interface not supported by DIA DLL (module: %S, error: 0x%X)", filename, hr);
				return nullptr;
			}
		}

		IDiaDataSource* diaDataSource = nullptr;
		{
			const HRESULT hr = classFactory->CreateInstance(NULL, __uuidof(IDiaDataSource), reinterpret_cast<void**>(&diaDataSource));
			if (FAILED(hr))
			{
				LC_ERROR_USER("Cannot create IDiaDataSource instance (module: %S, error: 0x%X)", filename, hr);
				return nullptr;
			}

			classFactory->Release();
		}

		LoadCallback* callback = new LoadCallback(openOptions);

		HRESULT hr = S_OK;
		if (openOptions & symbols::OpenOptions::USE_SYMBOL_SERVER)
		{
			// allow DIA to use a symbol server.
			// symbols are always loaded from the MS symbol server and cached in the Live++\Symbols directory.
			std::wstring symbolPath(L"srv*");
			symbolPath += appSettings::GetSymbolsDirectory();
			symbolPath += L"*https://msdl.microsoft.com/download/symbols";

			hr = diaDataSource->loadDataForExe(filename, symbolPath.c_str(), callback);
		}
		else
		{
			hr = diaDataSource->loadDataForExe(filename, NULL, callback);
		}

		if (hr != S_OK)
		{
			// warn about PDB files without useful debug info
			if (hr == E_PDB_NO_DEBUG_INFO)
			{
				LC_WARNING_USER("PDB file for module %S does not contain debug info", filename);
			}
			// don't log an error if the PDB could not be found
			else if (hr != E_PDB_NOT_FOUND)
			{
				LC_ERROR_USER("Cannot load PDB file for module %S. Error: 0x%X", filename, hr);
			}

			// BEGIN EPIC MOD - Show a warning if we don't have a PDB for this module. Since we only enable Live++ for modules that we built, we should always have a PDB.
			if (hr == E_PDB_NOT_FOUND)
			{
				LC_WARNING_USER("No PDB file found for module %S. If this is a packaged build, make sure that debug files are being staged. Live coding will be disabled for this module.", Filesystem::GetFilename(filename).GetString());
			}
			// END EPIC MOD

			return nullptr;
		}

		IDiaSession* diaSession = nullptr;
		hr = diaDataSource->openSession(&diaSession);
		if (hr != S_OK)
		{
			LC_ERROR_USER("Cannot open PDB session for module %S. Error: 0x%X", filename, hr);

			return nullptr;
		}

		IDiaSymbol* globalScope = nullptr;
		hr = diaSession->get_globalScope(&globalScope);
		if (hr != S_OK)
		{
			LC_ERROR_USER("Cannot retrieve PDB global scope for module %S. Error: 0x%X", filename, hr);

			return nullptr;
		}

		const Filesystem::PathAttributes& attributes = Filesystem::GetAttributes(filename);
		const uint64_t lastModification = Filesystem::GetLastModificationTime(attributes);
		symbols::Provider* provider = new symbols::Provider { diaDataSource, diaSession, globalScope, lastModification };
		return provider;
	}


	static bool DoesCompilandBelongToLibrary(const dia::SymbolName& libraryName)
	{
		if (libraryName.GetString())
		{
			// library names also contain .obj files, we are not interested in those
			const Filesystem::Path libExtension = Filesystem::GetExtension(libraryName.GetString());
			if (libExtension.GetLength() != 0u)
			{
				// found an extension
				const Filesystem::Path uppercaseExtensionName = libExtension.ToUpper();
				if (string::Contains(uppercaseExtensionName.GetString(), L".LIB"))
				{
					return true;
				}
			}
			else
			{
				const std::wstring& uppercaseLibraryName = string::ToUpper(libraryName.GetString());
				if (string::Contains(uppercaseLibraryName.c_str(), L".LIB"))
				{
					return true;
				}
			}
		}

		return false;
	}


	static void AddFileDependency(symbols::CompilandDB* compilandDb, const ImmutableString& changedSrcFile, const ImmutableString& recompiledObjFile, uint64_t srcFileLastModificationTime)
	{
		// try updating dependencies for the given file and create a new dependency in case none exists yet
		const auto& insertPair = compilandDb->dependencies.emplace(changedSrcFile, nullptr);
		symbols::Dependency*& dependency = insertPair.first->second;

		if (insertPair.second)
		{
			// insertion was successful, create a new dependency
			dependency = LC_NEW(&g_dependencyAllocator, symbols::Dependency);
			dependency->lastModification = srcFileLastModificationTime;
			dependency->parentDirectory = nullptr;
			dependency->hadInitialChange = false;
		}

		// update entry
		dependency->objPaths.push_back(recompiledObjFile);
	}
}


namespace symbols
{
	Symbol* CreateNewSymbol(const ImmutableString& name, uint32_t rva, SymbolDB* db)
	{
		Symbol* symbol = LC_NEW(&g_symbolAllocator, Symbol) { name, rva };
		db->symbolsByName.emplace(name, symbol);
		db->symbolsByRva.emplace(rva, symbol);

		// test if this symbol contains an anonymous namespace in its name
		if (string::Find(name.c_str(), symbolPatterns::ANONYMOUS_NAMESPACE_PATTERN))
		{
			db->ansSymbols.push_back(symbol);
		}

		return symbol;
	}


	Provider* OpenEXE(const wchar_t* filename, uint32_t openOptions)
	{
		return CreateProvider(filename, openOptions);
	}


	void Close(Provider* provider)
	{
		if (provider)
		{
			memory::ReleaseAndNull(provider->globalScope);
			memory::ReleaseAndNull(provider->diaSession);
			memory::ReleaseAndNull(provider->diaDataSource);

			delete provider;
		}
	}

	// BEGIN EPIC MOD - Static grouping of compilands by unity blobs
	static CriticalSection g_objFileToCompilandIdCS;
	static std::unordered_map<std::wstring, uint32_t> g_objFileToCompilandId;
	static std::unordered_set<std::wstring> g_checkedUnityManifests;

	void ResetCachedUnityManifests()
	{
		CriticalSection::ScopedLock lock(&g_objFileToCompilandIdCS);
		g_checkedUnityManifests.clear();
	}

	bool TryGetCompilandIdFromUnityManifest(const std::wstring& objFile, uint32_t& compilandId)
	{
		std::wstring normalizedObjFile = Filesystem::NormalizePath(objFile.c_str()).GetString();
		CriticalSection::ScopedLock lock(&g_objFileToCompilandIdCS);

		// Check if it's already cached
		std::unordered_map<std::wstring, uint32_t>::iterator it = g_objFileToCompilandId.find(normalizedObjFile);
		if(it == g_objFileToCompilandId.end())
		{
			// Read the manifest file
			std::wstring BaseDir = Filesystem::GetDirectory(normalizedObjFile.c_str()).GetString();
			std::wstring ManifestFile = BaseDir + L"\\LiveCodingInfo.json";

			// If we've already tried to read this string, don't try again
			if(!g_checkedUnityManifests.insert(ManifestFile).second)
			{
				return false;
			}

			// Read the file to a string
			FString FileContents;
			if (!FFileHelper::LoadFileToString(FileContents, ManifestFile.c_str()))
			{
				return false;
			}

			// Deserialize a JSON object from the string
			TSharedPtr< FJsonObject > Object;
			TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(FileContents);
			if (!FJsonSerializer::Deserialize(Reader, Object) || !Object.IsValid())
			{
				LC_WARNING_USER("%S could not be parsed", ManifestFile.c_str());
				return false;
			}

			const TSharedPtr<FJsonObject>* FilesObject;
			if (!Object->TryGetObjectField(TEXT("RemapUnityFiles"), FilesObject))
			{
				LC_WARNING_USER("%S is not a valid manifest file", ManifestFile.c_str());
				return false;
			}

			for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : FilesObject->Get()->Values)
			{
				std::wstring UnityObjectFile = Filesystem::NormalizePath((BaseDir + L"\\" + *Pair.Key).c_str()).GetString();
				uint32_t UnityCompilandId = uniqueId::Generate(UnityObjectFile);
				g_objFileToCompilandId.insert(std::make_pair(UnityObjectFile, UnityCompilandId));

				const FJsonValue* Value = Pair.Value.Get();
				if (Value->Type != EJson::Array)
				{
					LC_WARNING_USER("%S is not a valid manifest file", ManifestFile.c_str());
					return false;
				}

				const TArray<TSharedPtr<FJsonValue>>& SourceFileValues = Value->AsArray();
				for (const TSharedPtr<FJsonValue>& SourceFileValue : SourceFileValues)
				{
					if (SourceFileValue->Type != EJson::String)
					{
						LC_WARNING_USER("%S is not a valid manifest file", ManifestFile.c_str());
						return false;
					}

					std::wstring MemberObjFile = Filesystem::NormalizePath((BaseDir + L"\\" + *SourceFileValue->AsString()).c_str()).GetString();
					g_objFileToCompilandId.insert(std::make_pair(MemberObjFile, UnityCompilandId));
				}
			}

			// Check again for the object file we're interested in
			it = g_objFileToCompilandId.find(normalizedObjFile);
			if(it == g_objFileToCompilandId.end())
			{
				return false;
			}
		}

		compilandId = it->second;
		return true;
	}

	uint32_t GetCompilandIdFromPath(const std::wstring& objPath)
	{
		uint32 compilandId;
		if(TryGetCompilandIdFromUnityManifest(objPath, compilandId))
		{
			return compilandId;
		}
		else
		{
			return uniqueId::Generate(Filesystem::NormalizePath(objPath.c_str()).GetString());
		}
	}
	// END EPIC MOD

	SymbolDB* GatherSymbols(Provider* provider)
	{
		telemetry::Scope telemetryScope("Gathering symbols");

		SymbolDB* symbolDB = new SymbolDB;

		// enumerate all public symbols
		{
			const types::vector<IDiaSymbol*>& publicSymbols = dia::GatherChildSymbols(provider->globalScope, SymTagPublicSymbol);
			const size_t symbolCount = publicSymbols.size();
			symbolDB->symbolsByName.reserve(symbolCount);
			symbolDB->symbolsByRva.reserve(symbolCount);
			symbolDB->patchableFunctionSymbols.reserve(symbolCount);
			symbolDB->ansSymbols.reserve(16u);	// we don't expect to find much of these

			for (size_t i = 0u; i < symbolCount; ++i)
			{
				IDiaSymbol* publicSymbol = publicSymbols[i];

				// public symbols always come with a decorated name that is unique across all translation units. otherwise, linking wouldn't work.
				const dia::SymbolName& name = dia::GetSymbolName(publicSymbol);
				const ImmutableString& symbolName = string::ToUtf8String(name.GetString());

				const uint32_t rva = dia::GetSymbolRVA(publicSymbol);
				if (rva == 0u)
				{
					// the linker-generated __ImageBase always sits at RVA zero. ignore it.

					// compiler-generated symbols such as __tls_array don't have any RVA, because they always reside at the same address, e.g. relative to a segment register.
					// one such example would be how thread-local storage variables are accessed:
					//   the generated code always fetches the flat address of the thread-local storage array from the TEB (https://en.wikipedia.org/wiki/Win32_Thread_Information_Block).
					//   the TEB itself can be accessed using segment register FS on x86, and GS on x64, so one of the first instructions of thread-local storage access is always going to
					//   access the member at 0x2C/0x58 relative to FS/GS, e.g.:
					//     mov eax, dword ptr fs:0x2C (x86)
					//     mov rax, qword ptr gs:0x58 (x64)
					// see http://www.nynaeve.net/?p=180 for more in-depth information about thread-local storage on Windows.

					// other compiler-generated or linker-generated symbols include CFG symbols (e.g. ___guard_fids_count, 
					// ___guard_iat_count, ___guard_iat_table, ___guard_fids_table) and others. we store them separately to be able
					// to ignore them when reconstructing symbols later.
					symbolDB->symbolsWithoutRva.insert(symbolName);
				}
				else
				{
					Symbol* symbol = CreateNewSymbol(symbolName, rva, symbolDB);

					if (dia::IsFunction(publicSymbol))
					{
						symbolDB->patchableFunctionSymbols.push_back(symbol);
					}
				}

				publicSymbol->Release();
			}
		}

		return symbolDB;
	}


	ContributionDB* GatherContributions(Provider* provider)
	{
		telemetry::Scope telemetryScope("Gathering contributions");

		ContributionDB* contributionDB = new ContributionDB;
		IDiaEnumSectionContribs* enumSectionContributions = dia::FindSectionContributionsEnumerator(provider->diaSession);
		if (enumSectionContributions)
		{
			LONG count = 0;
			enumSectionContributions->get_Count(&count);

			if (count > 0)
			{
				types::vector<IDiaSectionContrib*> sectionContributions;
				sectionContributions.resize(static_cast<size_t>(count));
				contributionDB->contributions.reserve(static_cast<size_t>(count));

				ULONG fetched = 0u;
				enumSectionContributions->Next(static_cast<ULONG>(count), &sectionContributions[0], &fetched);

				// find highest ID first
				DWORD highestId = 0u;
				for (ULONG i = 0u; i < fetched; ++i)
				{
					IDiaSectionContrib* sectionContribution = sectionContributions[i];

					DWORD id = 0u;
					sectionContribution->get_compilandId(&id);

					highestId = std::max<DWORD>(highestId, id);
				}

				// prepare size for string table. IDs are 1-based.
				contributionDB->originalStringTable.resize(highestId + 1u);

				for (ULONG i = 0u; i < fetched; ++i)
				{
					IDiaSectionContrib* sectionContribution = sectionContributions[i];

					DWORD rva = 0u;
					sectionContribution->get_relativeVirtualAddress(&rva);

					DWORD size = 0u;
					sectionContribution->get_length(&size);

					DWORD id = 0u;
					sectionContribution->get_compilandId(&id);

					if (contributionDB->originalStringTable[id].GetLength() == 0)
					{
						IDiaSymbol* contributingCompiland = nullptr;
						sectionContribution->get_compiland(&contributingCompiland);

						if (contributingCompiland)
						{
							// store the compiland name directly, even though it may be relative.
							// when doing lookups into the string table, we then convert this compiland name
							// to the real one that exists on disk.
							const dia::SymbolName& compilandName = dia::GetSymbolName(contributingCompiland);
							contributionDB->originalStringTable[id] = string::ToUtf8String(compilandName.GetString());

							contributingCompiland->Release();
						}
					}

					if ((rva != 0u) && (size != 0u))
					{
						Contribution* newContribution = LC_NEW(&g_contributionAllocator, Contribution) { id, rva, size };
						contributionDB->contributions.emplace_back(newContribution);
					}

					sectionContribution->Release();
				}
			}

			enumSectionContributions->Release();
		}

		// sort contributions by RVA
		std::sort(contributionDB->contributions.begin(), contributionDB->contributions.end(), &SortContributionByAscendingRVA);

		return contributionDB;
	}


	void FinalizeContributions(const CompilandDB* compilandDb, ContributionDB* db)
	{
		// first convert the string table of original compilands to their real names of the .objs on disk
		{
			const size_t count = db->originalStringTable.size();
			db->objOnDiskStringTable.reserve(count);

			for (size_t i = 0u; i < count; ++i)
			{
				const ImmutableString& originalCompilandName = db->originalStringTable[i];

				// try to find the real name of the .obj on disk
				const auto it = compilandDb->compilandNameToObjOnDisk.find(originalCompilandName);
				if (it != compilandDb->compilandNameToObjOnDisk.end())
				{
					// found, store this into the database
					db->objOnDiskStringTable.emplace_back(it->second);
				}
				else
				{
					// not found, store the original name instead
					db->objOnDiskStringTable.emplace_back(originalCompilandName);
				}
			}

			// clear the string table of original compilands, they are no longer needed and should not be accessed
			db->originalStringTable.clear();
			db->originalStringTable.shrink_to_fit();
		}

		// then build a per-compiland array of contributions for later use
		{
			const size_t count = db->objOnDiskStringTable.size();
			db->contributionsPerCompilandNameIndex.resize(count);

			// make some space first
			for (size_t i = 0u; i < count; ++i)
			{
				db->contributionsPerCompilandNameIndex[i].reserve(256u);
			}

			// all the contributions are sorted by ascending RVA already, so if we iterate through them and assign them
			// to their corresponding compiland, those contributions will be sorted as well.
			for (auto it : db->contributions)
			{
				symbols::Contribution* contribution = it;
				db->contributionsPerCompilandNameIndex[contribution->compilandNameIndex].emplace_back(contribution);
			}

			// free up wasted space
			for (size_t i = 0u; i < count; ++i)
			{
				db->contributionsPerCompilandNameIndex[i].shrink_to_fit();
			}
		}

		// finally, build a lookup-table for going from compiland name to compiland name index
		{
			const uint32_t count = static_cast<uint32_t>(db->objOnDiskStringTable.size());
			for (uint32_t i = 0u; i < count; ++i)
			{
				const ImmutableString& compilandName = db->objOnDiskStringTable[i];
				db->compilandNameToCompilandNameIndex.emplace(compilandName, i);
			}
		}
	}


	DiaCompilandDB* GatherDiaCompilands(Provider* provider)
	{
		telemetry::Scope telemetryScope("Gathering DIA compilands");

		DiaCompilandDB* database = new DiaCompilandDB;
		database->symbols = dia::GatherChildSymbols(provider->globalScope, SymTagCompiland);

		return database;
	}


	CompilandDB* GatherCompilands(const Provider* provider, const DiaCompilandDB* diaCompilandDb, unsigned int splitAmalgamatedFilesThreshold, uint32_t compilandOptions)
	{
		telemetry::Scope telemetryScope("Gathering compilands");

		// expand options
		const bool generateLogs = (compilandOptions & CompilandOptions::GENERATE_LOGS) != 0u;
		const bool forcePchPdbs = (compilandOptions & CompilandOptions::FORCE_PCH_PDBS) != 0u;
		const bool trackObjOnly = (compilandOptions & CompilandOptions::TRACK_OBJ_ONLY) != 0u;

		FileAttributeCache fileCache;

		const size_t count = diaCompilandDb->symbols.size();

		CompilandDB* compilandDb = new CompilandDB;
		compilandDb->compilands.reserve(count);

		for (size_t i = 0u; i < count; ++i)
		{
			IDiaSymbol* diaSymbol = diaCompilandDb->symbols[i];

			// get the name of the compiland and check if this is an object file.
			// there are other compilands like import .dll and resource files.
			const dia::SymbolName& diaCompilandPath = dia::GetSymbolName(diaSymbol);
			std::wstring compilandPath(diaCompilandPath.GetString());
			const std::wstring& uppercaseCompilandPath = string::ToUpper(compilandPath);

			const bool isObjPath = string::Contains(uppercaseCompilandPath.c_str(), L".OBJ") || string::Contains(uppercaseCompilandPath.c_str(), L".O");
			if (isObjPath)
			{
				// a valid compiland, gather more information.
				// getting the filename of the .obj file is surprisingly involved.
				// these are the facts:
				// - the compiland path sometimes stores relative paths.
				// - the 'obj' compiland environment always stores absolute paths. however, these
				//   paths point to the files that were *compiled*, not the ones that were *linked*.
				//   therefore, these paths can point to remote paths (when using distributed build systems such as FASTBuild),
				//   or temporary files (e.g. BAM uses .obj.tmp and then moves the file to .obj).
				// - we are not allowed to normalize these filenames. otherwise, normalizing will resolve symbolic links
				//   and virtual drives, which means that files compiled by Live++ will point to a different path than
				//   the original compilands.
				//   this can (and did!) break builds when including header files that use #pragma once.

				// to find the correct .obj in all cases, our strategy is the following:
				// - test the compiland path first
				// - if a file cannot be found there, try the absolute compiland environment directory combined with the compiland's filename
				// - if a file cannot be found there, try the compiler working directory plus compiland path
				// - if no file cannot be found, ignore this compiland
				std::wstring environmentCompilandPath;

				const types::vector<IDiaSymbol*>& environments = dia::GatherChildSymbols(diaSymbol, SymTagCompilandEnv);
				const size_t environmentCount = environments.size();

				unsigned int foundOptions = 0u;
				std::wstring optionsCache[5u];
				for (size_t j = 0u; j < environmentCount; ++j)
				{
					IDiaSymbol* environment = environments[j];
					const dia::SymbolName& environmentName = dia::GetSymbolName(environment);
					const dia::Variant& environmentOption = dia::GetSymbolEnvironmentOption(environment);

					if (string::Matches(environmentName.GetString(), L"src"))
					{
						optionsCache[0] = environmentOption.GetString();
						++foundOptions;
					}
					else if (string::Matches(environmentName.GetString(), L"obj"))
					{
						environmentCompilandPath = environmentOption.GetString();
					}
					else if (string::Matches(environmentName.GetString(), L"pdb"))
					{
						optionsCache[1] = environmentOption.GetString();
						++foundOptions;
					}
					else if (string::Matches(environmentName.GetString(), L"cwd"))
					{
						optionsCache[2] = environmentOption.GetString();
						++foundOptions;
					}
					else if (string::Matches(environmentName.GetString(), L"cl"))
					{
						// the path to the compiler is often not normalized, and contains wrong casing
						optionsCache[3] = Filesystem::NormalizePath(environmentOption.GetString()).GetString();
						++foundOptions;
					}
					else if (string::Matches(environmentName.GetString(), L"cmd"))
					{
						optionsCache[4] = environmentOption.GetString();
						++foundOptions;
					}

					environment->Release();
				}

				// if the PDB path does not exist, we assume that this file is part of a remote/distributed build.
				// in this case, the code must have been compiled with /Z7, and we won't need a PDB file and can
				// simply ignore this option.
				{
					const FileAttributeCache::Data& cacheData = fileCache.UpdateCacheData(optionsCache[1]);
					if (!cacheData.exists)
					{
						optionsCache[1].clear();
					}
				}

				ImmutableString envSrcPath(string::ToUtf8String(optionsCache[0]));
				ImmutableString envPdbPath(string::ToUtf8String(optionsCache[1]));
				ImmutableString envCompilerWorkingDirectory(string::ToUtf8String(optionsCache[2]));
				ImmutableString envCompilerPath(string::ToUtf8String(optionsCache[3]));
				ImmutableString envCompilerCommandLine(string::ToUtf8String(optionsCache[4]));

				// we cannot compile a compiland without having all the necessary options
				if (foundOptions < 5u)
				{
					if (generateLogs)
					{
						LC_LOG_DEV("Compiland missing info:");
						LC_LOG_INDENT_DEV;
						LC_LOG_DEV("obj: %S (env: %S)", compilandPath.c_str(), environmentCompilandPath.c_str());
						LC_LOG_DEV("src: %s", envSrcPath.c_str());
						LC_LOG_DEV("pdb: %s", envPdbPath.c_str());
						LC_LOG_DEV("cmp: %s", envCompilerPath.c_str());
						LC_LOG_DEV("cmd: %s", envCompilerCommandLine.c_str());
						LC_LOG_DEV("cwd: %s", envCompilerWorkingDirectory.c_str());
					}

					continue;
				}

				// only add compilands that exist on disk.
				// optimization: ignore files stored on optical drives because they cannot be changed anyway.
				{
					// test the compiland path first
					if (Filesystem::GetDriveType(compilandPath.c_str()) == Filesystem::DriveType::OPTICAL)
					{
						if (generateLogs)
						{
							LC_LOG_DEV("Ignoring file %S on optical drive", compilandPath.c_str());
						}

						continue;
					}

					FileAttributeCache::Data cacheData = fileCache.UpdateCacheData(compilandPath);
					if (!cacheData.exists)
					{
						if (generateLogs)
						{
							LC_LOG_DEV("File %S does not exist, trying next candidate", compilandPath.c_str());
						}

						// try the absolute compiland environment directory combined with the compiland's filename.
						// optimization: only do this if we were able to extract the compiland environment.
						std::wstring testPath;
						bool testFileExists = (environmentCompilandPath.length() != 0);
						if (testFileExists)
						{
							testPath = Filesystem::GetDirectory(environmentCompilandPath.c_str()).GetString();
							testPath += L"\\";
							testPath += Filesystem::GetFilename(compilandPath.c_str()).GetString();

							if (Filesystem::GetDriveType(testPath.c_str()) == Filesystem::DriveType::OPTICAL)
							{
								if (generateLogs)
								{
									LC_LOG_DEV("Ignoring file %S on optical drive", testPath.c_str());
								}

								continue;
							}

							cacheData = fileCache.UpdateCacheData(testPath);
							if (!cacheData.exists)
							{
								if (generateLogs)
								{
									LC_LOG_DEV("File %S does not exist, trying final candidate", testPath.c_str());
								}
							}

							testFileExists = cacheData.exists;
						}
						
						if (!testFileExists)
						{
							// try the compiler working directory plus compiland path.
							// optimization: this can only work if the compiland path is relative
							if (Filesystem::IsRelativePath(compilandPath.c_str()))
							{
								testPath = optionsCache[2];
								testPath += L"\\";
								testPath += compilandPath;

								if (Filesystem::GetDriveType(testPath.c_str()) == Filesystem::DriveType::OPTICAL)
								{
									if (generateLogs)
									{
										LC_LOG_DEV("Ignoring file %S on optical drive", testPath.c_str());
									}

									continue;
								}

								cacheData = fileCache.UpdateCacheData(testPath);
								testFileExists = cacheData.exists;
							}

							if (!testFileExists)
							{
								if (generateLogs)
								{
									LC_LOG_DEV("Compiland does not exist on disk:");
									LC_LOG_INDENT_DEV;
									LC_LOG_DEV("obj: %S (env: %S)", testPath.c_str(), environmentCompilandPath.c_str());
									LC_LOG_DEV("src: %s", envSrcPath.c_str());
									LC_LOG_DEV("pdb: %s", envPdbPath.c_str());
									LC_LOG_DEV("cmp: %s", envCompilerPath.c_str());
									LC_LOG_DEV("cmd: %s", envCompilerCommandLine.c_str());
									LC_LOG_DEV("cwd: %s", envCompilerWorkingDirectory.c_str());
								}

								continue;
							}
						}

						compilandPath = testPath;
					}

					// ignore compilands that are newer than the module itself.
					// we cannot use those for reconstructing symbol information, because they weren't linked into the executable.
					if (cacheData.lastModificationTime > provider->lastModificationTime)
					{
						LC_WARNING_USER("Ignoring compiland %S because it is newer than the module it belongs to.", compilandPath.c_str());
						continue;
					}
				}

				const std::wstring normalizedCompilandPath = Filesystem::NormalizePath(compilandPath.c_str()).GetString();

				// check for incompatible compiler/linker settings depending on enabled features
				const bool splitAmalgamatedFiles = (splitAmalgamatedFilesThreshold > 1u);
				if (splitAmalgamatedFiles)
				{
					if (compilerOptions::UsesMinimalRebuild(envCompilerCommandLine.c_str()))
					{
						LC_ERROR_USER("Compiland %S uses compiler option \"Enable Minimal Rebuild (/Gm)\" which is incompatible with automatic splitting of amalgamated/unity files. Recompilation of this file will most likely be skipped by the compiler.", compilandPath.c_str());
					}
				}

				// whole program optimization/link-time code generation is not supported because the corresponding COFF
				// cannot be read. additionally, check whether compilands were compiled with /hotpatch option and inform
				// the user if not.
				{
					bool usesLTCG = false;
					bool isHotpatchable = false;
					const types::vector<IDiaSymbol*>& details = dia::GatherChildSymbols(diaSymbol, SymTagCompilandDetails);
					const size_t detailCount = details.size();
					for (size_t j = 0u; j < detailCount; ++j)
					{
						IDiaSymbol* detail = details[j];
						if (dia::WasCompiledWithLTCG(detail))
						{
							usesLTCG = true;
						}
						if (dia::WasCompiledWithHotpatch(detail))
						{
							isHotpatchable = true;
						}
						detail->Release();
					}

					if (!isHotpatchable)
					{
						LC_WARNING_USER("Compiland %S was not compiled with Hotpatch support, some functions might not be patchable", compilandPath.c_str());
					}

					if (usesLTCG)
					{
						LC_ERROR_USER("Compiland %S was compiled with unsupported option \"Whole Program Optimization (/GL)\" and cannot be analyzed", compilandPath.c_str());
						continue;
					}
				}

				const bool isPartOfLibrary = DoesCompilandBelongToLibrary(dia::GetSymbolLibraryName(diaSymbol));
				Compiland* compiland = LC_NEW(&g_compilandAllocator, Compiland)
				{
					string::ToUtf8String(compilandPath),
					envSrcPath,
					envPdbPath,
					envCompilerPath,
					envCompilerCommandLine,
					envCompilerWorkingDirectory,
					ImmutableString(""),							// amalgamation .obj path
					nullptr,										// file indices
					// BEGIN EPIC MOD
					GetCompilandIdFromPath(normalizedCompilandPath),	// unique ID
					// END EPIC MOD
					Compiland::Type::SINGLE_FILE,					// type of file
					isPartOfLibrary,								// isPartOfLibrary
					false											// wasRecompiled
				};

				// find all source files that contributed to this compiland.
				// note that DIA has en enumerator for going through all IDiaSourceFiles and grabbing the compilands from
				// there, but doing it like this is much faster.
				if (generateLogs)
				{
					LC_LOG_DEV("Adding compiland %S", compilandPath.c_str());
					LC_LOG_INDENT_DEV;
				}

				// prepare the filename-only part of the source file, the full path of the source file is then
				// extracted from the dependencies. compiland dependencies are always given with their full paths.
				// we cannot fully rely on the filename given in the compiland environment, because it will point to
				// remote filenames in distributed builds.
				// if we find a file dependency matching the given source file, we take that one instead to get
				// full absolute file paths.
				// optimization: ignore all files on optical drives because they cannot be changed anyway
				const std::wstring srcFileOnlyLowercase = string::ToLower(Filesystem::GetFilename(optionsCache[0].c_str()).GetString());

				const ObjPath objPath(string::ToUtf8String(normalizedCompilandPath));

				if (trackObjOnly)
				{
					// we are only interested in tracking .obj files. we will never be able to recompile files
					// and we don't know anything about source files, dependencies, etc.
					// but we still use our dependency tracking system by letting each .obj depend on itself.
					if (Filesystem::GetDriveType(normalizedCompilandPath.c_str()) == Filesystem::DriveType::OPTICAL)
					{
						if (generateLogs)
						{
							LC_LOG_DEV("Ignoring file %S on optical drive", normalizedCompilandPath.c_str());
						}

						continue;
					}

					const FileAttributeCache::Data& cacheData = fileCache.UpdateCacheData(normalizedCompilandPath);
					if (cacheData.exists)
					{
						AddFileDependency(compilandDb, objPath, objPath, cacheData.lastModificationTime);
						compilandDb->compilands.insert(std::make_pair(objPath, compiland));
						compilandDb->compilandNameToObjOnDisk.emplace(string::ToUtf8String(diaCompilandPath.GetString()), objPath);
					}

					continue;
				}

				// fetch all include files for this compiland
				types::vector<IDiaSourceFile*> sourceFiles = dia::GatherCompilandFiles(provider->diaSession, diaSymbol);
				const size_t fileCount = sourceFiles.size();

				types::vector<std::wstring> includeFilePaths;
				includeFilePaths.reserve(fileCount);

				for (size_t j = 0u; j < fileCount; ++j)
				{
					IDiaSourceFile* sourceFile = sourceFiles[j];
					const dia::SymbolName& filename = dia::GetSymbolFilename(sourceFile);
					const std::wstring wideFilename(filename.GetString());

					includeFilePaths.push_back(wideFilename);

					sourceFile->Release();

					// repair paths to remote machines, e.g. when using FASTBuild.
					// we are not allowed to normalize this filename. otherwise, normalizing will resolve symbolic links
					// and virtual drives, which means that files compiled by Live++ will use a different path than
					// the original compilands.
					// this could break when including header files that use #pragma once.
					const std::wstring lowercaseFilename = string::ToLower(wideFilename);
					const std::wstring lowercaseFilenameOnly = Filesystem::GetFilename(lowercaseFilename.c_str()).GetString();
					if (string::Matches(lowercaseFilenameOnly.c_str(), srcFileOnlyLowercase.c_str()))
					{
						// replace the source path with the full absolute path to make remote builds work.
						// we convert the path to lower case to be absolutely sure it is at least consistent across PDBs of
						// patches, executables and DLLs, given the fact that we cannot normalize it.
						// note that when compiling with Clang, there may be more than one file that matches the filename,
						// e.g. one file pointing to the local path, and one file pointing to a remote path.
						// therefore, we need to make sure to only take files that really exist on disk.
						const FileAttributeCache::Data& cacheData = fileCache.UpdateCacheData(wideFilename);
						if (cacheData.exists)
						{
							compiland->srcPath = string::ToUtf8String(lowercaseFilename);
						}
					}
				}

				amalgamation::SplitAmalgamatedCompilands(splitAmalgamatedFilesThreshold,
					compilandDb, compiland, includeFilePaths, diaCompilandPath.GetString(), compilandPath, fileCache, generateLogs, provider->lastModificationTime);
			}
		}

		// workaround for Incredibuild hackery. Incredibuild builds PCHs once on the main machine, and then copies them to
		// remote machines, which is illegal to start with. it then compiles translation units into different PDBs on
		// different agents. normally, this would yield C2858, because translation units need to use the same PDB the PCH
		// was built with.
		// I suspect Incredibuild patches the path stored in the PCH in order to make this compile. this, in turn, leads
		// to compilands having different PDBs stored in the environment than what the PCH used, which ultimately leads
		// to a C2858 when Live++ tries to compile the file.
		if (forcePchPdbs && !trackObjOnly)
		{
			// first find all PCH compilands and the names of the PCHs they create.
			// store this in a map for faster lookup.
			struct Hasher
			{
				inline size_t operator()(const std::string& key) const
				{
					return XXH32(key.c_str(), key.length() * sizeof(char), 0u);
				}
			};

			types::unordered_map_with_hash<std::string, ImmutableString, Hasher> pchPathToPdbPath;

			for (auto it : compilandDb->compilands)
			{
				symbols::Compiland* compiland = it.second;
				if (compilerOptions::CreatesPrecompiledHeader(compiland->commandLine.c_str()))
				{
					const std::string pchPath = compilerOptions::GetPrecompiledHeaderPath(compiland->commandLine.c_str());
					if (pchPath.length() != 0u)
					{
						pchPathToPdbPath.emplace(pchPath, compiland->pdbPath);
						LC_LOG_DEV("Found PCH %s using PDB %s", pchPath.c_str(), compiland->pdbPath.c_str());
					}
				}
			}

			// now walk all compilands. for each one that uses a PCH, assign the same PDB as the PCH uses.
			for (auto it : compilandDb->compilands)
			{
				Compiland* compiland = it.second;
				if (compilerOptions::UsesPrecompiledHeader(compiland->commandLine.c_str()))
				{
					const std::string pchPath = compilerOptions::GetPrecompiledHeaderPath(compiland->commandLine.c_str());
					if (pchPath.length() != 0u)
					{
						const auto findIt = pchPathToPdbPath.find(pchPath);
						if (findIt != pchPathToPdbPath.end())
						{
							const ImmutableString pchPdbPath = findIt->second;
							const ImmutableString& objPath = it.first;
							LC_LOG_DEV("Forcing compiland %s to use PCH PDB %s", objPath.c_str(), pchPdbPath.c_str());
							compiland->pdbPath = ImmutableString(pchPdbPath.c_str());
						}
					}
				}
			}
		}

		LC_LOG_TELEMETRY("Compiland filecache touched %zu files", fileCache.GetEntryCount());

		return compilandDb;
	}


	LibraryDB* GatherLibraries(const DiaCompilandDB* diaCompilandDb)
	{
		telemetry::Scope telemetryScope("Gathering libraries");

		// the way we gather libraries may look convoluted, but it is *absolutely paramount* to
		// store the libraries in the order they appear in the PDB, because that also is the order
		// they were linked into the executable.
		// we need to use the exact same order, otherwise linking of weak external symbols might
		// fail when recompiling (e.g. overwritten new and delete operators).
		LibraryDB* libraryDb = new LibraryDB;
		libraryDb->libraries.reserve(64u);

		types::StringSet foundLibraries;
		foundLibraries.reserve(64u);

		const size_t count = diaCompilandDb->symbols.size();
		for (size_t i = 0u; i < count; ++i)
		{
			IDiaSymbol* diaSymbol = diaCompilandDb->symbols[i];

			// check if this file is part of a library
			const dia::SymbolName& libraryName = dia::GetSymbolLibraryName(diaSymbol);
			if (DoesCompilandBelongToLibrary(libraryName))
			{
				ImmutableString lib = string::ToUtf8String(libraryName.GetString());

				// try inserting the library into the set.
				// only add new libs to the database. this ensures that libs are stored in
				// the order of insertion (which would not be guaranteed by the std::set).
				const auto insertIt = foundLibraries.emplace(lib);
				if (insertIt.second)
				{
					// data was inserted, so add it to the database
					libraryDb->libraries.emplace_back(std::move(lib));
				}
			}
		}

		return libraryDb;
	}


	IDiaSymbol* FindLinkerSymbol(const DiaCompilandDB* diaCompilandDb)
	{
		telemetry::Scope telemetryScope("Finding linker symbol");

		const size_t count = diaCompilandDb->symbols.size();
		for (size_t i = 0u; i < count; ++i)
		{
			IDiaSymbol* diaSymbol = diaCompilandDb->symbols[i];

			// check if this is a linker symbol
			const dia::SymbolName& compilandPath = dia::GetSymbolName(diaSymbol);
			const bool isLinkerInfo = string::Matches(compilandPath.GetString(), L"* Linker *");
			if (isLinkerInfo)
			{
				// linker symbol and DIA compiland DB will both be freed
				diaSymbol->AddRef();
				return diaSymbol;
			}
		}

		return nullptr;
	}


	LinkerDB* GatherLinker(IDiaSymbol* linkerSymbol)
	{
		telemetry::Scope telemetryScope("Gathering linker");

		LinkerDB* linkerDb = new LinkerDB;
		if (!linkerSymbol)
		{
			LC_ERROR_DEV("Invalid linker symbol in GatherLinker");
			return linkerDb;
		}

		// the linker path is used in several places. at least set it to something empty.
		linkerDb->linkerPath = ImmutableString("");

		// find environment options
		unsigned int foundOptions = 0u;
		const types::vector<IDiaSymbol*>& environments = dia::GatherChildSymbols(linkerSymbol, SymTagCompilandEnv);
		const size_t count = environments.size();
		for (size_t i = 0u; i < count; ++i)
		{
			IDiaSymbol* environment = environments[i];
			const dia::SymbolName& environmentName = dia::GetSymbolName(environment);
			const dia::Variant& environmentOption = dia::GetSymbolEnvironmentOption(environment);

			if (string::Matches(environmentName.GetString(), L"pdb"))
			{
				linkerDb->pdbPath = string::ToUtf8String(environmentOption.GetString());
				++foundOptions;
			}
			else if (string::Matches(environmentName.GetString(), L"cwd"))
			{
				// the working directory is optional, we can deal with it not being there
				linkerDb->workingDirectory = string::ToUtf8String(environmentOption.GetString());
			}
			else if (string::Matches(environmentName.GetString(), L"exe"))
			{
				// the path to the linker is often not normalized, and contains wrong casing
				linkerDb->linkerPath = string::ToUtf8String(Filesystem::NormalizePath(environmentOption.GetString()).GetString());
				++foundOptions;
			}
			else if (string::Matches(environmentName.GetString(), L"cmd"))
			{
				// optional linker command line emitted by VS2015 and later
				linkerDb->commandLine = string::ToUtf8String(environmentOption.GetString());
			}

			environment->Release();
		}

		if (foundOptions < 2u)
		{
			LC_WARNING_USER("Could not find linker environment in PDB. Make sure to generate a full PDB (e.g. using /DEBUG:FULL) and not a partial PDB (e.g. using /DEBUG:FASTLINK)");
		}

		return linkerDb;
	}


	ThunkDB* GatherThunks(IDiaSymbol* linkerSymbol)
	{
		// find thunks generated by incremental linking
		telemetry::Scope telemetryScope("Gathering thunks");

		ThunkDB* thunkDb = new ThunkDB;
		if (!linkerSymbol)
		{
			LC_ERROR_DEV("Invalid linker symbol in GatherThunks");
			return thunkDb;
		}

		const types::vector<IDiaSymbol*>& thunks = dia::GatherChildSymbols(linkerSymbol, SymTagThunk);
		const size_t count = thunks.size();
		thunkDb->thunksFromTableEntryToTarget.reserve(count);
		thunkDb->thunksFromTargetToTableEntries.reserve(count);

		for (size_t i = 0u; i < count; ++i)
		{
			IDiaSymbol* thunk = thunks[i];

			DWORD rva = 0u;
			thunk->get_relativeVirtualAddress(&rva);

			DWORD targetRva = 0u;
			thunk->get_targetRelativeVirtualAddress(&targetRva);

			if ((rva != 0u) && (targetRva != 0u))
			{
				thunkDb->thunksFromTableEntryToTarget.emplace(rva, targetRva);
				thunkDb->thunksFromTargetToTableEntries[targetRva].push_back(rva);
			}

			thunk->Release();
		}

		return thunkDb;
	}


	ImageSectionDB* GatherImageSections(IDiaSymbol* linkerSymbol)
	{
		// find image sections
		telemetry::Scope telemetryScope("Gathering image sections");

		ImageSectionDB* imageSectionDb = new ImageSectionDB;
		if (!linkerSymbol)
		{
			LC_ERROR_DEV("Invalid linker symbol in GatherImageSections");
			return imageSectionDb;
		}

		const types::vector<IDiaSymbol*>& sections = dia::GatherChildSymbols(linkerSymbol, SymTagCoffGroup);
		const size_t count = sections.size();
		imageSectionDb->sectionNames.reserve(count);
		imageSectionDb->sectionsByName.reserve(count);
		imageSectionDb->sections.reserve(count);

		for (size_t i = 0u; i < count; ++i)
		{
			IDiaSymbol* diaSection = sections[i];
			const dia::SymbolName& diaSectionName = dia::GetSymbolName(diaSection);

			const ImmutableString sectionName = string::ToUtf8String(diaSectionName.GetString());
			const ImageSection section = { static_cast<uint32_t>(i), dia::GetSymbolRVA(diaSection), dia::GetSymbolSize(diaSection) };

			imageSectionDb->sectionNames.push_back(sectionName);
			imageSectionDb->sections.push_back(section);
			imageSectionDb->sectionsByName.emplace(std::move(sectionName), std::move(section));

			diaSection->Release();
		}

		// sort sections by RVA
		std::sort(imageSectionDb->sections.begin(), imageSectionDb->sections.end(), &SortImageSectionByAscendingRVA);

		return imageSectionDb;
	}


	DynamicInitializerDB GatherDynamicInitializers(const Provider* provider, const executable::Image* image, const executable::ImageSectionDB* imageSections, const ImageSectionDB* imageSectionDb, const ContributionDB* contributionDb, const CompilandDB* compilandDb, const CoffCache<coff::CoffDB>* coffCache, SymbolDB* symbolDb)
	{
		telemetry::Scope telemetryScope("Gathering dynamic initializers");

		DynamicInitializerDB initializerDb;

		// note that x86 and x64 have different name mangling schemes for these symbols
		const symbols::Symbol* firstInitializerSymbol = symbols::FindSymbolByName(symbolDb, ImmutableString(LC_IDENTIFIER("__xc_a")));
		const symbols::Symbol* lastInitializerSymbol = symbols::FindSymbolByName(symbolDb, ImmutableString(LC_IDENTIFIER("__xc_z")));

		if (!firstInitializerSymbol)
		{
			LC_ERROR_DEV("Cannot find start of dynamic initializer range");
			return initializerDb;
		}

		if (!lastInitializerSymbol)
		{
			LC_ERROR_DEV("Cannot find end of dynamic initializer range");
			return initializerDb;
		}

		LC_LOG_DEV("Found dynamic initializer range from 0x%X to 0x%X", firstInitializerSymbol->rva, lastInitializerSymbol->rva);
		LC_LOG_INDENT_DEV;

		// this is defined in the CRT, which also defines all the special sections
		typedef _PVFV DynamicInitializer;

		// the first symbol is always __xc_a, which we are not interested in.
		// similarly, the last symbol is always __xc_z, which we are also not interested in.
		const uint32_t firstRva = firstInitializerSymbol->rva + sizeof(DynamicInitializer);
		const uint32_t lastRva = lastInitializerSymbol->rva - sizeof(DynamicInitializer);

		// find sections that hold first and last symbol
		const symbols::ImageSection* firstSection = symbols::FindImageSectionByRVA(imageSectionDb, firstRva);
		if (!firstSection)
		{
			LC_ERROR_USER("Could not find image section containing dynamic initializers.\nThis will lead to constructors of global and static variables being called again for the next patch, likely leading to unexpected behaviour.");
			return initializerDb;
		}

		const symbols::ImageSection* lastSection = symbols::FindImageSectionByRVA(imageSectionDb, lastRva);
		if (!lastSection)
		{
			LC_ERROR_USER("Could not find image section containing dynamic initializers.\nThis will lead to constructors of global and static variables being called again for the next patch, likely leading to unexpected behaviour.");
			return initializerDb;
		}

		const size_t maxInitializerCount = (lastSection->rva + lastSection->size - firstSection->rva) / sizeof(DynamicInitializer);
		initializerDb.dynamicInitializers.reserve(maxInitializerCount);

		// walk through these sections, finding their contributions from COFF files.
		auto contributionIt = std::lower_bound(contributionDb->contributions.begin(), contributionDb->contributions.end(), firstRva, &ContributionHasLowerRvaLowerBound);
		for (const symbols::ImageSection* section = firstSection; section <= lastSection; ++section)
		{
			const uint32_t sectionStart = section->rva;
			const uint32_t sectionEnd = sectionStart + section->size;
			const ImmutableString& sectionName = symbols::GetImageSectionName(imageSectionDb, section);
			LC_LOG_DEV("Section %s from 0x%X to 0x%X", sectionName.c_str(), sectionStart, sectionEnd);
			LC_LOG_INDENT_DEV;

			types::StringMap<uint32_t> unknownInitializers;
			unknownInitializers.reserve(64u);
			while (contributionIt != contributionDb->contributions.end())
			{
				const symbols::Contribution* contribution = *contributionIt;

				// make sure there are no gaps between sections
				if (contribution->rva < sectionStart)
				{
					continue;
				}

				// is this contribution still part of the current section?
				if (contribution->rva >= sectionEnd)
				{
					break;
				}

				const ImmutableString& compilandName = symbols::GetContributionCompilandName(contributionDb, contribution);
				LC_LOG_DEV("Contribution from file %s at RVA 0x%X with size %d", compilandName.c_str(), contribution->rva, contribution->size);
				++contributionIt;

				// fetch the section from the compiland that contributed it.
				// note that we probably don't have a COFF database for "external" files, e.g. coming from vendor and platform libs.
				const coff::CoffDB* coffDb = coffCache->Lookup(compilandName);
				if (coffDb)
				{
					// find the CRT section with that name and size
					const std::vector<const coff::CrtSection*>& crtSections = coff::FindMatchingCrtSections(coffDb, sectionName, contribution->size);
					if (crtSections.size() == 1u)
					{
						// fast path: exactly one matching section was found, extract symbols directly from there
						const coff::CrtSection* crtSection = crtSections[0];
						const size_t count = crtSection->symbols.size();
						for (size_t i = 0u; i < count; ++i)
						{
							const coff::Symbol* symbol = crtSection->symbols[i];
							const ImmutableString& symbolName = coff::GetSymbolName(coffDb, symbol);
							const uint32_t sectionRelativeRva = symbol->rva - crtSection->rawDataRva;
							const uint32_t rva = contribution->rva + sectionRelativeRva;
							LC_LOG_DEV("Found dynamic initializer %s at 0x%X (fast path)", symbolName.c_str(), rva);

							// note that symbols coming from COFFs have already been disambiguated, so we can
							// directly use their name
							symbols::Symbol* newSymbol = CreateNewSymbol(symbolName, rva, symbolDb);

							initializerDb.dynamicInitializers.push_back(newSymbol);
						}
					}
					else
					{
						// slow path: unfortunately, no unambiguous CRT section could be found, so we have to use the
						// PDB provider in order to reconstruct dynamic initializers. this is not as fast as walking the
						// CRT section directly, and introduces additional complexity.
						// when trying to simply get the symbol at the RVAs in the contribution's range, the PDB often
						// does *not* hold a symbol at that address, making it impossible to find all "$initializer$" symbols
						// that way.
						// however, the PDB *does* store addresses for all "?__E" dynamic initializer functions. these are
						// the functions that are being pointed at by all of the "$initializer$" symbols.
						// so rather than trying to find the "$initializer$" symbols directly, we do the following:
						// - fetch the address the "$initializer$" symbol in question points to
						// - get the symbol and its name at that address (this will always be a "?__E" dynamic initializer function)
						// - scan all symbols of possible sections to check which one has a relocation to this function
						// - the symbol with this relocation is our "$initializer$" symbol
						size_t symbolIndex = 0u;
						for (uint32_t initializerRva = contribution->rva; initializerRva < contribution->rva + contribution->size; initializerRva += sizeof(DynamicInitializer), ++symbolIndex)
						{
							const symbols::Symbol* knownSymbol = symbols::FindSymbolByRVA(symbolDb, initializerRva);
							if (knownSymbol)
							{
								LC_LOG_DEV("Known dynamic initializer %s at 0x%X (slow path)", knownSymbol->name.c_str(), initializerRva);
								initializerDb.dynamicInitializers.push_back(knownSymbol);

								continue;
							}

							// our "$initializer$" symbol sits at initializerRva, so find the address of the dynamic initializer
							// symbol it points to.
#if LC_64_BIT
							const uint64_t dynamicInitializerAddress = executable::ReadFromImage<uint64_t>(image, imageSections, initializerRva);
#else
							const uint32_t dynamicInitializerAddress = executable::ReadFromImage<uint32_t>(image, imageSections, initializerRva);
#endif

							// the relocations from "$initializer$" to a dynamic initializer are always absolute, so its
							// easy to reconstruct the dynamic initializer's RVA.
							const uint32_t dynamicInitializerRva = static_cast<uint32_t>(dynamicInitializerAddress - executable::GetPreferredBase(image));

							// using the PDB, we can find the dynamic initializer function with this RVA
							IDiaSymbol* dynamicInitializerSymbol = dia::FindSymbolByRVA(provider->diaSession, dynamicInitializerRva);
							if (dynamicInitializerSymbol)
							{
								// we now know the RVA and name of the dynamic initializer function.
								// scan relocations of symbols of all potential CRT sections to find the relocation
								// that points to this dynamic initializer function.
								std::wstring diaSymbolName(dia::GetSymbolName(dynamicInitializerSymbol).GetString());

								// NOTE: when comparing/matching undecorated names, names stored for DIA symbols are normally structured
								// differently than the undecorated names for COFF symbols when using nameMangling::UndecorateSymbol
								// without flags.
								// however, using the correct (undocumented) flags yields the same name as stored in DIA.
								size_t crtSectionSymbolCount = 0u;
								const size_t crtSectionCount = crtSections.size();
								for (size_t i = 0u; i < crtSectionCount; ++i)
								{
									const coff::CrtSection* crtSection = crtSections[i];
									crtSectionSymbolCount += crtSection->symbols.size();

									// when using Clang, .CRT sections often have no symbols assigned in the COFF symbol table
									if (crtSection->symbols.size() == 0u)
									{
										continue;
									}
									if (symbolIndex >= crtSection->symbols.size())
									{
										continue;
									}

									const coff::Symbol* coffSymbol = crtSection->symbols[symbolIndex];
									const size_t relocationCount = coffSymbol->relocations.size();

									// "$initializer$" symbols in .CRT$XCU sections should always have only one relocation
									// to the dynamic initializer function.
									if (relocationCount == 1u)
									{
										const coff::Relocation* relocation = coffSymbol->relocations[0];
										const ImmutableString& dstSymbolName = coff::GetRelocationDstSymbolName(coffDb, relocation);

										// note that the name of the DIA symbol is the undecorated name, but the COFF
										// stores mangled names, so undecorate the COFF name first.
										std::wstring dstSymbolUndecoratedName = string::ToWideString(UndecorateSymbolName(dstSymbolName));
										if (string::Contains(dstSymbolUndecoratedName.c_str(), diaSymbolName.c_str()))
										{
											// this relocation points to the dynamic initializer function, which means we
											// found the source "$initializer$" symbol
											const ImmutableString& coffSymbolName = coff::GetSymbolName(coffDb, coffSymbol);
											LC_LOG_DEV("Found dynamic initializer %s at 0x%X (points to %s at 0x%X) (slow path)",
												coffSymbolName.c_str(),
												initializerRva,
												dstSymbolName.c_str(),
												dynamicInitializerRva);

											symbols::Symbol* newSymbol = CreateNewSymbol(coffSymbolName, initializerRva, symbolDb);

											initializerDb.dynamicInitializers.push_back(newSymbol);

											goto symbolFound;
										}
									}
								}

								// only output an error in case we had a real chance to find the initializer. when code is compiled with Clang,
								// there will be no symbols in the CRT sections.
								if (crtSectionSymbolCount != 0u)
								{
									LC_ERROR_DEV("Could not find dynamic initializer symbol %S for compiland %s", diaSymbolName.c_str(), compilandName.c_str());
								}

							symbolFound:
								dynamicInitializerSymbol->Release();
							}
							else
							{
								LC_ERROR_DEV("Could not find DIA dynamic initializer symbol at 0x%X in compiland %s", dynamicInitializerRva, compilandName.c_str());
							}
						}
					}
				}
				else
				{
					const Compiland* compiland = FindCompiland(compilandDb, compilandName);
					if (compiland)
					{
						// we don't have a COFF database for this compiland. the compiland is part of the module and can be
						// live coded, but hasn't been reconstructed because it is not part of this recompilation cycle.
						// it is safe to ignore these initializers, but we take what we already know.
						for (uint32_t initializerRva = contribution->rva; initializerRva < contribution->rva + contribution->size; initializerRva += sizeof(DynamicInitializer))
						{
							const symbols::Symbol* knownSymbol = symbols::FindSymbolByRVA(symbolDb, initializerRva);
							if (knownSymbol)
							{
								LC_LOG_DEV("Known dynamic initializer %s at 0x%X (compiland, no DB)", knownSymbol->name.c_str(), initializerRva);
								initializerDb.dynamicInitializers.push_back(knownSymbol);
							}
						}
					}
					else
					{
						// we don't have a COFF database for this compiland. the compiland is not part of the module and
						// must be part of e.g. an external library.
						// in this case, the name of an initializer's symbol doesn't really matter, as long as it is unique
						// and the same during a live code session.
						// the reason for that is that these files cannot be changed and recompiled anyway, but will only be used for
						// linking. therefore, the COFF used for linking is always the same, and we only need to assign unique
						// names for these initializers.

						// try adding a new counter for this compiland. if this succeeds, the counter will start at zero.
						// if not, we get the existing counter's value.
						const auto counterIt = unknownInitializers.emplace(compilandName, 0u);
						uint32_t& compilandCounter = counterIt.first->second;
						for (uint32_t rva = contribution->rva; rva < contribution->rva + contribution->size; rva += sizeof(DynamicInitializer))
						{
							// unique names are generated by using a per-compiland increasing counter, as well as appending the
							// name (or rather unique ID) of the compiland the symbol originated from.
							// keep the name short to make use of the short string optimization.
							std::string symbolName("$di$");
							symbolName += std::to_string(compilandCounter);
							symbolName += coff::GetCoffSuffix();
							symbolName += std::to_string(uniqueId::Generate(string::ToWideString(compilandName)));

							ImmutableString fullPath(symbolName.c_str());
							LC_LOG_DEV("Found dynamic initializer %s at 0x%X", fullPath.c_str(), rva);

							symbols::Symbol* newSymbol = CreateNewSymbol(fullPath, rva, symbolDb);

							initializerDb.dynamicInitializers.push_back(newSymbol);

							++compilandCounter;
						}
					}
				}
			}
		}

		return initializerDb;
	}


	void DestroyLinkerSymbol(IDiaSymbol* symbol)
	{
		if (symbol)
		{
			symbol->Release();
		}
	}


	void DestroyDiaCompilandDB(DiaCompilandDB* db)
	{
		const size_t count = db->symbols.size();
		for (size_t i = 0u; i < count; ++i)
		{
			IDiaSymbol* symbol = db->symbols[i];
			symbol->Release();
		}

		delete db;
	}


	void DestroyModuleDB(ModuleDB* db)
	{
		delete db;
	}


	void DestroyCompilandDB(CompilandDB* db)
	{
		for (auto it = db->compilands.begin(); it != db->compilands.end(); ++it)
		{
			Compiland* compiland = it->second;
			delete compiland->sourceFiles;
			LC_FREE(&g_compilandAllocator, compiland, sizeof(Compiland));
		}

		for (auto it = db->dependencies.begin(); it != db->dependencies.end(); ++it)
		{
			Dependency* dependency = it->second;
			LC_FREE(&g_dependencyAllocator, dependency, sizeof(Dependency));
		}

		delete db;
	}


	void MergeCompilandsAndDependencies(CompilandDB* existingDb, CompilandDB* mergedDb)
	{
		// merge compilands
		for (auto compilandIt = mergedDb->compilands.begin(); compilandIt != mergedDb->compilands.end(); ++compilandIt)
		{
			const symbols::FilePath& filePath = compilandIt->first;
			symbols::Compiland* newCompiland = compilandIt->second;

			auto it = existingDb->compilands.find(filePath);
			if (it == existingDb->compilands.end())
			{
				// this compiland is not in the DB yet, move it over
				existingDb->compilands.emplace(filePath, newCompiland);
			}
			else
			{
				// transfer ownership of compiland source files
				it->second->sourceFiles = newCompiland->sourceFiles;
			}

			newCompiland->sourceFiles = nullptr;
		}

		// merge/update dependencies
		for (auto compilandIt = mergedDb->dependencies.begin(); compilandIt != mergedDb->dependencies.end(); ++compilandIt)
		{
			const symbols::FilePath& filePath = compilandIt->first;
			symbols::Dependency* newDependency = compilandIt->second;

			// get dependency entry in existing database
			auto it = existingDb->dependencies.find(filePath);
			if (it != existingDb->dependencies.end())
			{
				// merge and update dependent .obj paths.
				// do not update modification time, because the file might have been changed while we were still compiling it.
				symbols::Dependency* existingDependency = it->second;

				types::StringSet paths;
				paths.insert(existingDependency->objPaths.begin(), existingDependency->objPaths.end());
				paths.insert(newDependency->objPaths.begin(), newDependency->objPaths.end());

				existingDependency->objPaths.clear();
				for (auto pathIt = paths.begin(); pathIt != paths.end(); ++pathIt)
				{
					const ImmutableString& obj = *pathIt;
					existingDependency->objPaths.push_back(std::move(obj));
				}
			}
			else
			{
				// this compiland is not in the DB yet, move it over
				existingDb->dependencies.emplace(filePath, newDependency);
			}
		}
	}


	void MarkCompilandAsRecompiled(Compiland* compiland)
	{
		compiland->wasRecompiled = true;
	}


	void ClearCompilandAsRecompiled(Compiland* compiland)
	{
		compiland->wasRecompiled = false;
	}


	bool IsCompilandRecompiled(const Compiland* compiland)
	{
		return compiland->wasRecompiled;
	}


	Compiland* FindCompiland(CompilandDB* db, const ObjPath& objPath)
	{
		auto it = db->compilands.find(objPath);
		if (it != db->compilands.end())
		{
			return it->second;
		}

		return nullptr;
	}


	const Compiland* FindCompiland(const CompilandDB* db, const ObjPath& objPath)
	{
		const auto it = db->compilands.find(objPath);
		if (it != db->compilands.end())
		{
			return it->second;
		}

		return nullptr;
	}


	AmalgamatedCompiland* FindAmalgamatedCompiland(CompilandDB* db, const ObjPath& objPath)
	{
		auto it = db->amalgamatedCompilands.find(objPath);
		if (it != db->amalgamatedCompilands.end())
		{
			return it->second;
		}

		return nullptr;
	}


	const AmalgamatedCompiland* FindAmalgamatedCompiland(const CompilandDB* db, const ObjPath& objPath)
	{
		const auto it = db->amalgamatedCompilands.find(objPath);
		if (it != db->amalgamatedCompilands.end())
		{
			return it->second;
		}

		return nullptr;
	}


	bool IsAmalgamation(const Compiland* compiland)
	{
		return (compiland->type == Compiland::Type::AMALGAMATION);
	}


	bool IsPartOfAmalgamation(const Compiland* compiland)
	{
		return (compiland->type == Compiland::Type::PART_OF_AMALGAMATION);
	}


	const symbols::Symbol* FindSymbolByName(const SymbolDB* db, const ImmutableString& name)
	{
		const auto it = db->symbolsByName.find(name);
		if (it != db->symbolsByName.end())
		{
			return it->second;
		}

		return nullptr;
	}


	const symbols::Symbol* FindSymbolByRVA(const SymbolDB* db, uint32_t rva)
	{
		const auto it = db->symbolsByRva.find(rva);
		if (it != db->symbolsByRva.end())
		{
			return it->second;
		}

		return nullptr;
	}


	const ImageSection* FindImageSectionByName(const ImageSectionDB* db, const ImmutableString& name)
	{
		const auto it = db->sectionsByName.find(name);
		if (it != db->sectionsByName.end())
		{
			return &it->second;
		}

		return nullptr;
	}


	const ImageSection* FindImageSectionByRVA(const ImageSectionDB* db, uint32_t rva)
	{
		auto it = std::upper_bound(db->sections.begin(), db->sections.end(), rva, &ImageSectionHasLowerRVA);

		// iterator points to first element with greater RVA, hence it can never be the first element
		if (it == db->sections.begin())
		{
			return nullptr;
		}

		--it;
		const ImageSection& section = *it;
		if ((rva >= section.rva) && (rva < section.rva + section.size))
		{
			return &section;
		}

		return nullptr;
	}


	uint32_t FindThunkTargetByRVA(const ThunkDB* db, uint32_t tableEntryRva)
	{
		const auto it = db->thunksFromTableEntryToTarget.find(tableEntryRva);
		if (it != db->thunksFromTableEntryToTarget.end())
		{
			return it->second;
		}

		return 0u;
	}


	types::vector<uint32_t> FindThunkTableEntriesByRVA(const ThunkDB* db, uint32_t targetRva)
	{
		const auto it = db->thunksFromTargetToTableEntries.find(targetRva);
		if (it != db->thunksFromTargetToTableEntries.end())
		{
			return it->second;
		}

		return types::vector<uint32_t>();
	}


	std::string UndecorateSymbolName(const ImmutableString& symbolName)
	{
		const uint32_t coffSuffixPos = coff::FindCoffSuffix(symbolName);
		if (coffSuffixPos != ImmutableString::NOT_FOUND)
		{
			// this name contains the name of the COFF file as suffix.
			// ignore that when undecorating the symbol name.
			char* tempName = static_cast<char*>(_alloca(coffSuffixPos + 1u));
			memcpy(tempName, symbolName.c_str(), coffSuffixPos);
			tempName[coffSuffixPos] = '\0';

			// unfortunately, undecorating symbols with these flags still leaves "__ptr64" in the undecorated name,
			// which is different to how names are stored in the PDB.
			// we therefore remove "__ptr64" ourselves, as the corresponding flag in the "undname.exe" tool cannot be used in our case.
			return string::EraseAll(nameMangling::UndecorateSymbol(tempName, 0x1000u), " __ptr64");
		}

		return string::EraseAll(nameMangling::UndecorateSymbol(symbolName.c_str(), 0x1000u), " __ptr64");
	}


	const Contribution* FindContributionByRVA(const ContributionDB* db, uint32_t rva)
	{
		auto it = std::upper_bound(db->contributions.begin(), db->contributions.end(), rva, &ContributionHasLowerRvaUpperBound);

		// iterator points to first element with greater RVA, hence it can never be the first element
		if (it == db->contributions.begin())
		{
			return nullptr;
		}

		--it;
		const Contribution* contribution = *it;
		if ((rva >= contribution->rva) && (rva < contribution->rva + contribution->size))
		{
			return contribution;
		}

		return nullptr;
	}


	ImmutableString GetContributionCompilandName(const ContributionDB* db, const Contribution* contribution)
	{
		return db->objOnDiskStringTable[contribution->compilandNameIndex];
	}


	const ContributionDB::ContributionsPerCompiland* GetContributionsForCompilandName(const ContributionDB* db, const ImmutableString& compilandName)
	{
		const auto it = db->compilandNameToCompilandNameIndex.find(compilandName);
		if (it != db->compilandNameToCompilandNameIndex.end())
		{
			// we know this compiland
			const uint32_t compilandNameIndex = it->second;
			return GetContributionsForCompilandNameIndex(db, compilandNameIndex);
		}

		return nullptr;
	}


	const ContributionDB::ContributionsPerCompiland* GetContributionsForCompilandNameIndex(const ContributionDB* db, uint32_t compilandNameIndex)
	{
		return &db->contributionsPerCompilandNameIndex[compilandNameIndex];
	}


	const ImmutableString& GetImageSectionName(const ImageSectionDB* db, const ImageSection* imageSection)
	{
		return db->sectionNames[imageSection->nameIndex];
	}


	uint32_t GetCompilandId(const Compiland* compiland, const wchar_t* const objPath, const types::vector<ModifiedObjFile>& modifiedObjFiles)
	{
		// try to find the given .obj path in the array of modified object files to check if there's an original amalgamated object path for it
		for (size_t i = 0u; i < modifiedObjFiles.size(); ++i)
		{
			const ModifiedObjFile& objFile = modifiedObjFiles[i];

			// don't bother checking strings if the amalgamated object path is empty anyway
			if (!objFile.amalgamatedObjPath.empty())
			{
				if (string::Matches(objPath, objFile.objPath.c_str()))
				{
					return uniqueId::Generate(Filesystem::NormalizePath(objFile.amalgamatedObjPath.c_str()).GetString());
				}
			}
		}

		if (compiland)
		{
			// the compiland already exists
			return compiland->uniqueId;
		}
		else
		{
			// BEGIN EPIC MOD - Fixes problems with such things as statics failing to resolve
			return GetCompilandIdFromPath(objPath);
			// END EPIC MD
		}
	}

	ImmutableString TransformAnonymousNamespacePattern(const ImmutableString& immutableString, uint32_t uniqueId)
	{
		// an ANS symbol name is always of the form ?identifier@?A0x12345678, where the hex code following the "@?A0x" part is most likely a hash of the
		// filename the ANS appears in, generated by the compiler.

		// when splitting amalgamated files, we need to make sure that symbols in anonymous namespaces compiled into
		// those files are also found when compiled into single-part files.
		// however, single-part files get assigned a different hash by the compiler, leading to different
		// symbol names for symbols that reside in anonymous namespaces.
		// in order to "correct" this, we generate our own hex identifier for ANS symbols, making sure that this
		// identifier yields the same result for both amalgamated as well as single-part files.
		// this is done by using the uniqueId as identifier, which is the same for amalgamated files as well
		// as their split single-file counterparts.

		// the same issue occurs when compiling files with Clang, because relative paths and absolute paths get assigned
		// different hashes, even though the resulting file is the same.
		const char* str = immutableString.c_str();
		const char* anonNamespaceCursor = string::Find(str, symbolPatterns::ANONYMOUS_NAMESPACE_PATTERN);
		if (!anonNamespaceCursor)
		{
			return immutableString;
		}

		// convert the unique ID to a hex string with 8 characters
		const char hexUniqueId[8u] =
		{
			"0123456789ABCDEF"[(uniqueId >> 28u) & 15u],
			"0123456789ABCDEF"[(uniqueId >> 24u) & 15u],
			"0123456789ABCDEF"[(uniqueId >> 20u) & 15u],
			"0123456789ABCDEF"[(uniqueId >> 16u) & 15u],
			"0123456789ABCDEF"[(uniqueId >> 12u) & 15u],
			"0123456789ABCDEF"[(uniqueId >> 8u) & 15u],
			"0123456789ABCDEF"[(uniqueId >> 4u) & 15u],
			"0123456789ABCDEF"[(uniqueId >> 0u) & 15u]
		};

		const size_t patternLength = strlen(symbolPatterns::ANONYMOUS_NAMESPACE_PATTERN);

		// Clang does not add leading zeros to the hash, so in theory, the hash after ?A0x can be of any length.
		// we therefore cannot simply replace the hash in the original string, but must build a new one from individual pieces.
		std::string newStr;
		newStr.reserve(immutableString.GetLength());

		const char* lastFound = str;
		do
		{
			// skip anonymous namespace pattern
			anonNamespaceCursor += patternLength;

			// copy until current cursor
			newStr.append(lastFound, static_cast<size_t>(anonNamespaceCursor - lastFound));

			// append hex ID
			newStr.append(hexUniqueId, 8u);

			// skip until the end of the anonymous namespace pattern
			anonNamespaceCursor = string::Find(anonNamespaceCursor, "@");
			lastFound = anonNamespaceCursor;

			// the identifier could contain several more anonymous namespace patterns
			anonNamespaceCursor = string::Find(anonNamespaceCursor, symbolPatterns::ANONYMOUS_NAMESPACE_PATTERN);
		}
		while (anonNamespaceCursor);

		newStr.append(lastFound, static_cast<size_t>(str + immutableString.GetLength() - lastFound));

		return ImmutableString(newStr.c_str());
	}


	void TransformAnonymousNamespaceSymbols(SymbolDB* symbolDb, ContributionDB* contributionDb, CompilandDB* compilandDb, const types::vector<ModifiedObjFile>& modifiedObjFiles)
	{
		const size_t count = symbolDb->ansSymbols.size();
		for (size_t i = 0u; i < count; ++i)
		{
			symbols::Symbol* symbol = symbolDb->ansSymbols[i];

			// generate a new name that is based on the (original amalgamated) filename or the compiland ID
			const symbols::Contribution* contribution = symbols::FindContributionByRVA(contributionDb, symbol->rva);
			if (contribution)
			{
				const ImmutableString compilandName = symbols::GetContributionCompilandName(contributionDb, contribution);
				const symbols::Compiland* compiland = symbols::FindCompiland(compilandDb, compilandName);
				if (compiland)
				{
					// remove the symbol with the old name from the database
					symbolDb->symbolsByName.erase(symbol->name);

					const uint32_t uniqueId = GetCompilandId(compiland, string::ToWideString(compilandName).c_str(), modifiedObjFiles);
					symbol->name = TransformAnonymousNamespacePattern(symbol->name, uniqueId);

					// store the symbol with the new name in the database again
					symbolDb->symbolsByName.emplace(symbol->name, symbol);
				}
			}
		}

		// all symbols have been transformed, no need to touch them again next time
		symbolDb->ansSymbols.clear();
	}


	template <typename T, size_t N>
	static inline bool ContainsPatterns(const char* name, const T (&patterns)[N])
	{
		for (size_t i = 0u; i < N; ++i)
		{
			if (string::Contains(name, patterns[i]))
			{
				return true;
			}
		}

		return false;
	}


	template <typename T, size_t N>
	static inline bool StartsWithPatterns(const char* name, const T (&patterns)[N])
	{
		for (size_t i = 0u; i < N; ++i)
		{
			if (string::StartsWith(name, patterns[i]))
			{
				return true;
			}
		}

		return false;
	}


	// BEGIN EPIC MOD
	template <typename T, size_t N>
	static inline const char* StartsWithPatternsEx(const char* name, const T(&patterns)[N])
	{
		for (size_t i = 0u; i < N; ++i)
		{
			const char* out = string::StartsWithEx(name, patterns[i]);
			if (out != nullptr)
			{
				return out;
			}
		}

		return nullptr;
	}


	template <typename T, size_t N>
	static inline const char* StartsWithPatternsEx(const char* name, const char*& outName, size_t& outNameLength, const T(&patterns)[N])
	{
		name = StartsWithPatternsEx(name, patterns);
		if (name == nullptr)
		{
			return nullptr;
		}

		outName = name;
		for (;;)
		{
			char c = *name++;
			if (c == 0)
			{
				return nullptr;
			}
			if (c == '$' || c == '@')
			{
				break;
			}
		}

		outNameLength = name - outName - 1;
		return name;
	}

	template <typename T, size_t N>
	static inline bool MatchWildcardPatterns(const char* name, const T(&patterns)[N])
	{
		for (size_t i = 0u; i < N; ++i)
		{
			if (string::MatchWildcard(name, patterns[i]))
			{
				return true;
			}
		}

		return false;
	}
	// END EPIC MOD


	bool IsPchSymbol(const ImmutableString& symbolName)
	{
		return ContainsPatterns(symbolName.c_str(), symbolPatterns::PCH_SYMBOL_PATTERNS);
	}


	bool IsVTable(const ImmutableString& symbolName)
	{
		return StartsWithPatterns(symbolName.c_str(), symbolPatterns::VTABLE_PATTERNS);
	}


	bool IsRttiObjectLocator(const ImmutableString& symbolName)
	{
		return StartsWithPatterns(symbolName.c_str(), symbolPatterns::RTTI_OBJECT_LOCATOR_PATTERNS);
	}


	bool IsDynamicInitializer(const ImmutableString& symbolName)
	{
		return StartsWithPatterns(symbolName.c_str(), symbolPatterns::DYNAMIC_INITIALIZER_PATTERNS);
	}


	bool IsDynamicAtexitDestructor(const ImmutableString& symbolName)
	{
		return StartsWithPatterns(symbolName.c_str(), symbolPatterns::DYNAMIC_ATEXIT_DESTRUCTORS);
	}


	bool IsPointerToDynamicInitializer(const ImmutableString& symbolName)
	{
		return ContainsPatterns(symbolName.c_str(), symbolPatterns::POINTER_TO_DYNAMIC_INITIALIZER_PATTERNS);
	}


	bool IsStringLiteral(const ImmutableString& symbolName)
	{
		return StartsWithPatterns(symbolName.c_str(), symbolPatterns::STRING_LITERAL_PATTERNS);
	}


	bool IsLineNumber(const ImmutableString& symbolName)
	{
		return ContainsPatterns(symbolName.c_str(), symbolPatterns::LINE_NUMBER_PATTERNS);
	}


	bool IsFloatingPointSseAvxConstant(const ImmutableString& symbolName)
	{
		return StartsWithPatterns(symbolName.c_str(), symbolPatterns::FLOATING_POINT_CONSTANT_PATTERNS);
	}


	bool IsExceptionRelatedSymbol(const ImmutableString& symbolName)
	{
		return ContainsPatterns(symbolName.c_str(), symbolPatterns::EXCEPTION_RELATED_PATTERNS);
	}


	bool IsExceptionClauseSymbol(const ImmutableString& symbolName)
	{
		return StartsWithPatterns(symbolName.c_str(), symbolPatterns::EXCEPTION_CLAUSE_PATTERNS);
	}


	bool IsExceptionUnwindSymbolForDynamicInitializer(const ImmutableString& symbolName)
	{
		return (StartsWithPatterns(symbolName.c_str(), symbolPatterns::EXCEPTION_UNWIND_PATTERNS) &&
			ContainsPatterns(symbolName.c_str(), symbolPatterns::DYNAMIC_INITIALIZER_PATTERNS));
	}


	bool IsRuntimeCheckRelatedSymbol(const ImmutableString& symbolName)
	{
		return ContainsPatterns(symbolName.c_str(), symbolPatterns::RTC_PATTERNS);
	}


	bool IsSdlCheckRelatedSymbol(const ImmutableString& symbolName)
	{
		return ContainsPatterns(symbolName.c_str(), symbolPatterns::SDL_CHECK_PATTERNS);
	}


	bool IsControlFlowGuardRelatedSymbol(const ImmutableString& symbolName)
	{
		return ContainsPatterns(symbolName.c_str(), symbolPatterns::CFG_PATTERNS);
	}


	bool IsImageBaseRelatedSymbol(const ImmutableString& symbolName)
	{
		return ContainsPatterns(symbolName.c_str(), symbolPatterns::IMAGE_BASE_PATTERNS);
	}


	bool IsTlsArrayRelatedSymbol(const ImmutableString& symbolName)
	{
		return ContainsPatterns(symbolName.c_str(), symbolPatterns::TLS_ARRAY_PATTERNS);
	}


	bool IsTlsIndexRelatedSymbol(const ImmutableString& symbolName)
	{
		return ContainsPatterns(symbolName.c_str(), symbolPatterns::TLS_INDEX_PATTERNS);
	}


	bool IsTlsInitRelatedSymbol(const ImmutableString& symbolName)
	{
		return ContainsPatterns(symbolName.c_str(), symbolPatterns::TLS_INIT_PATTERNS);
	}


	bool IsTlsStaticsRelatedSymbol(const ImmutableString& symbolName)
	{
		return ContainsPatterns(symbolName.c_str(), symbolPatterns::TLS_STATICS_PATTERNS);
	}


	bool IsSectionSymbol(const ImmutableString& symbolName)
	{
		return symbolName.c_str()[0] == '.';
	}

	// BEGIN EPIC MOD
	bool IsUERegisterSymbol(const ImmutableString& symbolName, const char*& name, size_t& nameLength)
	{
		const char* remaining = StartsWithPatternsEx(symbolName.c_str(), name, nameLength, symbolPatterns::UE_REGISTER_PATTERNS);
		return remaining != nullptr && string::StartsWith(remaining, "initializer$");
	}

	bool IsUERegisterSymbol(const ImmutableString& symbolName)
	{
		const char* name;
		size_t nameLength;
		return IsUERegisterSymbol(symbolName, name, nameLength);
	}

	bool IsUENoStripSymbol(const ImmutableString& symbolName)
	{
		return IsUEStaticsSymbol(symbolName);
	}

	bool IsUEInitializerSymbol(const ImmutableString& symbolName)
	{
		return IsUERegisterSymbol(symbolName) || IsUEStaticsSymbol(symbolName);
	}

	bool IsUEReversePatchSymbol(const ImmutableString& symbolName)
	{
		return IsUEStaticsSymbol(symbolName);
	}

	bool IsUEStaticsSymbol(const ImmutableString& symbolName)
	{
		return MatchWildcardPatterns(symbolName.c_str(), symbolPatterns::UE_STATICS_BLOCK_PATTERNS);
	}
	// END EPIC MOD

}

// BEGIN EPIC MOD
#include "Windows/HideWindowsPlatformAtomics.h"
// END EPIC MOD

#endif