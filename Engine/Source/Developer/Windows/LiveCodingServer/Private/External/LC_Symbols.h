// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ImmutableString.h"
#include "LC_ProcessTypes.h"
#include "LC_Types.h"
#include "LC_Executable.h"
#include "LC_ChangeNotification.h"
#include "LC_DirectoryCache.h"
// BEGIN EPIC MOD
#include <dia2.h>
// END EPIC MOD

template <typename T> class CoffCache;

namespace coff
{
	struct CoffDB;
}


namespace symbols
{
	struct Symbol
	{
		ImmutableString name;
		uint32_t rva;

		// BEGIN EPIC MOD
		Symbol(ImmutableString _name, uint32_t _rva) : name(_name), rva(_rva) {}
		// END EPIC MOD

		LC_DISABLE_ASSIGNMENT(Symbol);
		LC_DISABLE_COPY(Symbol);
	};

	struct SymbolDB
	{
		// symbols accessibly by name or RVA
		types::StringMap<Symbol*> symbolsByName;
		types::unordered_map<uint32_t, Symbol*> symbolsByRva;

		// a set of symbols that need to be ignored upon reconstruction
		types::StringSet symbolsWithoutRva;

		// public function symbols that can be patched
		types::vector<Symbol*> patchableFunctionSymbols;

		// symbols that contain an anonymous namespace in their mangled name
		types::vector<Symbol*> ansSymbols;
	};

	struct Contribution
	{
		// index of the name of the compiland (.obj file) that contributed this
		uint32_t compilandNameIndex;
		uint32_t rva;
		uint32_t size;
	};

	struct ContributionDB
	{
		// string table that stores the name of a compiland (.obj file).
		// indexed by a contribution's compiland name index.
		// this is loaded initially when gathering contributions.
		types::vector<ImmutableString> originalStringTable;

		// string table that stores the name of an .obj file on disk.
		// indexed by a contribution's compiland name index.
		// this is filled after both contributions and compilands have been loaded.
		types::vector<ImmutableString> objOnDiskStringTable;

		// stores all contributions gathered from a .PDB file, ordered by ascending RVA
		types::vector<Contribution*> contributions;

		// array of all contributions per compiland name index, each ordered by ascending RVA.
		// indexed by a contribution's compiland name index.
		typedef types::vector<Contribution*> ContributionsPerCompiland;
		types::vector<ContributionsPerCompiland> contributionsPerCompilandNameIndex;

		// lookup-table from compiland name to corresponding compiland name index
		types::StringMap<uint32_t> compilandNameToCompilandNameIndex;
	};

	struct CompilandSourceFiles
	{
		// source files used by compilands
		types::vector<ImmutableString> files;
	};

	struct Compiland
	{
		struct Type
		{
			enum Enum : uint8_t
			{
				SINGLE_FILE,
				AMALGAMATION,
				PART_OF_AMALGAMATION
			};
		};

		ImmutableString originalObjPath;		// full path to the original, non-normalized .obj file
		ImmutableString srcPath;				// full path to source file
		ImmutableString pdbPath;				// full path to .pdb file
		ImmutableString compilerPath;			// full path to compiler used to compile the compiland
		ImmutableString commandLine;			// command line arguments passed to the compiler
		ImmutableString workingDirectory;		// full path to working directory
		ImmutableString amalgamationPath;		// full path to the amalgamation .obj in case this is part of an amalgamation
		CompilandSourceFiles* sourceFiles;
		uint32_t uniqueId;
		Type::Enum type;
		bool isPartOfLibrary;
		bool wasRecompiled;

		// BEGIN EPIC MOD
		Compiland(ImmutableString _originalObjPath, ImmutableString _srcPath, ImmutableString _pdbPath, ImmutableString _compilerPath, ImmutableString _commandLine, ImmutableString _workingDirectory, ImmutableString _amalgamationPath, CompilandSourceFiles* _sourceFiles, uint32_t _uniqueId, Type::Enum _type, bool _isPartOfLibrary, bool _wasRecompiled) :
			originalObjPath(_originalObjPath), srcPath(_srcPath), pdbPath(_pdbPath), compilerPath(_compilerPath), commandLine(_commandLine), workingDirectory(_workingDirectory), amalgamationPath(_amalgamationPath), sourceFiles(_sourceFiles), uniqueId(_uniqueId), type(_type), isPartOfLibrary(_isPartOfLibrary), wasRecompiled(_wasRecompiled)
		{
		}
		// END EPIC MOD

		LC_DISABLE_ASSIGNMENT(Compiland);
		LC_DISABLE_COPY(Compiland);
	};

	struct AmalgamatedCompiland
	{
		types::vector<ImmutableString> singleParts;		// .obj of the single files that make up an amalgamation
		bool isSplit;
	};

	struct ModifiedObjFile
	{
		std::wstring objPath;
		std::wstring amalgamatedObjPath;	// optional
	};


	// full path to .obj file
	typedef ImmutableString ObjPath;
	
	// full path to any file
	typedef ImmutableString FilePath;

	typedef uint64_t TimeStamp;

	struct Dependency
	{
		DirectoryCache::Directory* parentDirectory;
		TimeStamp lastModification;
		types::vector<ObjPath> objPaths;
		bool hadInitialChange;
	};

	struct CompilandDB
	{
		// stores a compiland for each .obj file that contributed to a module and has its source available
		types::StringMap<Compiland*> compilands;

		// stores an array of .obj files which are part of amalgamated compilands
		types::StringMap<AmalgamatedCompiland*> amalgamatedCompilands;

		// stores an array of .obj files that need to be recompiled when a certain file changes
		types::StringMap<Dependency*> dependencies;

		// stores the real name of the .obj on disk for each original DIA compiland name.
		// this lookup table is needed because DIA compilands coming from static libraries often don't contain the full path,
		// but a path that is relative to the project instead, e.g. obj\x64\Debug\SimpleFunction.obj
		types::StringMap<ImmutableString> compilandNameToObjOnDisk;
	};

	struct DiaCompilandDB
	{
		types::vector<IDiaSymbol*> symbols;
	};

	struct ModuleDB
	{
		types::vector<std::wstring> modules;		// relative path to .exe or .dll
	};

	struct LibraryDB
	{
		// full path to all .lib files that need to be used for linking
		types::vector<FilePath> libraries;
	};

	struct LinkerDB
	{
		ImmutableString pdbPath;				// full path to .pdb file
		ImmutableString linkerPath;				// full path to linker used to link the executable
		ImmutableString commandLine;			// command line arguments passed to the linker (emitted by VS2015 and later)
		ImmutableString workingDirectory;		// full path to working directory

		LC_DISABLE_ASSIGNMENT(LinkerDB);
	};

	struct ThunkDB
	{
		// stores thunks from table entry RVA to target RVA.
		types::unordered_map<uint32_t, uint32_t> thunksFromTableEntryToTarget;

		// stores thunks from target RVA to table entry RVA
		// there can be several different thunks referring to the same RVA!
		types::unordered_map<uint32_t, types::vector<uint32_t>> thunksFromTargetToTableEntries;
	};

	struct ImageSection
	{
		uint32_t nameIndex;
		uint32_t rva;
		uint32_t size;
	};

	struct ImageSectionDB
	{
		types::vector<ImmutableString> sectionNames;
		types::vector<ImageSection> sections;				// sorted by RVA
		types::StringMap<ImageSection> sectionsByName;
	};

	struct Provider
	{
		IDiaDataSource* diaDataSource;
		IDiaSession* diaSession;
		IDiaSymbol* globalScope;
		TimeStamp lastModificationTime;
	};

	struct DynamicInitializerDB
	{
		// no symbol ownership
		types::vector<const Symbol*> dynamicInitializers;
	};

	struct UserDefinedTypesDB
	{
		types::unordered_set<uint32_t> typeIds;
	};


	// LIFETIME
	struct OpenOptions
	{
		enum Enum : uint32_t
		{
			NONE = 0u, 
			ACCUMULATE_SIZE = 1u << 0u,
			USE_SYMBOL_SERVER = 1u << 1u
		};
	};


	// TODO: helper function to make fixing and refactoring easier for now
	Symbol* CreateNewSymbol(const ImmutableString& name, uint32_t rva, SymbolDB* db);


	Provider* OpenEXE(const wchar_t* filename, uint32_t openOptions);
	void Close(Provider* provider);

	// BEGIN EPIC MOD - Static grouping of compilands by unity blobs
	void ResetCachedUnityManifests();
	bool TryGetCompilandIdFromUnityManifest(const std::wstring& objPath, uint32_t& compilandId);
	uint32_t GetCompilandIdFromPath(const std::wstring& objPath);
	// END EPIC MOD


	// UPDATE

	// CACHE: must be kept per module
	SymbolDB* GatherSymbols(Provider* provider);

	// CACHE: must be kept per module
	ContributionDB* GatherContributions(Provider* provider);

	// converts the string table of .obj files to a string table of real .obj files on disk,
	// and builds a per-compiland array of contributions.
	void FinalizeContributions(const CompilandDB* compilandDb, ContributionDB* db);

	// CACHE: temporary, throw away after use
	DiaCompilandDB* GatherDiaCompilands(Provider* provider);


	struct CompilandOptions
	{
		enum Enum : uint32_t
		{
			NONE = 0u,
			GENERATE_LOGS = 1u << 0u,
			FORCE_PCH_PDBS = 1u << 1u,
			TRACK_OBJ_ONLY = 1u << 2u
		};
	};

	// CACHE: stored only ONCE, after initial load, cannot change
	CompilandDB* GatherCompilands(const Provider* provider, const DiaCompilandDB* diaCompilandDb, unsigned int splitAmalgamatedFilesThreshold, uint32_t compilandOptions);

	// CACHE: stored only ONCE, after initial load, cannot change
	LibraryDB* GatherLibraries(const DiaCompilandDB* diaCompilandDb);


	// CACHE: temporary, throw away after use
	IDiaSymbol* FindLinkerSymbol(const DiaCompilandDB* diaCompilandDb);

	// CACHE: stored only ONCE, after initial load, cannot change
	LinkerDB* GatherLinker(IDiaSymbol* linkerSymbol);

	// CACHE: must be kept per module
	ThunkDB* GatherThunks(IDiaSymbol* linkerSymbol);

	// CACHE: must be kept per module
	ImageSectionDB* GatherImageSections(IDiaSymbol* linkerSymbol);

	// CACHE: must be kept per module, stores symbols directly into the given symbol database
	DynamicInitializerDB GatherDynamicInitializers(const Provider* provider, const executable::Image* image, const executable::ImageSectionDB* imageSections, const ImageSectionDB* imageSectionDb, const ContributionDB* contributionDb, const CompilandDB* compilandDb, const CoffCache<coff::CoffDB>* coffCache, SymbolDB* symbolDb);



	void DestroyLinkerSymbol(IDiaSymbol* symbol);

	void DestroyDiaCompilandDB(DiaCompilandDB* db);
	void DestroyModuleDB(ModuleDB* db);

	void DestroyCompilandDB(CompilandDB* db);

	void MergeCompilandsAndDependencies(CompilandDB* existingDb, CompilandDB* mergedDb);



	void MarkCompilandAsRecompiled(Compiland* compiland);
	void ClearCompilandAsRecompiled(Compiland* compiland);

	bool IsCompilandRecompiled(const Compiland* compiland);


	// ACCESS
	Compiland* FindCompiland(CompilandDB* db, const ObjPath& objPath);
	const Compiland* FindCompiland(const CompilandDB* db, const ObjPath& objPath);
	AmalgamatedCompiland* FindAmalgamatedCompiland(CompilandDB* db, const ObjPath& objPath);
	const AmalgamatedCompiland* FindAmalgamatedCompiland(const CompilandDB* db, const ObjPath& objPath);

	bool IsAmalgamation(const Compiland* compiland);
	bool IsPartOfAmalgamation(const Compiland* compiland);

	// finds a symbol by name. must match exactly
	const Symbol* FindSymbolByName(const SymbolDB* db, const ImmutableString& name);
	const Symbol* FindSymbolByRVA(const SymbolDB* db, uint32_t rva);

	// finds a section by name. must match exactly
	const ImageSection* FindImageSectionByName(const ImageSectionDB* db, const ImmutableString& name);

	// finds a section by RVA, binary search. RVA need not be exact, only must lie inside any of the sections
	const ImageSection* FindImageSectionByRVA(const ImageSectionDB* db, uint32_t rva);

	// finds a thunk using the table entry RVA, returns the target RVA
	uint32_t FindThunkTargetByRVA(const ThunkDB* db, uint32_t tableEntryRva);

	// finds thunks using the target RVA, returns all thunks that point to the target RVA
	types::vector<uint32_t> FindThunkTableEntriesByRVA(const ThunkDB* db, uint32_t targetRva);


	// takes into account that symbol names might contain COFF suffixes
	std::string UndecorateSymbolName(const ImmutableString& symbolName);


	// finds a contribution by RVA, binary search. RVA need not be exact, only must lie inside any of the contributions
	const Contribution* FindContributionByRVA(const ContributionDB* db, uint32_t rva);

	// returns the compiland name of a given contribution
	ImmutableString GetContributionCompilandName(const ContributionDB* db, const Contribution* contribution);

	// returns all contributions for a certain compiland name
	const ContributionDB::ContributionsPerCompiland* GetContributionsForCompilandName(const ContributionDB* db, const ImmutableString& compilandName);

	// returns all contributions for a certain compiland name index
	const ContributionDB::ContributionsPerCompiland* GetContributionsForCompilandNameIndex(const ContributionDB* db, uint32_t compilandNameIndex);

	const ImmutableString& GetImageSectionName(const ImageSectionDB* db, const ImageSection* imageSection);



	// helper function that returns or generates the unique ID of an optional compiland.
	// for files split off from amalgamated files, we need to use the original object path of the amalgamated file here,
	// otherwise names of symbols would differ, leading to constructors of global instances being called again.
	uint32_t GetCompilandId(const Compiland* compiland, const wchar_t* const objPath, const types::vector<ModifiedObjFile>& modifiedObjFiles);

	ImmutableString TransformAnonymousNamespacePattern(const ImmutableString& immutableString, uint32_t uniqueId);

	void TransformAnonymousNamespaceSymbols(SymbolDB* symbolDb, ContributionDB* contributionDb, CompilandDB* compilandDb, const types::vector<ModifiedObjFile>& modifiedObjFiles);


	bool IsPchSymbol(const ImmutableString& symbolName);

	bool IsVTable(const ImmutableString& symbolName);
	bool IsRttiObjectLocator(const ImmutableString& symbolName);

	bool IsDynamicInitializer(const ImmutableString& functionSymbolName);
	bool IsDynamicAtexitDestructor(const ImmutableString& functionSymbolName);
	bool IsPointerToDynamicInitializer(const ImmutableString& symbolName);

	bool IsStringLiteral(const ImmutableString& symbolName);
	bool IsLineNumber(const ImmutableString& symbolName);

	bool IsFloatingPointSseAvxConstant(const ImmutableString& symbolName);

	bool IsExceptionRelatedSymbol(const ImmutableString& symbolName);
	bool IsExceptionClauseSymbol(const ImmutableString& symbolName);
	bool IsExceptionUnwindSymbolForDynamicInitializer(const ImmutableString& symbolName);

	bool IsRuntimeCheckRelatedSymbol(const ImmutableString& symbolName);
	bool IsSdlCheckRelatedSymbol(const ImmutableString& symbolName);
	bool IsControlFlowGuardRelatedSymbol(const ImmutableString& symbolName);

	bool IsImageBaseRelatedSymbol(const ImmutableString& symbolName);
	bool IsTlsArrayRelatedSymbol(const ImmutableString& symbolName);
	bool IsTlsIndexRelatedSymbol(const ImmutableString& symbolName);
	bool IsTlsInitRelatedSymbol(const ImmutableString& symbolName);
	bool IsTlsStaticsRelatedSymbol(const ImmutableString& symbolName);

	bool IsSectionSymbol(const ImmutableString& symbolName);

	// BEGIN EPIC MOD
	bool IsUERegisterSymbol(const ImmutableString& symbolName, const char*& name, size_t& nameLength);
	bool IsUERegisterSymbol(const ImmutableString& symbolName);
	bool IsUENoStripSymbol(const ImmutableString& symbolName);
	bool IsUEInitializerSymbol(const ImmutableString& symbolName);
	bool IsUEReversePatchSymbol(const ImmutableString& symbolName);
	bool IsUEStaticsSymbol(const ImmutableString& symbolName);
	// END EPIC MOD
}


#endif