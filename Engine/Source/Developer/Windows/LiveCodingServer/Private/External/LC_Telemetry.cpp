// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_Telemetry.h"
#include "LC_TimeStamp.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include <ratio>
#include <inttypes.h>

#include "Windows/WindowsHWrapper.h"
#include "Windows/AllowWindowsPlatformAtomics.h"
// END EPIC MOD

namespace
{
	static void Print(const char* name, uint64_t start)
	{
		const uint64_t now = timeStamp::Get();
		const uint64_t delta = now - start;
		LC_LOG_TELEMETRY("Scope \"%s\" took %.3fs (%.3fms)", name, timeStamp::ToSeconds(delta), timeStamp::ToMilliSeconds(delta));
	}
}


telemetry::Scope::Scope(const char* name)
	: m_name(name)
	, m_start(timeStamp::Get())
	, m_cs()
{
}


telemetry::Scope::~Scope(void)
{
	CriticalSection::ScopedLock lock(&m_cs);

	if (m_name)
	{
		Print(m_name, m_start);
	}
}


double telemetry::Scope::ReadSeconds(void) const
{
	const uint64_t now = timeStamp::Get();
	return timeStamp::ToSeconds(now - m_start);
}


double telemetry::Scope::ReadMilliSeconds(void) const
{
	const uint64_t now = timeStamp::Get();
	return timeStamp::ToMilliSeconds(now - m_start);
}


double telemetry::Scope::ReadMicroSeconds(void) const
{
	const uint64_t now = timeStamp::Get();
	return timeStamp::ToMicroSeconds(now - m_start);
}


void telemetry::Scope::Restart(void)
{
	m_start = timeStamp::Get();
}


void telemetry::Scope::End(void)
{
	CriticalSection::ScopedLock lock(&m_cs);

	Print(m_name, m_start);

	// do not print again when going out of scope
	m_name = nullptr;
}


telemetry::Accumulator::Accumulator(const char* name)
	: m_name(name)
	, m_current(0ull)
	, m_accumulated(0ull)
{
}


void telemetry::Accumulator::Accumulate(uint64_t value)
{
	::InterlockedAdd64(reinterpret_cast<volatile LONG64*>(&m_current), static_cast<LONG64>(value));
	::InterlockedAdd64(reinterpret_cast<volatile LONG64*>(&m_accumulated), static_cast<LONG64>(value));
}


void telemetry::Accumulator::ResetCurrent(void)
{
	m_current = 0ull;
}


uint64_t telemetry::Accumulator::ReadCurrent(void) const
{
	return m_current;
}


uint64_t telemetry::Accumulator::ReadAccumulated(void) const
{
	return m_accumulated;
}


void telemetry::Accumulator::Print(void)
{
	LC_LOG_TELEMETRY("Accumulator \"%s\"", m_name);

	LC_LOG_INDENT_TELEMETRY;
	LC_LOG_TELEMETRY("Current: %" PRId64 " (%.3f KB, %.3f MB)", m_current, static_cast<double>(m_current) / 1024.0, static_cast<double>(m_current) / 1048576.0);
	LC_LOG_TELEMETRY("Accumulated: %" PRId64 " (%.3f KB, %.3f MB)", m_accumulated, static_cast<double>(m_accumulated) / 1024.0, static_cast<double>(m_accumulated) / 1048576.0);
}

// BEGIN EPIC MOD
#include "Windows/HideWindowsPlatformAtomics.h"
// END EPIC MOD


#endif