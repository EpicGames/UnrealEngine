// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_CriticalSection.h"
// BEGIN EPIC MOD
#include <chrono>
// END EPIC MOD

namespace telemetry
{
	// scoped timing information
	class Scope
	{
	public:
		explicit Scope(const char* name);
		~Scope(void);

		double ReadSeconds(void) const;
		double ReadMilliSeconds(void) const;
		double ReadMicroSeconds(void) const;
		void Restart(void);
		void End(void);

	private:
		const char* m_name;
		uint64_t m_start;
		CriticalSection m_cs;
	};


	class Accumulator
	{
	public:
		explicit Accumulator(const char* name);

		void Accumulate(uint64_t value);
		void ResetCurrent(void);

		uint64_t ReadCurrent(void) const;
		uint64_t ReadAccumulated(void) const;
		
		void Print(void);

	private:
		const char* m_name;
		uint64_t m_current;
		uint64_t m_accumulated;
	};
}


#endif