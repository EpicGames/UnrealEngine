// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_TimeDate.h"
// BEGIN EPIC MOD
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

TimeDate::TimeDate(unsigned short year, unsigned char month, unsigned char day, unsigned char hour, unsigned char minute, unsigned char second, unsigned short milliSecond)
	: m_year(year)
	, m_month(month)
	, m_day(day)
	, m_hour(hour)
	, m_minute(minute)
	, m_second(second)
	, m_milliSecond(milliSecond)
{
}


TimeDate TimeDate::GetCurrent(void)
{
	SYSTEMTIME localTime = {};
	::GetLocalTime(&localTime);

	return TimeDate(static_cast<unsigned short>(localTime.wYear), static_cast<unsigned char>(localTime.wMonth), static_cast<unsigned char>(localTime.wDay), static_cast<unsigned char>(localTime.wHour), static_cast<unsigned char>(localTime.wMinute), static_cast<unsigned char>(localTime.wSecond), static_cast<unsigned short>(localTime.wMilliseconds));
}


const char* TimeDate::ToTimeString(TimeDescription& desc) const
{
	_snprintf_s(desc, _TRUNCATE, "%02d:%02d:%02d,%04d", m_hour, m_minute, m_second, m_milliSecond);
	return desc;
}


const char* TimeDate::ToDateString(DateDescription& desc) const
{
	_snprintf_s(desc, _TRUNCATE, "%d-%02d-%02d", m_year, m_month, m_day);
	return desc;
}


#endif