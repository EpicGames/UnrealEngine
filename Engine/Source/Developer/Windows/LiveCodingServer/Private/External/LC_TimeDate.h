// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1


class TimeDate
{
public:
	// A type representing the time format: extended ISO (ISO 8601) hh:mm:ss,mmmm plus a null terminator.
	typedef char TimeDescription[14];

	// A type representing the date format: ISO (ISO 8601) YYYY-MM-DD plus a null terminator.
	typedef char DateDescription[11];

	TimeDate(unsigned short year, unsigned char month, unsigned char day, unsigned char hour, unsigned char minute, unsigned char second, unsigned short milliSecond);

	// Returns the current time and date in local time.
	static TimeDate GetCurrent(void);

	// Converts the time into the extended ISO 8601 format, and returns a pointer to the description string.
	const char* ToTimeString(TimeDescription& desc) const;

	// Converts the date into the standard ISO 8601 format, and returns a pointer to the description string.
	const char* ToDateString(DateDescription& desc) const;

private:
	unsigned short m_year;
	unsigned char m_month;
	unsigned char m_day;
	unsigned char m_hour;
	unsigned char m_minute;
	unsigned char m_second;
	unsigned short m_milliSecond;
};


#endif