// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_TimeStamp.h"
// BEGIN EPIC MOD
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

namespace timeStamp
{
	double g_oneOverFrequency = 0.0;
	double g_thousandOverFrequency = 0.0;
	double g_millionOverFrequency = 0.0;


	void Startup(void)
	{
		::LARGE_INTEGER frequency = {};
		::QueryPerformanceFrequency(&frequency);
		g_oneOverFrequency = 1.0 / frequency.QuadPart;
		g_thousandOverFrequency = 1000.0 / frequency.QuadPart;
		g_millionOverFrequency = 1000000.0 / frequency.QuadPart;
	}


	void Shutdown(void)
	{
		g_oneOverFrequency = 0.0;
		g_thousandOverFrequency = 0.0;
		g_millionOverFrequency = 0.0;
	}


	uint64_t Get(void)
	{
		::LARGE_INTEGER now = {};
		::QueryPerformanceCounter(&now);
		return static_cast<uint64_t>(now.QuadPart);
	}


	double ToSeconds(uint64_t count)
	{
		return count * g_oneOverFrequency;
	}


	double ToMilliSeconds(uint64_t count)
	{
		return count * g_thousandOverFrequency;
	}


	double ToMicroSeconds(uint64_t count)
	{
		return count * g_millionOverFrequency;
	}
}


#endif