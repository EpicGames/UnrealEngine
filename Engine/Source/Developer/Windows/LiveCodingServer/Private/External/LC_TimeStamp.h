// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "LC_Platform.h"
// END EPIC MOD

namespace timeStamp
{
	void Startup(void);
	void Shutdown(void);


	// Returns the current value of the high-resolution performance counter.
	// This is not the same as actual CPU cycles.
	uint64_t Get(void);

	// Converts counts returned by Get() into seconds.
	double ToSeconds(uint64_t count);

	// Converts counts returned by Get() into milliseconds.
	double ToMilliSeconds(uint64_t count);

	// Converts counts returned by Get() into microseconds.
	double ToMicroSeconds(uint64_t count);
}


#endif