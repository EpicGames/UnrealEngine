// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ImmutableString.h"
// BEGIN EPIC MOD
#include <unordered_set>
#include <unordered_map>
// END EPIC MOD

namespace types
{
	// using these types allows us to quickly switch allocators for STL data structures if we want to
	template <typename T>
	using Allocator = std::allocator<T>;

	template <typename T>
	using StringMap = std::unordered_map
	<
		ImmutableString,
		T,
		ImmutableString::Hasher,
		ImmutableString::Comparator,
		Allocator<std::pair<const ImmutableString, T>>
	>;

	using StringSet = std::unordered_set
	<
		ImmutableString,
		ImmutableString::Hasher,
		ImmutableString::Comparator,
		Allocator<ImmutableString>
	>;

	template <typename T>
	using vector = std::vector
	<
		T,
		Allocator<T>
	>;

	template <typename Key, typename Value>
	using unordered_map = std::unordered_map
	<
		Key,
		Value,
		std::hash<Key>,
		std::equal_to<Key>,
		Allocator<std::pair<const Key, Value>>
	>;

	template <typename Key, typename Value, typename Hash>
	using unordered_map_with_hash = std::unordered_map
	<
		Key,
		Value,
		Hash,
		std::equal_to<Key>,
		Allocator<std::pair<const Key, Value>>
	>;

	template <typename Key>
	using unordered_set = std::unordered_set
	<
		Key,
		std::hash<Key>,
		std::equal_to<Key>,
		Allocator<Key>
	>;
}


#endif