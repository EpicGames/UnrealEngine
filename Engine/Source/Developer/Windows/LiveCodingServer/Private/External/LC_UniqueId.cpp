// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_UniqueId.h"
#include "LC_Hashing.h"
// BEGIN EPIC MOD
#include "LC_CriticalSection.h"
#include "LC_Types.h"
// END EPIC MOD

namespace detail
{
	class UniqueId
	{
		struct Hasher
		{
			inline size_t operator()(const std::wstring& key) const
			{
				return Hashing::Hash32(key.c_str(), key.length() * sizeof(wchar_t), 0u);
			}
		};

	public:
		UniqueId(void)
			: m_data()
			, m_uniqueId(0u)
			, m_cs()
		{
			m_data.reserve(1024u);
		}

		uint32_t Generate(const std::wstring& path)
		{
			CriticalSection::ScopedLock lock(&m_cs);

			// try to insert the element into the cache. if it exists, return the cached data.
			// if it doesn't exist, generate the unique ID and store it.
			const std::pair<Cache::iterator, bool>& optional = m_data.emplace(path, 0u);
			uint32_t& data = optional.first->second;
			if (optional.second)
			{
				// value was inserted, update it with the correct data
				data = m_uniqueId;
				++m_uniqueId;
			}

			return data;
		}

	private:
		typedef types::unordered_map_with_hash<std::wstring, uint32_t, Hasher> Cache;
		Cache m_data;
		uint32_t m_uniqueId;
		CriticalSection m_cs;
	};
}


namespace uniqueId
{
	static detail::UniqueId* g_uniqueId = nullptr;


	void Startup(void)
	{
		g_uniqueId = new detail::UniqueId;
	}


	void Shutdown(void)
	{
		delete g_uniqueId;
	}


	uint32_t Generate(const std::wstring& path)
	{
		return g_uniqueId->Generate(path);
	}
}


#endif