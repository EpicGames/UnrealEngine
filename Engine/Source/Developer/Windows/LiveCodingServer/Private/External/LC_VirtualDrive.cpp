// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_VirtualDrive.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include "Windows/WindowsHWrapper.h"
// END EPIC MOD

void virtualDrive::Add(const wchar_t* driveLetterPlusColon, const wchar_t* path)
{
	const BOOL success = ::DefineDosDeviceW(DDD_NO_BROADCAST_SYSTEM, driveLetterPlusColon, path);
	if (success == 0)
	{
		const DWORD error = ::GetLastError();
		LC_ERROR_USER("Cannot add virtual drive %S for path %S. Error: 0x%X", driveLetterPlusColon, path, error);
	}
}


void virtualDrive::Remove(const wchar_t* driveLetterPlusColon, const wchar_t* path)
{
	const BOOL success = ::DefineDosDeviceW(DDD_NO_BROADCAST_SYSTEM | DDD_REMOVE_DEFINITION | DDD_EXACT_MATCH_ON_REMOVE, driveLetterPlusColon, path);
	if (success == 0)
	{
		const DWORD error = ::GetLastError();
		LC_ERROR_USER("Cannot remove virtual drive %S for path %S. Error: 0x%X", driveLetterPlusColon, path, error);
	}
}


#endif