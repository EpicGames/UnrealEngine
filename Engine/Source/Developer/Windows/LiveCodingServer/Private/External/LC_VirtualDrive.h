// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD

namespace virtualDrive
{
	void Add(const wchar_t* driveLetterPlusColon, const wchar_t* path);
	void Remove(const wchar_t* driveLetterPlusColon, const wchar_t* path);
}


#endif