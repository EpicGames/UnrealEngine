// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_VirtualMemoryRange.h"
#include "LC_PointerUtil.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include <cinttypes>
// END EPIC MOD

VirtualMemoryRange::VirtualMemoryRange(Process::Handle processHandle)
	: m_processHandle(processHandle)
	, m_pageData()
	, m_cs()
{
	m_pageData.reserve(256u);
}


VirtualMemoryRange::~VirtualMemoryRange(void)
{
	for (const auto& it : m_pageData)
	{
		const bool success = ::VirtualFreeEx(+m_processHandle, it.address, 0u, MEM_RELEASE);
		if (!success)
		{
			LC_WARNING_DEV("Cannot free virtual memory region at 0x%p", it.address);
		}
	}

	m_pageData.clear();
}


void VirtualMemoryRange::ReservePages(const void* addressStart, const void* addressEnd, size_t alignment)
{
	CriticalSection::ScopedLock lock(&m_cs);

	LC_LOG_DEV("Reserving pages in the range 0x%p to 0x%p", addressStart, addressEnd);
	LC_LOG_INDENT_DEV;

	// reserve all free pages in the virtual memory range.
	// pages must be aligned to the given alignment.
	for (const void* address = addressStart; address < addressEnd; /* nothing */)
	{
		// align address to be scanned
		address = pointer::AlignTop<const void*>(address, alignment);

		if (address < addressStart)
		{
			// overflow happened because we scanned too far
			break;
		}

		::MEMORY_BASIC_INFORMATION memoryInfo = {};
		const size_t bytesReturned = ::VirtualQueryEx(+m_processHandle, address, &memoryInfo, sizeof(::MEMORY_BASIC_INFORMATION));
		// BEGIN EPIC MOD
		if (bytesReturned == 0)
			break;
		// END EPIC MOD

		// we are only interested in free pages
		if ((bytesReturned > 0u) && (memoryInfo.State == MEM_FREE))
		{
			// work out the maximum size of the page allocation.
			// we should not allocate past the end of the range.
			const size_t bytesLeft = pointer::Displacement<size_t>(memoryInfo.BaseAddress, addressEnd);
			const size_t size = std::min<size_t>(memoryInfo.RegionSize, bytesLeft);

			// try to reserve this page.
			// if we are really unlucky, the process might have allocated this region in the meantime.
			void* baseAddress = ::VirtualAllocEx(+m_processHandle, memoryInfo.BaseAddress, size, MEM_RESERVE, PAGE_NOACCESS);
			if (baseAddress)
			{
				LC_LOG_DEV("Reserving virtual memory region at 0x%p with size 0x%" PRIX64, baseAddress, size);
				m_pageData.emplace_back(PageData { baseAddress });
			}
		}

		// keep on searching
		address = pointer::Offset<const void*>(memoryInfo.BaseAddress, memoryInfo.RegionSize);
	}
}


void VirtualMemoryRange::FreePages(const void* addressStart, const void* addressEnd)
{
	CriticalSection::ScopedLock lock(&m_cs);

	LC_LOG_DEV("Freeing pages in the range 0x%p to 0x%p", addressStart, addressEnd);
	LC_LOG_INDENT_DEV;

	for (auto it = m_pageData.begin(); it != m_pageData.end(); /* nothing */)
	{
		const PageData& data = *it;
		if ((data.address >= addressStart) && (data.address < addressEnd))
		{
			const bool success = ::VirtualFreeEx(+m_processHandle, data.address, 0u, MEM_RELEASE);
			if (success)
			{
				LC_LOG_DEV("Freeing virtual memory region at 0x%p", data.address);
			}
			else
			{
				LC_WARNING_DEV("Cannot free virtual memory region at 0x%p", data.address);
			}

			it = m_pageData.erase(it);
		}
		else
		{
			++it;
		}
	}
}


// BEGIN EPIC MOD
void VirtualMemoryRange::AddPage(void* page)
{
	m_pageData.emplace_back(PageData{ page });
}
// END EPIC MOD



#endif