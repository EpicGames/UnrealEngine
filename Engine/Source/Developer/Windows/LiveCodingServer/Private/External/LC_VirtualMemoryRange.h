// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

// BEGIN EPIC MOD
#include "CoreTypes.h"
// END EPIC MOD
#include "LC_ProcessTypes.h"
// BEGIN EPIC MOD
#include "LC_CriticalSection.h"
#include "LC_Types.h"
// END EPIC MOD


class VirtualMemoryRange
{
public:
	explicit VirtualMemoryRange(Process::Handle processHandle);
	~VirtualMemoryRange(void);

	void ReservePages(const void* addressStart, const void* addressEnd, size_t alignment);
	void FreePages(const void* addressStart, const void* addressEnd);

	// BEGIN EPIC MOD
	void AddPage(void* page);
	// END EPIC MOD

private:
	struct PageData
	{
		void* address;
	};

	Process::Handle m_processHandle;
	types::vector<PageData> m_pageData;
	CriticalSection m_cs;
};


#endif