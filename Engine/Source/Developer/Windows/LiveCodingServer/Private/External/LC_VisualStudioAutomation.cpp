// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#if LC_VERSION == 1

// BEGIN EPIC MOD
//#include PCH_INCLUDE
// END EPIC MOD
#include "LC_VisualStudioAutomation.h"
#include "LC_COMThread.h"
#include "LC_StringUtil.h"
#include "LC_Thread.h"
// BEGIN EPIC MOD
#include "LC_Logging.h"
#include "Microsoft/COMPointer.h"
// END EPIC MOD

// BEGIN EPIC MOD
#if WITH_VISUALSTUDIO_DTE
// END EPIC MOD

namespace
{
	static COMThread* g_comThread = nullptr;


	// helper function that checks whether the debugger is currently held for debugging, e.g. at a breakpoint, an exception
	static bool IsHeldForDebugging(const EnvDTE::DebuggerPtr& debugger)
	{
		EnvDTE::dbgDebugMode mode = EnvDTE::dbgDesignMode;
		debugger->get_CurrentMode(&mode);

		return (mode == EnvDTE::dbgBreakMode);
	}


	// helper function that waits until the debugger finished executing a command and is in break mode again
	static bool WaitUntilBreakMode(const EnvDTE::DebuggerPtr& debugger)
	{
		const unsigned int RETRY_WAIT_TIME = 5u;
		const unsigned int RETRY_TIMEOUT = 500u;

		EnvDTE::dbgDebugMode mode = EnvDTE::dbgDesignMode;
		debugger->get_CurrentMode(&mode);

		unsigned int msWaited = 0u;
		while (mode != EnvDTE::dbgBreakMode)
		{
			Thread::Current::SleepMilliSeconds(RETRY_WAIT_TIME);
			msWaited += RETRY_WAIT_TIME;

			debugger->get_CurrentMode(&mode);

			if (msWaited >= RETRY_TIMEOUT)
			{
				// enough time has passed, bail out
				return false;
			}
		}

		return true;
	}
}


void visualStudio::Startup(void)
{
	// start the COM thread that does the actual work
	g_comThread = new COMThread;
}


void visualStudio::Shutdown(void)
{
	// shut down the COM thread
	memory::DeleteAndNull(g_comThread);
}


static EnvDTE::DebuggerPtr FindDebuggerAttachedToProcess(Process::Id processId)
{
	// get all objects from the running object table (ROT)
	// BEGIN EPIC MOD
	TComPtr<IRunningObjectTable> rot = nullptr;
	// END EPIC MOD
	HRESULT result = ::GetRunningObjectTable(0u, &rot);
	if (FAILED(result) || (!rot))
	{
		LC_ERROR_DEV("Could not initialize running object table. Error: 0x%X", result);
		return nullptr;
	}

	// BEGIN EPIC MOD
	TComPtr<IEnumMoniker> enumMoniker = nullptr;
	// END EPIC MOD
	result = rot->EnumRunning(&enumMoniker);
	if (FAILED(result) || (!enumMoniker))
	{
		LC_ERROR_DEV("Could not enumerate running objects. Error: 0x%X", result);
		return nullptr;
	}

	enumMoniker->Reset();
	do
	{
		// BEGIN EPIC MOD
		TComPtr<IMoniker> next = nullptr;
		// END EPIC MOD
		result = enumMoniker->Next(1u, &next, NULL);

		if (SUCCEEDED(result) && next)
		{
			// BEGIN EPIC MOD
			TComPtr<IBindCtx> context = nullptr;
			// END EPIC MOD
			result = ::CreateBindCtx(0, &context);

			if (FAILED(result) || (!context))
			{
				LC_ERROR_DEV("Could not create COM binding context. Error: 0x%X", result);
				continue;
			}

			wchar_t* displayName = nullptr;
			result = next->GetDisplayName(context, NULL, &displayName);
			if (FAILED(result) || (!displayName))
			{
				LC_ERROR_DEV("Could not retrieve display name. Error: 0x%X", result);
				continue;
			}

			// only try objects which are a specific version of Visual Studio
			if (string::Contains(displayName, L"VisualStudio.DTE."))
			{
				// free display name using COM allocator
				// BEGIN EPIC MOD
				TComPtr<IMalloc> comMalloc = nullptr;
				// END EPIC MOD
				result = ::CoGetMalloc(1u, &comMalloc);
				if (SUCCEEDED(result) && comMalloc)
				{
					comMalloc->Free(displayName);
				}

				// BEGIN EPIC MOD
				TComPtr<IUnknown> unknown = nullptr;
				result = rot->GetObject(next, &unknown);
				// END EPIC MOD
				if (FAILED(result) || (!unknown))
				{
					LC_ERROR_DEV("Could not retrieve COM object from running object table. Error: 0x%X", result);
					continue;
				}

				EnvDTE::_DTEPtr dte = nullptr;
				result = unknown->QueryInterface(&dte);
				if (FAILED(result) || (!dte))
				{
					// this COM object doesn't support the DTE interface
					LC_ERROR_DEV("Could not convert IUnknown to DTE interface. Error: 0x%X", result);
					continue;
				}

				EnvDTE::DebuggerPtr debugger = nullptr;
				result = dte->get_Debugger(&debugger);
				if (FAILED(result) || (!debugger))
				{
					// cannot access debugger, which means that the process is currently not being debugged
					LC_LOG_DEV("Could not access debugger interface. Error: 0x%X", result);
					continue;
				}

				// fetch all processes to which this debugger is attached
				EnvDTE::ProcessesPtr allProcesses = nullptr;
				result = debugger->get_DebuggedProcesses(&allProcesses);
				if (FAILED(result) || (!allProcesses))
				{
					LC_ERROR_DEV("Could not retrieve processes from debugger. Error: 0x%X", result);
					continue;
				}

				long processCount = 0;
				result = allProcesses->get_Count(&processCount);
				if (FAILED(result) || (processCount <= 0))
				{
					LC_ERROR_DEV("Could not retrieve process count from debugger. Error: 0x%X", result);
					continue;
				}

				// check all processes if any of them is the one we're looking for
				for (long i = 0u; i < processCount; ++i)
				{
					EnvDTE::ProcessPtr singleProcess = nullptr;
					result = allProcesses->Item(variant_t(i + 1), &singleProcess);

					if (FAILED(result) || (!singleProcess))
					{
						LC_ERROR_DEV("Could not retrieve process from debugger. Error: 0x%X", result);
						continue;
					}

					long debuggerProcessId = 0;
					result = singleProcess->get_ProcessID(&debuggerProcessId);
					if (FAILED(result) || (debuggerProcessId <= 0))
					{
						LC_ERROR_DEV("Could not retrieve process ID from debugger. Error: 0x%X", result);
						continue;
					}

					// we got a valid processId
					if (static_cast<unsigned int>(debuggerProcessId) == +processId)
					{
						// found debugger attached to our process
						return debugger;
					}
				}
			}
		}
	}
	while (result != S_FALSE);

	return nullptr;
}


EnvDTE::DebuggerPtr visualStudio::FindDebuggerAttachedToProcess(Process::Id processId)
{
	return g_comThread->CallInThread(&::FindDebuggerAttachedToProcess, processId);
}


static EnvDTE::DebuggerPtr FindDebuggerForProcess(Process::Id processId)
{
	// get all objects from the running object table (ROT)
	// BEGIN EPIC MOD
	TComPtr<IRunningObjectTable> rot = nullptr;
	// END EPIC MOD
	HRESULT result = ::GetRunningObjectTable(0u, &rot);
	if (FAILED(result) || (!rot))
	{
		LC_ERROR_DEV("Could not initialize running object table. Error: 0x%X", result);
		return nullptr;
	}

	// BEGIN EPIC MOD
	TComPtr<IEnumMoniker> enumMoniker = nullptr;
	// END EPIC MOD
	result = rot->EnumRunning(&enumMoniker);
	if (FAILED(result) || (!enumMoniker))
	{
		LC_ERROR_DEV("Could not enumerate running objects. Error: 0x%X", result);
		return nullptr;
	}

	enumMoniker->Reset();
	do
	{
		// BEGIN EPIC MOD
		TComPtr<IMoniker> next = nullptr;
		// END EPIC MOD
		result = enumMoniker->Next(1u, &next, NULL);

		if (SUCCEEDED(result) && next)
		{
			// BEGIN EPIC MOD
			TComPtr<IBindCtx> context = nullptr;
			// END EPIC MOD
			result = ::CreateBindCtx(0, &context);

			if (FAILED(result) || (!context))
			{
				LC_ERROR_DEV("Could not create COM binding context. Error: 0x%X", result);
				continue;
			}

			wchar_t* displayName = nullptr;
			result = next->GetDisplayName(context, NULL, &displayName);
			if (FAILED(result) || (!displayName))
			{
				LC_ERROR_DEV("Could not retrieve display name. Error: 0x%X", result);
				continue;
			}

			// only try objects which are a specific version of Visual Studio
			if (string::Contains(displayName, L"VisualStudio.DTE."))
			{
				// free display name using COM allocator
				// BEGIN EPIC MOD
				TComPtr<IMalloc> comMalloc = nullptr;
				// END EPIC MOD
				result = ::CoGetMalloc(1u, &comMalloc);
				if (SUCCEEDED(result) && comMalloc)
				{
					comMalloc->Free(displayName);
				}

				// BEGIN EPIC MOD
				TComPtr<IUnknown> unknown = nullptr;
				result = rot->GetObject(next, &unknown);
				// END EPIC MOD
				if (FAILED(result) || (!unknown))
				{
					LC_ERROR_DEV("Could not retrieve COM object from running object table. Error: 0x%X", result);
					continue;
				}

				EnvDTE::_DTEPtr dte = nullptr;
				result = unknown->QueryInterface(&dte);
				if (FAILED(result) || (!dte))
				{
					// this COM object doesn't support the DTE interface
					LC_ERROR_DEV("Could not convert IUnknown to DTE interface. Error: 0x%X", result);
					continue;
				}

				EnvDTE::DebuggerPtr debugger = nullptr;
				result = dte->get_Debugger(&debugger);
				if (FAILED(result) || (!debugger))
				{
					// cannot access debugger, which means that the process is currently not being debugged
					LC_LOG_DEV("Could not access debugger interface. Error: 0x%X", result);
					continue;
				}

				EnvDTE::ProcessPtr process = nullptr;
				result = debugger->get_CurrentProcess(&process);
				if (FAILED(result) || (!process))
				{
					// cannot access current process, reason unknown
					LC_ERROR_DEV("Could not access current process in debugger. Error: 0x%X", result);
					continue;
				}

				long debuggerProcessId = 0;
				result = process->get_ProcessID(&debuggerProcessId);
				if (FAILED(result) || (debuggerProcessId <= 0))
				{
					LC_ERROR_DEV("Could not retrieve process ID from debugger. Error: 0x%X", result);
					continue;
				}

				// we got a valid processId
				if (static_cast<unsigned int>(debuggerProcessId) == +processId)
				{
					// found debugger debugging our process
					return debugger;
				}
			}
		}
	}
	while (result != S_FALSE);

	return nullptr;
}


EnvDTE::DebuggerPtr visualStudio::FindDebuggerForProcess(Process::Id processId)
{
	return g_comThread->CallInThread(&::FindDebuggerForProcess, processId);
}


static bool AttachToProcess(const EnvDTE::DebuggerPtr& debugger, Process::Id processId)
{
	// fetch all local processes running on this machine
	EnvDTE::ProcessesPtr allProcesses = nullptr;
	HRESULT result = debugger->get_LocalProcesses(&allProcesses);
	if (FAILED(result) || (!allProcesses))
	{
		LC_ERROR_DEV("Could not retrieve local processes from debugger. Error: 0x%X", result);
		return false;
	}

	long processCount = 0;
	result = allProcesses->get_Count(&processCount);
	if (FAILED(result) || (processCount <= 0))
	{
		LC_ERROR_DEV("Could not retrieve local process count from debugger. Error: 0x%X", result);
		return false;
	}

	// check all processes if any of them is the one we're looking for
	for (long i = 0u; i < processCount; ++i)
	{
		EnvDTE::ProcessPtr singleProcess = nullptr;
		result = allProcesses->Item(variant_t(i + 1), &singleProcess);

		if (FAILED(result) || (!singleProcess))
		{
			LC_ERROR_DEV("Could not retrieve local process from debugger. Error: 0x%X", result);
			continue;
		}

		long localProcessId = 0;
		result = singleProcess->get_ProcessID(&localProcessId);
		if (FAILED(result) || (localProcessId <= 0))
		{
			LC_ERROR_DEV("Could not retrieve local process ID from debugger. Error: 0x%X", result);
			continue;
		}

		// we got a valid processId
		if (static_cast<unsigned int>(localProcessId) == +processId)
		{
			// this is the process we want to attach to
			result = singleProcess->Attach();
			if (FAILED(result))
			{
				LC_ERROR_USER("Could not attach debugger to process. Error: 0x%X", result);
				return false;
			}

			return true;
		}
	}

	return false;
}


bool visualStudio::AttachToProcess(const EnvDTE::DebuggerPtr& debugger, Process::Id processId)
{
	return g_comThread->CallInThread(&::AttachToProcess, debugger, processId);
}


static types::vector<EnvDTE::ThreadPtr> EnumerateThreads(const EnvDTE::DebuggerPtr& debugger)
{
	types::vector<EnvDTE::ThreadPtr> threads;

	EnvDTE::ProgramPtr program = nullptr;
	HRESULT result = debugger->get_CurrentProgram(&program);
	if (FAILED(result) || (!program))
	{
		LC_ERROR_DEV("Could not retrieve current program from debugger. Error: 0x%X", result);
		return threads;
	}

	EnvDTE::ThreadsPtr allThreads = nullptr;
	result = program->get_Threads(&allThreads);
	if (FAILED(result) || (!allThreads))
	{
		LC_ERROR_DEV("Could not retrieve running threads from debugger. Error: 0x%X", result);
		return threads;
	}

	long threadCount = 0;
	result = allThreads->get_Count(&threadCount);
	if (FAILED(result) || (threadCount <= 0))
	{
		LC_ERROR_DEV("Could not retrieve thread count from debugger. Error: 0x%X", result);
		return threads;
	}

	threads.reserve(static_cast<size_t>(threadCount));

	for (long i = 0u; i < threadCount; ++i)
	{
		EnvDTE::ThreadPtr singleThread = nullptr;
		result = allThreads->Item(variant_t(i + 1), &singleThread);
		if (FAILED(result) || (!singleThread))
		{
			LC_ERROR_DEV("Could not retrieve thread from debugger. Error: 0x%X", result);
			continue;
		}

		threads.push_back(singleThread);
	}

	return threads;
}


types::vector<EnvDTE::ThreadPtr> visualStudio::EnumerateThreads(const EnvDTE::DebuggerPtr& debugger)
{
	return g_comThread->CallInThread(&::EnumerateThreads, debugger);
}


static bool FreezeThreads(const EnvDTE::DebuggerPtr& debugger, const types::vector<EnvDTE::ThreadPtr>& threads)
{
	if (!IsHeldForDebugging(debugger))
	{
		LC_WARNING_USER("FreezeThreads: Debugger is no longer held for debugging");
		return false;
	}

	bool success = true;

	const size_t count = threads.size();
	for (size_t i = 0u; i < count; ++i)
	{
		EnvDTE::ThreadPtr singleThread = threads[i];
		const HRESULT result = singleThread->Freeze();
		if (FAILED(result))
		{
			LC_ERROR_USER("FreezeThreads: Failed to freeze thread");
			return false;
		}

		if (!WaitUntilBreakMode(debugger))
		{
			LC_WARNING_USER("FreezeThreads: Cannot wait for debugger break mode");
			return false;
		}

		success &= SUCCEEDED(result);
	}

	return success;
}


bool visualStudio::FreezeThreads(const EnvDTE::DebuggerPtr& debugger, const types::vector<EnvDTE::ThreadPtr>& threads)
{
	return g_comThread->CallInThread(&::FreezeThreads, debugger, threads);
}


static bool FreezeThread(const EnvDTE::DebuggerPtr& debugger, const types::vector<EnvDTE::ThreadPtr>& threads, Thread::Id threadId)
{
	if (!IsHeldForDebugging(debugger))
	{
		LC_WARNING_USER("FreezeThread: Debugger is no longer held for debugging");
		return false;
	}

	const size_t count = threads.size();
	for (size_t i = 0u; i < count; ++i)
	{
		EnvDTE::ThreadPtr singleThread = threads[i];

		long id = 0;
		HRESULT result = singleThread->get_ID(&id);
		if (FAILED(result) || (id <= 0))
		{
			continue;
		}

		if (static_cast<unsigned int>(id) == +threadId)
		{
			// found the thread we're looking for
			result = singleThread->Freeze();
			if (FAILED(result))
			{
				LC_ERROR_USER("FreezeThread: Failed to freeze thread");
				return false;
			}

			if (!WaitUntilBreakMode(debugger))
			{
				LC_WARNING_USER("FreezeThread: Cannot wait for debugger break mode");
				return false;
			}

			return SUCCEEDED(result);
		}
	}

	return false;
}


bool visualStudio::FreezeThread(const EnvDTE::DebuggerPtr& debugger, const types::vector<EnvDTE::ThreadPtr>& threads, Thread::Id threadId)
{
	return g_comThread->CallInThread(&::FreezeThread, debugger, threads, threadId);
}


static bool ThawThreads(const EnvDTE::DebuggerPtr& debugger, const types::vector<EnvDTE::ThreadPtr>& threads)
{
	if (!IsHeldForDebugging(debugger))
	{
		LC_WARNING_USER("ThawThreads: Debugger is no longer held for debugging");
		return false;
	}

	bool success = true;

	const size_t count = threads.size();
	for (size_t i = 0u; i < count; ++i)
	{
		EnvDTE::ThreadPtr singleThread = threads[i];
		const HRESULT result = singleThread->Thaw();
		if (FAILED(result))
		{
			LC_ERROR_USER("ThawThreads: Failed to thaw thread");
			return false;
		}

		if (!WaitUntilBreakMode(debugger))
		{
			LC_WARNING_USER("ThawThreads: Cannot wait for debugger break mode");
			return false;
		}

		success &= SUCCEEDED(result);
	}

	return success;
}


bool visualStudio::ThawThreads(const EnvDTE::DebuggerPtr& debugger, const types::vector<EnvDTE::ThreadPtr>& threads)
{
	return g_comThread->CallInThread(&::ThawThreads, debugger, threads);
}


static bool ThawThread(const EnvDTE::DebuggerPtr& debugger, const types::vector<EnvDTE::ThreadPtr>& threads, Thread::Id threadId)
{
	if (!IsHeldForDebugging(debugger))
	{
		LC_WARNING_USER("ThawThread: Debugger is no longer held for debugging");
		return false;
	}

	const size_t count = threads.size();
	for (size_t i = 0u; i < count; ++i)
	{
		EnvDTE::ThreadPtr singleThread = threads[i];

		long id = 0;
		HRESULT result = singleThread->get_ID(&id);
		if (FAILED(result) || (id <= 0))
		{
			continue;
		}

		if (static_cast<unsigned int>(id) == +threadId)
		{
			// found the thread we're looking for
			result = singleThread->Thaw();
			if (FAILED(result))
			{
				LC_ERROR_USER("ThawThread: Failed to thaw thread");
				return false;
			}

			if (!WaitUntilBreakMode(debugger))
			{
				LC_WARNING_USER("ThawThread: Cannot wait for debugger break mode");
				return false;
			}

			return SUCCEEDED(result);
		}
	}

	return false;
}


bool visualStudio::ThawThread(const EnvDTE::DebuggerPtr& debugger, const types::vector<EnvDTE::ThreadPtr>& threads, Thread::Id threadId)
{
	return g_comThread->CallInThread(&::ThawThread, debugger, threads, threadId);
}


static bool Resume(const EnvDTE::DebuggerPtr& debugger)
{
	// BEGIN EPIC MOD
	const HRESULT result = debugger->Go(variant_t(Windows::FALSE));
	// END EPIC MOD
	return SUCCEEDED(result);
}


bool visualStudio::Resume(const EnvDTE::DebuggerPtr& debugger)
{
	return g_comThread->CallInThread(&::Resume, debugger);
}


static bool Break(const EnvDTE::DebuggerPtr& debugger)
{
	// wait until the debugger really enters break-mode
	// BEGIN EPIC MOD
	const HRESULT result = debugger->Break(variant_t(Windows::TRUE));
	// END EPIC MOD
	return SUCCEEDED(result);
}


bool visualStudio::Break(const EnvDTE::DebuggerPtr& debugger)
{
	return g_comThread->CallInThread(&::Break, debugger);
}

// BEGIN EPIC MOD
#endif
// END EPIC MOD

#endif