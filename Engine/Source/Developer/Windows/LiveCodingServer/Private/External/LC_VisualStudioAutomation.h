// Copyright 2011-2020 Molecular Matters GmbH, all rights reserved.

#pragma once

#if LC_VERSION == 1

#include "LC_Types.h"
#include "LC_ProcessTypes.h"
#include "LC_ThreadTypes.h"
// BEGIN EPIC MOD
#include "VisualStudioDTE.h"

#if WITH_VISUALSTUDIO_DTE
// END EPIC MOD

namespace visualStudio
{
	void Startup(void);
	void Shutdown(void);


	// Finds a Visual Studio debugger instance currently attached to the process with the given ID
	EnvDTE::DebuggerPtr FindDebuggerAttachedToProcess(Process::Id processId);

	// Finds a Visual Studio debugger instance currently debugging the process with the given ID
	EnvDTE::DebuggerPtr FindDebuggerForProcess(Process::Id processId);

	// Attaches a Visual Studio debugger instance to the process with the given ID
	bool AttachToProcess(const EnvDTE::DebuggerPtr& debugger, Process::Id processId);

	// Enumerates all threads of a debugger instance
	types::vector<EnvDTE::ThreadPtr> EnumerateThreads(const EnvDTE::DebuggerPtr& debugger);

	// Freezes all given threads
	bool FreezeThreads(const EnvDTE::DebuggerPtr& debugger, const types::vector<EnvDTE::ThreadPtr>& threads);

	// Freezes a single thread with the given thread ID
	bool FreezeThread(const EnvDTE::DebuggerPtr& debugger, const types::vector<EnvDTE::ThreadPtr>& threads, Thread::Id threadId);

	// Thaws all given threads
	bool ThawThreads(const EnvDTE::DebuggerPtr& debugger, const types::vector<EnvDTE::ThreadPtr>& threads);

	// Thaws a single thread with the given thread ID
	bool ThawThread(const EnvDTE::DebuggerPtr& debugger, const types::vector<EnvDTE::ThreadPtr>& threads, Thread::Id threadId);

	// Resumes the process in the debugger
	bool Resume(const EnvDTE::DebuggerPtr& debugger);

	// Breaks the process in the debugger
	bool Break(const EnvDTE::DebuggerPtr& debugger);
}

// BEGIN EPIC MOD
#endif
// END EPIC MOD

#endif