// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveCodingServer.h"
#include "External/LC_Scheduler.h"
#include "External/LC_UniqueId.h"
#include "External/LC_Filesystem.h"
#include "External/LC_AppSettings.h"
#include "External/LC_ServerCommandThread.h"
#include "External/LC_Compiler.h"
#include "External/LC_Environment.h"
#include "External/LC_Process.h"

FLiveCodingServer* GLiveCodingServer = nullptr;

////////////////

FLiveCodingServer::FLiveCodingServer()
{
	CommandThread = nullptr;
}

FLiveCodingServer::~FLiveCodingServer()
{
	check(CommandThread == nullptr);
}

void FLiveCodingServer::Start(const wchar_t* InProcessGroupName)
{
	ProcessGroupName = InProcessGroupName;

	scheduler::Startup();
	Filesystem::Startup();
	uniqueId::Startup();
	appSettings::Startup(ProcessGroupName.c_str());

	check(CommandThread == nullptr);
	CommandThread = new ServerCommandThread(nullptr, ProcessGroupName.c_str(), RunMode::EXTERNAL_BUILD_SYSTEM);
}

void FLiveCodingServer::Stop()
{
	delete CommandThread;
	CommandThread = nullptr;

	appSettings::Shutdown();
	uniqueId::Shutdown();
	Filesystem::Shutdown();
	scheduler::Shutdown();

	ProcessGroupName.clear();
}

void FLiveCodingServer::RestartTargets()
{
	if (CommandThread != nullptr)
	{
		CommandThread->RestartTargets();
	}
}

void FLiveCodingServer::SetLinkerPath(const wchar_t* LinkerPath, const TMap<FString, FString>& LinkerEnvironment)
{
	appSettings::g_linkerPath->SetValueWithoutSaving(LinkerPath);
	appSettings::UpdateLinkerPathCache();

	if (LinkerEnvironment.Num() > 0)
	{
		Process::Environment* environment = Process::CreateEnvironmentFromMap(LinkerEnvironment);
		compiler::AddEnvironmentToCache(LinkerPath, environment);
	}
}

ILiveCodingServer::FBringToFrontDelegate& FLiveCodingServer::GetBringToFrontDelegate()
{
	return BringToFrontDelegate;
}

ILiveCodingServer::FClearOutputDelegate& FLiveCodingServer::GetClearOutputDelegate()
{
	return ClearOutputDelegate;
}

ILiveCodingServer::FStatusChangeDelegate& FLiveCodingServer::GetStatusChangeDelegate()
{
	return StatusChangeDelegate;
}

ILiveCodingServer::FLogOutputDelegate& FLiveCodingServer::GetLogOutputDelegate()
{
	return LogOutputDelegate;
}

ILiveCodingServer::FCompileDelegate& FLiveCodingServer::GetCompileDelegate()
{
	return CompileDelegate;
}

ILiveCodingServer::FCompileStartedDelegate& FLiveCodingServer::GetCompileStartedDelegate()
{
	return CompileStartedDelegate;
}

ILiveCodingServer::FCompileFinishedDelegate& FLiveCodingServer::GetCompileFinishedDelegate()
{
	return CompileFinishedDelegate;
}

ILiveCodingServer::FShowConsoleDelegate& FLiveCodingServer::GetShowConsoleDelegate()
{
	return ShowConsoleDelegate;
}

ILiveCodingServer::FSetVisibleDelegate& FLiveCodingServer::GetSetVisibleDelegate()
{
	return SetVisibleDelegate;
}

bool FLiveCodingServer::HasReinstancingProcess()
{
	return CommandThread != nullptr ? CommandThread->HasReinstancingProcess() : false;
}

bool FLiveCodingServer::ShowCompileFinishNotification()
{
	return CommandThread != nullptr ? CommandThread->ShowCompileFinishNotification() : false;
}
