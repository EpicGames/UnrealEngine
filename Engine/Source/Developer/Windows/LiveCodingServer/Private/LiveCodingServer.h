// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ILiveCodingServer.h"
#include "LC_RunMode.h"
#include <string>

class ServerCommandThread;

class FLiveCodingServer final : public ILiveCodingServer
{
public:
	FLiveCodingServer();
	virtual ~FLiveCodingServer();

	virtual void Start(const wchar_t* ProcessGroupName) override;
	virtual void Stop() override;

	virtual void RestartTargets() override;

	virtual void SetLinkerPath(const wchar_t* LinkerPath, const TMap<FString, FString>& LinkerEnvironment) override;

	// ILiveCodingServer implementation
	virtual FBringToFrontDelegate& GetBringToFrontDelegate() override final;
	virtual FClearOutputDelegate& GetClearOutputDelegate() override final;
	virtual FStatusChangeDelegate& GetStatusChangeDelegate() override final;
	virtual FLogOutputDelegate& GetLogOutputDelegate() override final;
	virtual FCompileDelegate& GetCompileDelegate() override final;
	virtual FCompileStartedDelegate& GetCompileStartedDelegate() override final;
	virtual FCompileFinishedDelegate& GetCompileFinishedDelegate() override final;
	virtual FShowConsoleDelegate& GetShowConsoleDelegate() override final;
	virtual FSetVisibleDelegate& GetSetVisibleDelegate() override final;
	virtual bool HasReinstancingProcess() override final;
	virtual bool ShowCompileFinishNotification() override final;

private:
	std::wstring ProcessGroupName;
	ServerCommandThread* CommandThread;

	FBringToFrontDelegate BringToFrontDelegate;
	FClearOutputDelegate ClearOutputDelegate;
	FStatusChangeDelegate StatusChangeDelegate;
	FLogOutputDelegate LogOutputDelegate;
	FCompileDelegate CompileDelegate;
	FCompileStartedDelegate CompileStartedDelegate;
	FCompileFinishedDelegate CompileFinishedDelegate;
	FShowConsoleDelegate ShowConsoleDelegate;
	FSetVisibleDelegate SetVisibleDelegate;
};

extern FLiveCodingServer *GLiveCodingServer;