// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveCodingServerModule.h"
#include "LiveCodingServer.h"
#include "Misc/ScopeLock.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "External/LC_Logging.h"

IMPLEMENT_MODULE(FLiveCodingServerModule, LiveCodingServer)

DEFINE_LOG_CATEGORY_STATIC(LogLiveCodingServer, Display, All);

static void ServerLogOutput(Logging::Channel::Enum Channel, Logging::Type::Enum Type, const wchar_t* const Text, int Count)
{
#if !NO_LOGGING
	ELogVerbosity::Type Verbosity = ELogVerbosity::Error;

	switch (Type)
	{
	case Logging::Type::LOG_ERROR:
		Verbosity = ELogVerbosity::Error;
		break;
	case Logging::Type::LOG_WARNING:
		// There are some warnings generated in the dev channel that aren't really actionable by the users.
		// For example, warnings about symbols being eliminated by the linker.  It would be nice to just 
		// filter that specific warning, but we can't.
		Verbosity = Channel == Logging::Channel::DEV ? ELogVerbosity::Verbose : ELogVerbosity::Warning;
		break;
	default:
		Verbosity = ELogVerbosity::Display;
		break;
	}

	if (LogLiveCodingServer.GetVerbosity() < Verbosity)
	{
		return;
	}

	if (Count == 1)
	{
		FMsg::Logf(__FILE__, __LINE__, LogLiveCodingServer.GetCategoryName(), Verbosity, TEXT("%s"), Text);
	}
	else
	{
		FMsg::Logf(__FILE__, __LINE__, LogLiveCodingServer.GetCategoryName(), Verbosity, TEXT("%s (repeated %d more times)"), Text, Count);
	}
#endif
}

static void ServerOutputHandler(Logging::Channel::Enum Channel, Logging::Type::Enum Type, const wchar_t* const Text)
{
	static FCriticalSection CriticalSection;
	static Logging::Channel::Enum LastChannel = Logging::Channel::DEV;
	static Logging::Type::Enum LastType = Logging::Type::LOG_ERROR;
	static int LastCount = 0;
	static FString LastText;

	FString TrimText = FString(Text).TrimEnd();
	{
		FScopeLock Lock(&CriticalSection);
		if (LastCount != 0 && LastType == Type && LastText.Equals(TrimText, ESearchCase::CaseSensitive))
		{
			++LastCount;
		}
		else
		{
			if (LastCount > 1)
			{
				ServerLogOutput(LastChannel, LastType, *LastText, LastCount);
			}
			LastCount = 1;
			LastChannel = Channel;
			LastType = Type;
			LastText = TrimText;
			ServerLogOutput(LastChannel, LastType, *LastText, LastCount);
		}
	}

	if (Channel == Logging::Channel::USER)
	{
		ELiveCodingLogVerbosity Verbosity;
		switch (Type)
		{
		case Logging::Type::LOG_SUCCESS:
			Verbosity = ELiveCodingLogVerbosity::Success;
			break;
		case Logging::Type::LOG_ERROR:
			Verbosity = ELiveCodingLogVerbosity::Failure;
			break;
		case Logging::Type::LOG_WARNING:
			Verbosity = ELiveCodingLogVerbosity::Warning;
			break;
		default:
			Verbosity = ELiveCodingLogVerbosity::Info;
			break;
		}
		GLiveCodingServer->GetLogOutputDelegate().ExecuteIfBound(Verbosity, Text);
	}
}

void FLiveCodingServerModule::StartupModule()
{
	Logging::SetOutputHandler(&ServerOutputHandler);

	GLiveCodingServer = new FLiveCodingServer();

	IModularFeatures::Get().RegisterModularFeature(LIVE_CODING_SERVER_FEATURE_NAME, GLiveCodingServer);
}

void FLiveCodingServerModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(LIVE_CODING_SERVER_FEATURE_NAME, GLiveCodingServer);

	if(GLiveCodingServer != nullptr)
	{
		delete GLiveCodingServer;
		GLiveCodingServer = nullptr;
	}

	Logging::SetOutputHandler(nullptr);
}
