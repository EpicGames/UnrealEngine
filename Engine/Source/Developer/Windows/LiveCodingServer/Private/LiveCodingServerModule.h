// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ILiveCodingServer.h"

class FLiveCodingServerModule : public ILiveCodingServerModule
{
public:
	// IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
