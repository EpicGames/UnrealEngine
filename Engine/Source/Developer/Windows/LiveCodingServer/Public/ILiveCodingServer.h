// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Features/IModularFeature.h"
#include "Modules/ModuleInterface.h"
#include "Delegates/Delegate.h"

#define LIVE_CODING_SERVER_FEATURE_NAME "LiveCodingServer"

enum class ELiveCodingResult
{
	Success,
	Error
};

enum class ELiveCodingLogVerbosity
{
	Info,
	Success,
	Warning,
	Failure,
};

enum class ELiveCodingCompileReason
{
	Initial,
	Retry,
};

enum class ELiveCodingCompileResult
{
	Success,
	Canceled,
	Failure,
	Retry,
};

class ILiveCodingServerModule : public IModuleInterface
{
public:
	// IModuleInterface implementation
	virtual void StartupModule() override = 0;
	virtual void ShutdownModule() override = 0;
};

struct FModuleFiles
{
	TArray<FString> Objects;
	TArray<FString> Libraries;
};
typedef TMap<FString, FModuleFiles> FModuleToModuleFiles;

class ILiveCodingServer : public IModularFeature
{
public:
	virtual void Start(const wchar_t* ProcessGroupName) = 0;
	virtual void Stop() = 0;

	virtual void RestartTargets() = 0;

	virtual void SetLinkerPath(const wchar_t* LinkerPath, const TMap<FString, FString>& LinkerEnvironment) = 0;

	DECLARE_DELEGATE(FBringToFrontDelegate);
	virtual FBringToFrontDelegate& GetBringToFrontDelegate() = 0;

	DECLARE_DELEGATE(FClearOutputDelegate);
	virtual FClearOutputDelegate& GetClearOutputDelegate() = 0;

	DECLARE_DELEGATE_OneParam(FStatusChangeDelegate, const wchar_t*);
	virtual FStatusChangeDelegate& GetStatusChangeDelegate() = 0;

	DECLARE_DELEGATE_TwoParams(FLogOutputDelegate, ELiveCodingLogVerbosity, const wchar_t*);
	virtual FLogOutputDelegate& GetLogOutputDelegate() = 0;

	DECLARE_DELEGATE_RetVal_SixParams(ELiveCodingCompileResult, FCompileDelegate, const TArray<FString>&, const TArray<FString>&, const TSet<FString>&, TArray<FString>&, FModuleToModuleFiles&, ELiveCodingCompileReason);
	virtual FCompileDelegate& GetCompileDelegate() = 0;

	DECLARE_DELEGATE(FCompileStartedDelegate);
	virtual FCompileStartedDelegate& GetCompileStartedDelegate() = 0;

	DECLARE_DELEGATE_TwoParams(FCompileFinishedDelegate, ELiveCodingResult, const wchar_t*);
	virtual FCompileFinishedDelegate& GetCompileFinishedDelegate() = 0;

	DECLARE_DELEGATE(FShowConsoleDelegate);
	virtual FShowConsoleDelegate& GetShowConsoleDelegate() = 0;

	DECLARE_DELEGATE_OneParam(FSetVisibleDelegate, bool);
	virtual FSetVisibleDelegate& GetSetVisibleDelegate() = 0;

	virtual bool HasReinstancingProcess() = 0;

	virtual bool ShowCompileFinishNotification() = 0;
};

