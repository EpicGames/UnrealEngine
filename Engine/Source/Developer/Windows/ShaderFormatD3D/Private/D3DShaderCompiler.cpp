// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12RHI.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "RayTracingDefinitions.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderFormatD3D.h"
#include "ShaderCompilerCommon.h"
#include "ShaderCompilerCommonInternal.h"
#include "ShaderCompilerDefinitions.h"
#include "ShaderCompileWorkerUtil.h"
#include "ShaderMinifier.h"
#include "ShaderParameterParser.h"
#include "ShaderPreprocessTypes.h"
#include "SpirvCommon.h"
#include "Algo/Transform.h"
#include "HlslParserInternal.h"
#include "ShaderSDCE.h"

#define DEBUG_SHADERS 0

// D3D doesn't define a mask for this, so we do so here
#define SHADER_OPTIMIZATION_LEVEL_MASK (D3DCOMPILE_OPTIMIZATION_LEVEL0 | D3DCOMPILE_OPTIMIZATION_LEVEL1 | D3DCOMPILE_OPTIMIZATION_LEVEL2 | D3DCOMPILE_OPTIMIZATION_LEVEL3)

// Disable macro redefinition warning for compatibility with Windows SDK 8+
#pragma warning(push)
#pragma warning(disable : 4005)	// macro redefinition

#include "Windows/AllowWindowsPlatformTypes.h"
	#include <D3D11.h>
	#include <D3Dcompiler.h>
	#include <d3d11Shader.h>
#include "Windows/HideWindowsPlatformTypes.h"
#undef DrawText

#include "D3DShaderCompiler.inl"

#pragma warning(pop)

static const uint32 GD3DMaximumNumUAVs = 8; // Limit for feature level 11.0

static int32 GD3DCheckForDoubles = 1;
static int32 GD3DDumpAMDCodeXLFile = 0;

/**
 * TranslateCompilerFlag - translates the platform-independent compiler flags into D3DX defines
 * @param CompilerFlag - the platform-independent compiler flag to translate
 * @return uint32 - the value of the appropriate D3DX enum
 */
static uint32 TranslateCompilerFlagD3D11(ECompilerFlags CompilerFlag)
{
	switch(CompilerFlag)
	{
	case CFLAG_PreferFlowControl: return D3DCOMPILE_PREFER_FLOW_CONTROL;
	case CFLAG_AvoidFlowControl: return D3DCOMPILE_AVOID_FLOW_CONTROL;
	case CFLAG_WarningsAsErrors: return D3DCOMPILE_WARNINGS_ARE_ERRORS;
	default: return 0;
	};
}

/*
 * Turns invalid absolute paths that FXC generated back into a virtual file paths,
 * e.g. "D:\\Engine\\Private\\Common.ush" into "/Engine/Private/Common.ush"
 */
static void D3D11SanitizeErrorVirtualFilePath(FString& ErrorLine)
{
	if (ErrorLine.Len() > 3 && ErrorLine[1] == TEXT(':') && ErrorLine[2] == TEXT('\\'))
	{
		const int32 EndOfFilePath = ErrorLine.Find(TEXT(":"), ESearchCase::CaseSensitive, ESearchDir::FromStart, 3);
		if (EndOfFilePath != INDEX_NONE)
		{
			for (int32 ErrorLineStringPosition = 2; ErrorLineStringPosition < EndOfFilePath; ++ErrorLineStringPosition)
			{
				if (ErrorLine[ErrorLineStringPosition] == TEXT('\\'))
				{
					ErrorLine[ErrorLineStringPosition] = TEXT('/');
				}
			}
			ErrorLine.RightChopInline(2);
		}
	}
}

/**
 * Filters out unwanted shader compile warnings
 */
static void D3D11FilterShaderCompileWarnings(const FString& CompileWarnings, TArray<FString>& FilteredWarnings)
{
	TArray<FString> WarningArray;
	FString OutWarningString = TEXT("");
	CompileWarnings.ParseIntoArray(WarningArray, TEXT("\n"), true);
	
	//go through each warning line
	for (int32 WarningIndex = 0; WarningIndex < WarningArray.Num(); WarningIndex++)
	{
		//suppress "warning X3557: Loop only executes for 1 iteration(s), forcing loop to unroll"
		if (!WarningArray[WarningIndex].Contains(TEXT("X3557"))
			// "warning X3205: conversion from larger type to smaller, possible loss of data"
			// Gets spammed when converting from float to half
			&& !WarningArray[WarningIndex].Contains(TEXT("X3205")))
		{
			D3D11SanitizeErrorVirtualFilePath(WarningArray[WarningIndex]);
			FilteredWarnings.AddUnique(WarningArray[WarningIndex]);
		}
	}
}

// @return 0 if not recognized
static const TCHAR* GetShaderProfileName(const FShaderCompilerInput& Input, ED3DShaderModel ShaderModel)
{
	if (ShaderModel == ED3DShaderModel::SM6_8)
	{
		switch (Input.Target.GetFrequency())
		{
		case SF_Pixel:                return TEXT("ps_6_8");
		case SF_Vertex:               return TEXT("vs_6_8");
		case SF_Mesh:                 return TEXT("ms_6_8");
		case SF_Amplification:        return TEXT("as_6_8");
		case SF_Geometry:             return TEXT("gs_6_8");
		case SF_Compute:              return TEXT("cs_6_8");
		case SF_RayGen:
		case SF_RayMiss:
		case SF_RayHitGroup:
		case SF_RayCallable:
		case SF_WorkGraphRoot:
		case SF_WorkGraphComputeNode: return TEXT("lib_6_8");
		}
	}
	else if (ShaderModel == ED3DShaderModel::SM6_6)
	{
		switch (Input.Target.GetFrequency())
		{
		case SF_Pixel:         return TEXT("ps_6_6");
		case SF_Vertex:        return TEXT("vs_6_6");
		case SF_Mesh:          return TEXT("ms_6_6");
		case SF_Amplification: return TEXT("as_6_6");
		case SF_Geometry:      return TEXT("gs_6_6");
		case SF_Compute:       return TEXT("cs_6_6");
		case SF_RayGen:
		case SF_RayMiss:
		case SF_RayHitGroup:
		case SF_RayCallable:   return TEXT("lib_6_6");
		}
	}
	else if (ShaderModel == ED3DShaderModel::SM6_0)
	{
		//set defines and profiles for the appropriate shader paths
		switch (Input.Target.GetFrequency())
		{
		case SF_Pixel:    return TEXT("ps_6_0");
		case SF_Vertex:   return TEXT("vs_6_0");
		case SF_Geometry: return TEXT("gs_6_0");
		case SF_Compute:  return TEXT("cs_6_0");
		}
	}
	else
	{
		//set defines and profiles for the appropriate shader paths
		switch (Input.Target.GetFrequency())
		{
		case SF_Pixel:    return TEXT("ps_5_0");
		case SF_Vertex:   return TEXT("vs_5_0");
		case SF_Geometry: return TEXT("gs_5_0");
		case SF_Compute:  return TEXT("cs_5_0");
		}
	}

	checkfSlow(false, TEXT("Unexpected shader frequency"));
	return nullptr;
}

/**
 * D3D11CreateShaderCompileCommandLine - takes shader parameters used to compile with the DX11
 * compiler and returns an fxc command to compile from the command line
 */
static FString D3D11CreateShaderCompileCommandLine(
	const FString& ShaderPath, 
	const TCHAR* EntryFunction, 
	const TCHAR* ShaderProfile, 
	uint32 CompileFlags,
	FShaderCompilerOutput& Output
	)
{
	// fxc is our command line compiler
	FString FXCCommandline = FString(TEXT("\"%FXC%\" ")) + ShaderPath;

	// add the entry point reference
	FXCCommandline += FString(TEXT(" /E ")) + EntryFunction;

	// go through and add other switches
	if(CompileFlags & D3DCOMPILE_PREFER_FLOW_CONTROL)
	{
		CompileFlags &= ~D3DCOMPILE_PREFER_FLOW_CONTROL;
		FXCCommandline += FString(TEXT(" /Gfp"));
	}

	if(CompileFlags & D3DCOMPILE_DEBUG)
	{
		CompileFlags &= ~D3DCOMPILE_DEBUG;
		FXCCommandline += FString(TEXT(" /Zi"));
	}

	if(CompileFlags & D3DCOMPILE_SKIP_OPTIMIZATION)
	{
		CompileFlags &= ~D3DCOMPILE_SKIP_OPTIMIZATION;
		FXCCommandline += FString(TEXT(" /Od"));
	}

	if (CompileFlags & D3DCOMPILE_SKIP_VALIDATION)
	{
		CompileFlags &= ~D3DCOMPILE_SKIP_VALIDATION;
		FXCCommandline += FString(TEXT(" /Vd"));
	}

	if(CompileFlags & D3DCOMPILE_AVOID_FLOW_CONTROL)
	{
		CompileFlags &= ~D3DCOMPILE_AVOID_FLOW_CONTROL;
		FXCCommandline += FString(TEXT(" /Gfa"));
	}

	if(CompileFlags & D3DCOMPILE_PACK_MATRIX_ROW_MAJOR)
	{
		CompileFlags &= ~D3DCOMPILE_PACK_MATRIX_ROW_MAJOR;
		FXCCommandline += FString(TEXT(" /Zpr"));
	}

	if(CompileFlags & D3DCOMPILE_ENABLE_BACKWARDS_COMPATIBILITY)
	{
		CompileFlags &= ~D3DCOMPILE_ENABLE_BACKWARDS_COMPATIBILITY;
		FXCCommandline += FString(TEXT(" /Gec"));
	}

	if (CompileFlags & D3DCOMPILE_WARNINGS_ARE_ERRORS)
	{
		CompileFlags &= ~D3DCOMPILE_WARNINGS_ARE_ERRORS;
		FXCCommandline += FString(TEXT(" /WX"));
	}

	if (CompileFlags & D3DCOMPILE_DEBUG_NAME_FOR_BINARY)
	{
		CompileFlags &= ~D3DCOMPILE_DEBUG_NAME_FOR_BINARY;
		FXCCommandline += FString(TEXT(" /Zsb"));
	}
	else if (CompileFlags & D3DCOMPILE_DEBUG_NAME_FOR_SOURCE)
	{
		CompileFlags &= ~D3DCOMPILE_DEBUG_NAME_FOR_SOURCE;
		FXCCommandline += FString(TEXT(" /Zss"));
	}

	switch (CompileFlags & SHADER_OPTIMIZATION_LEVEL_MASK)
	{
		case D3DCOMPILE_OPTIMIZATION_LEVEL2:
		CompileFlags &= ~D3DCOMPILE_OPTIMIZATION_LEVEL2;
		FXCCommandline += FString(TEXT(" /O2"));
			break;

		case D3DCOMPILE_OPTIMIZATION_LEVEL3:
		CompileFlags &= ~D3DCOMPILE_OPTIMIZATION_LEVEL3;
		FXCCommandline += FString(TEXT(" /O3"));
			break;

		case D3DCOMPILE_OPTIMIZATION_LEVEL1:
		CompileFlags &= ~D3DCOMPILE_OPTIMIZATION_LEVEL1;
		FXCCommandline += FString(TEXT(" /O1"));
			break;

		case D3DCOMPILE_OPTIMIZATION_LEVEL0:
			CompileFlags &= ~D3DCOMPILE_OPTIMIZATION_LEVEL0;
			break;

		default:
			Output.AddErrorf(TEXT("Unknown D3DCOMPILE optimization level"));
			break;
	}

	checkf(CompileFlags == 0, TEXT("Unhandled d3d11 shader compiler flag!"));

	// add the target instruction set
	FXCCommandline += FString(TEXT(" /T ")) + ShaderProfile;

	// Assembly instruction numbering
	FXCCommandline += TEXT(" /Ni");

	// Output to ShaderPath.d3dasm
	if (FPaths::GetExtension(ShaderPath) == TEXT("usf"))
	{
		FXCCommandline += FString::Printf(TEXT(" /Fc%sd3dasm"), *ShaderPath.LeftChop(3));
	}

	// add a pause on a newline
	FXCCommandline += FString(TEXT(" \r\n pause"));

	// Batch file header:
	/*
	@ECHO OFF
		SET FXC="C:\Program Files (x86)\Windows Kits\10\bin\x64\fxc.exe"
		IF EXIST %FXC% (
			REM
			) ELSE (
				ECHO Couldn't find Windows 10 SDK, falling back to DXSDK...
				SET FXC="%DXSDK_DIR%\Utilities\bin\x86\fxc.exe"
				IF EXIST %FXC% (
					REM
					) ELSE (
						ECHO Couldn't find DXSDK! Exiting...
						GOTO END
					)
			)
	*/
	const FString BatchFileHeader = TEXT(
		"@ECHO OFF\n"\
		"IF \"%FXC%\" == \"\" SET \"FXC=C:\\Program Files (x86)\\Windows Kits\\10\\bin\\x64\\fxc.exe\"\n"\
		"IF NOT EXIST \"%FXC%\" (\n"\
		"\t" "ECHO Couldn't find Windows 10 SDK, falling back to DXSDK...\n"\
		"\t" "SET \"FXC=%DXSDK_DIR%\\Utilities\\bin\\x86\\fxc.exe\"\n"\
		"\t" "IF NOT EXIST \"%FXC%\" (\n"\
		"\t" "\t" "ECHO Couldn't find DXSDK! Exiting...\n"\
		"\t" "\t" "GOTO END\n"\
		"\t)\n"\
		")\n"
	);
	return BatchFileHeader + FXCCommandline + TEXT("\n:END\nREM\n");
}


/** Creates a batch file string to call the AMD shader analyzer. */
static FString CreateAMDCodeXLCommandLine(
	const FString& ShaderPath, 
	const TCHAR* EntryFunction, 
	const TCHAR* ShaderProfile,
	uint32 DXFlags
	)
{
	// Hardcoded to the default install path since there's no Env variable or addition to PATH
	FString Commandline = FString(TEXT("\"C:\\Program Files (x86)\\AMD\\CodeXL\\CodeXLAnalyzer.exe\" -c Pitcairn")) 
		+ TEXT(" -f ") + EntryFunction
		+ TEXT(" -s HLSL")
		+ TEXT(" -p ") + ShaderProfile
		+ TEXT(" -a AnalyzerStats.csv")
		+ TEXT(" --isa ISA.txt")
		+ *FString::Printf(TEXT(" --DXFlags %u "), DXFlags)
		+ ShaderPath;

	// add a pause on a newline
	Commandline += FString(TEXT(" \r\n pause"));
	return Commandline;
}

// D3Dcompiler.h has function pointer typedefs for some functions, but not all
typedef HRESULT(WINAPI *pD3DReflect)
	(__in_bcount(SrcDataSize) LPCVOID pSrcData,
	 __in SIZE_T  SrcDataSize,
	 __in  REFIID pInterface,
	 __out void** ppReflector);

typedef HRESULT(WINAPI *pD3DStripShader)
	(__in_bcount(BytecodeLength) LPCVOID pShaderBytecode,
	 __in SIZE_T     BytecodeLength,
	 __in UINT       uStripFlags,
	__out ID3DBlob** ppStrippedBlob);

typedef HRESULT(WINAPI *pD3DGetBlobPart)
	(__in_bcount(SrcDataSize) LPCVOID pSrcData,
	 __in SIZE_T SrcDataSize,
	 __in D3D_BLOB_PART Part,
	 __in UINT Flags,
	 __out ID3DBlob** ppPart);

typedef HRESULT(WINAPI* pD3DGetDebugInfo)
	(__in_bcount(SrcDataSize) LPCVOID pSrcData,
	 __in SIZE_T SrcDataSize,
	 __out ID3DBlob** ppPart);

#define DEFINE_GUID_FOR_CURRENT_COMPILER(name, l, w1, w2, b1, b2, b3, b4, b5, b6, b7, b8) \
	static const GUID name = { l, w1, w2, { b1, b2,  b3,  b4,  b5,  b6,  b7,  b8 } }

// ShaderReflection IIDs may change between SDK versions if the reflection API changes.
// Define a GUID below that matches the desired IID for the DLL in CompilerPath. For example,
// look for IID_ID3D11ShaderReflection in d3d11shader.h for the SDK matching the compiler DLL.
DEFINE_GUID_FOR_CURRENT_COMPILER(IID_ID3D11ShaderReflectionForCurrentCompiler, 0x8d536ca1, 0x0cca, 0x4956, 0xa8, 0x37, 0x78, 0x69, 0x63, 0x75, 0x55, 0x84);

// Helper class to load the engine-packaged FXC DLL and retrieve function pointers for the various FXC functions from it.
class FxcCompilerFunctions
{
public:

	static pD3DCompile GetCompile() { return Instance().Compile; }
	static pD3DReflect GetReflect() { return Instance().Reflect; }
	static pD3DDisassemble GetDisassemble() { return Instance().Disassemble; }
	static pD3DStripShader GetStripShader() { return Instance().StripShader; }
	static pD3DGetBlobPart GetGetBlobPart() { return Instance().GetBlobPart; }
	static pD3DGetDebugInfo GetGetDebugInfo() { return Instance().GetDebugInfo; }

private:
	FxcCompilerFunctions()
	{
#if  PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
		FString CompilerPath = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/Windows/DirectX/arm64/d3dcompiler_47.dll");
#else
		FString CompilerPath = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/Windows/DirectX/x64/d3dcompiler_47.dll");
#endif
		CompilerDLL = LoadLibrary(*CompilerPath);
		if (!CompilerDLL)
		{
			UE_LOGF(LogD3DShaderCompiler, Fatal, "Cannot find the compiler DLL '%ls'", *CompilerPath);
		}
		Compile = (pD3DCompile)(void*)GetProcAddress(CompilerDLL, "D3DCompile");
		Reflect = (pD3DReflect)(void*)GetProcAddress(CompilerDLL, "D3DReflect");
		Disassemble = (pD3DDisassemble)(void*)GetProcAddress(CompilerDLL, "D3DDisassemble");
		StripShader = (pD3DStripShader)(void*)GetProcAddress(CompilerDLL, "D3DStripShader");
		GetBlobPart = (pD3DGetBlobPart)(void*)GetProcAddress(CompilerDLL, "D3DGetBlobPart");
		GetDebugInfo = (pD3DGetDebugInfo)(void*)GetProcAddress(CompilerDLL, "D3DGetDebugInfo");
	}

	static FxcCompilerFunctions& Instance()
	{
		static FxcCompilerFunctions Instance;
		return Instance;
	}

	HMODULE CompilerDLL = 0;
	pD3DCompile Compile = nullptr;
	pD3DReflect Reflect = nullptr;
	pD3DDisassemble Disassemble = nullptr;
	pD3DStripShader StripShader = nullptr;
	pD3DGetBlobPart GetBlobPart = nullptr;
	pD3DGetDebugInfo GetDebugInfo = nullptr;
};

static int D3DExceptionFilter(bool bCatchException)
{
	return bCatchException ? EXCEPTION_EXECUTE_HANDLER : EXCEPTION_CONTINUE_SEARCH;
}

static HRESULT D3DCompileWrapper(
	pD3DCompile				D3DCompileFunc,
	LPCVOID					pSrcData,
	SIZE_T					SrcDataSize,
	LPCSTR					pFileName,
	CONST D3D_SHADER_MACRO*	pDefines,
	ID3DInclude*			pInclude,
	LPCSTR					pEntrypoint,
	LPCSTR					pTarget,
	uint32					Flags1,
	uint32					Flags2,
	ID3DBlob**				ppCode,
	ID3DBlob**				ppErrorMsgs,
	bool					bCatchException = false
	)
{
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__try
#endif
	{
		return D3DCompileFunc(
			pSrcData,
			SrcDataSize,
			pFileName,
			pDefines,
			pInclude,
			pEntrypoint,
			pTarget,
			Flags1,
			Flags2,
			ppCode,
			ppErrorMsgs
		);
	}
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__except(D3DExceptionFilter(bCatchException))
	{
		FSCWErrorCode::Report(FSCWErrorCode::CrashInsidePlatformCompiler);
		return E_FAIL;
	}
#endif
}

inline bool IsCompatibleBinding(const D3D11_SHADER_INPUT_BIND_DESC& BindDesc, uint32 BindingSpace)
{
	return true;
}

static void PatchSpirvForPrecompilation(FSpirv& Spirv)
{
	// Remove [unroll] loop hints from SPIR-V as this can fail on infinite loops
	for (FSpirvIterator SpirvInstruction : Spirv)
	{
		if (SpirvInstruction.Opcode() == SpvOpLoopMerge && SpirvInstruction.Operand(3) == SpvLoopControlUnrollMask)
		{
			(*SpirvInstruction)[3] = SpvLoopControlMaskNone;
		}
	}
}

// Re-orders all input/ouput stage variables from the specified HLSL source if it was cross-compiled with SPIRV-Cross.
// SPIRV-Cross can arrange the stage variables in a way that causes a mismatch between vertex and pixel shader pipelines.
static bool PatchHlslWithReorderedIOVariables(
	FString& HlslSourceString,
	const FString& OriginalShaderSource,
	const FString& OriginalEntryPoint,
	UE::HlslParser::EShaderParameterStorageClass StageVariablesStorageClass,
	TArray<FShaderCompilerError>& OutErrors)
{
	// Find declaration struct for stage variables
	const FStringView StageVariableDeclarationName = (StageVariablesStorageClass == UE::HlslParser::EShaderParameterStorageClass::Input ? TEXTVIEW("SPIRV_Cross_Input") : TEXTVIEW("SPIRV_Cross_Output"));
	const int32 StageVariableDeclarationBegin = HlslSourceString.Find(StageVariableDeclarationName, ESearchCase::CaseSensitive);
	if (StageVariableDeclarationBegin == INDEX_NONE)
	{
		return false;
	}

	const int32 StageVariableDelcarationBlockBegin = HlslSourceString.Find(TEXT("{"), ESearchCase::CaseSensitive, ESearchDir::FromStart, StageVariableDeclarationBegin + StageVariableDeclarationName.Len());
	if (StageVariableDelcarationBlockBegin == INDEX_NONE)
	{
		return false;
	}

	const int32 StageVariableDelcarationBlockEnd = HlslSourceString.Find(TEXT("}"), ESearchCase::CaseSensitive, ESearchDir::FromStart, StageVariableDelcarationBlockBegin + 1);
	if (StageVariableDelcarationBlockEnd == INDEX_NONE)
	{
		return false;
	}

	// Parse variable names from SPIR-V input
	TArray<FString> Variables, ParsingErrors;
	if (!UE::HlslParser::FindEntryPointParameters(OriginalShaderSource, OriginalEntryPoint, StageVariablesStorageClass, {}, Variables, ParsingErrors))
	{
		for (FString& Error : ParsingErrors)
		{
			OutErrors.Emplace(MoveTemp(Error));
		}
		return false;
	}

	// Parse declaration struct for stage variables into array of individual lines
	const FString StageVariableDeclSource = HlslSourceString.Mid(StageVariableDelcarationBlockBegin + 1, StageVariableDelcarationBlockEnd - (StageVariableDelcarationBlockBegin + 1));

	TArray<FString> StageVariableDeclSourceLines;
	StageVariableDeclSource.ParseIntoArrayLines(StageVariableDeclSourceLines);

	if (Variables.Num() != StageVariableDeclSourceLines.Num())
	{
		// Failed to match SPIR-V variables to SPIRV-Cross generated source
		return false;
	}

	// Returns true if the specified source line contains a variable declaration with the specified semantic.
	// HLSL semantics are case insensitive, must always appear after a colon ':' and when declared in a structure end with a semicolon ';'.
	// Here are some examples of such variable declarations, generated by SPIRV-Cross, we are parsing:
	//          Variable             Semantic
	//  ------------------------------------------
	//  float4 out_var_TEXCOORD10 : TEXCOORD10;
	//  float4 out_var_TEXCOORD1[1] : TEXCOORD1;
	//  precise float4 gl_Position : SV_Position;
	auto FindSemanticDeclarationInSourceLine = [](const FString& SourceLine, const FString& SemanticToSearch) -> bool
	{
		// Exit early if source line is not long enough for semantic
		if (SemanticToSearch.Len() > SourceLine.Len())
		{
			return false;
		}

		// Scan souce line for semantic range starting with first ':' character
		int32 StartPosition = 0;
		if (!SourceLine.FindChar(TEXT(':'), StartPosition))
		{
			return false;
		}

		// Progress until we have no more whitespaces, then we found the final start position of the semantic
		do
		{
			++StartPosition;
		}
		while (StartPosition < SourceLine.Len() && FChar::IsWhitespace(SourceLine[StartPosition]));

		// Exit early if remainder of source line is not long enough for semantic
		if (SemanticToSearch.Len() + StartPosition > SourceLine.Len())
		{
			return false;
		}

		// Now find the first whitespace character or the statement terminator ';'
		int32 EndPosition = StartPosition;
		while (EndPosition < SourceLine.Len() && !(FChar::IsWhitespace(SourceLine[EndPosition]) || SourceLine[EndPosition] == TEXT(';')))
		{
			++EndPosition;
		}

		// Now compare the semantic to search for with the source line range
		const int32 SemanticLen = EndPosition - StartPosition;
		if (SemanticToSearch.Len() == SemanticLen && FCString::Strnicmp(*SemanticToSearch, &SourceLine[StartPosition], SemanticLen) == 0)
		{
			return true;
		}

		// Check for special case if semantic contains default index in source line, e.g. "SV_ClipDinstance0"
		if (SemanticToSearch.Len() == SemanticLen - 1 && SourceLine[StartPosition + SemanticLen - 1] == TEXT('0') && FCString::Strnicmp(*SemanticToSearch, &SourceLine[StartPosition], SemanticLen - 1) == 0)
		{
			return true;
		}

		// Check for special case if semantic contains default index as subscript in source line, e.g. "SV_ClipDinstance[0]"
		if (SemanticToSearch.Len() == SemanticLen - 3 && FCString::Strncmp(TEXT("[0]"), &SourceLine[StartPosition + SemanticLen - 3], 3) == 0 && FCString::Strnicmp(*SemanticToSearch, &SourceLine[StartPosition], SemanticLen - 3) == 0)
		{
			return true;
		}

		return false;
	};

	// Returns true if the specified semantic name starts with "SV_" (case insensitive).
	auto IsSemanticSystemValue = [](const FString& SemanticName) -> bool
	{
		return SemanticName.StartsWith(TEXT("SV_"), 3, ESearchCase::IgnoreCase);
	};

	auto BuildSortedStageVariableDeclSource = [&IsSemanticSystemValue, &FindSemanticDeclarationInSourceLine](
		FString& OutStageVariableDeclSource, TArray<FString>& StageVariableDeclLines, const TArray<FString>& InVariables)
	{
		for (const FString& Variable : InVariables)
		{
			for (FString& SourceLine : StageVariableDeclLines)
			{
				// Search for semantic name (always case insensitive) in current stage variable source line
				if (FindSemanticDeclarationInSourceLine(SourceLine, Variable))
				{
					// Append source line for current variable at the end of sorted declaration string.
					// Then empty this source line to avoid unnecessary string comparisons for next variables.
					OutStageVariableDeclSource += SourceLine;
					OutStageVariableDeclSource += TEXT('\n');
					SourceLine.Empty();
					break;
				}
			}
		}
	};

	// Re-arrange source lines of stage variable declarations and always emit system values last
	FString SortedStageVariableDeclSource = TEXT("\n");

	BuildSortedStageVariableDeclSource(SortedStageVariableDeclSource, StageVariableDeclSourceLines, Variables);

	// Replace old declaration with sorted one
	HlslSourceString.RemoveAt(StageVariableDelcarationBlockBegin + 1, StageVariableDelcarationBlockEnd - (StageVariableDelcarationBlockBegin + 1));
	HlslSourceString.InsertAt(StageVariableDelcarationBlockBegin + 1, SortedStageVariableDeclSource);

	return true;
}

// @todo-lh: use ANSI string class whenever UE core gets one
static void PatchHlslForPrecompilation(
	TArray<ANSICHAR>& HlslSource,
	const EShaderFrequency Frequency,
	const FString& OriginalShaderSource,
	const FString& OriginalEntryPoint,
	TArray<FShaderCompilerError>& OutErrors)
{
	FString HlslSourceString;

	// Disable some warnings that might be introduced by cross-compiled HLSL, we only want to see those warnings from the original source and not from intermediate high-level source
	HlslSourceString += TEXT("#pragma warning(disable : 3571) // pow() intrinsic suggested to be used with abs()\n");

	// Append original cross-compiled source code
	HlslSourceString += ANSI_TO_TCHAR(HlslSource.GetData());

	// Patch SPIRV-Cross renaming to retain original member names in RootShaderParameters cbuffer
	const int32 RootShaderParameterSourceLocation = HlslSourceString.Find("cbuffer RootShaderParameters");
	if (RootShaderParameterSourceLocation != INDEX_NONE)
	{
		HlslSourceString.ReplaceInline(TEXT("cbuffer RootShaderParameters"), TEXT("cbuffer _RootShaderParameters"), ESearchCase::CaseSensitive);
		HlslSourceString.ReplaceInline(TEXT("_RootShaderParameters_"), TEXT(""), ESearchCase::CaseSensitive);
	}

	// Patch separation of atomic counters: replace declarations of all counter_var_... declarations by their original buffer resource.
	const FString CounterPrefix = TEXT("counter_var_");
	const FString CounterDeclPrefix = TEXT("RWByteAddressBuffer ") + CounterPrefix;

	for (int32 ReadPos = 0, NextReadPos = 0;
		 (NextReadPos = HlslSourceString.Find(CounterDeclPrefix, ESearchCase::CaseSensitive, ESearchDir::FromStart, ReadPos)) != INDEX_NONE;
		 ReadPos = NextReadPos)
	{
		// Find original resource name without "counter_var_" prefix
		const int32 ResourceNameStartPos = NextReadPos + CounterDeclPrefix.Len();
		const int32 ResourceNameEndPos = HlslSourceString.Find(TEXT(";"), ESearchCase::CaseSensitive, ESearchDir::FromStart, ResourceNameStartPos);
		if (ResourceNameEndPos != INDEX_NONE)
		{
			const FString ResourceName = HlslSourceString.Mid(NextReadPos + CounterDeclPrefix.Len(), ResourceNameEndPos - ResourceNameStartPos);
			const FString ResourceCounterName = HlslSourceString.Mid(NextReadPos + CounterDeclPrefix.Len() - CounterPrefix.Len(), ResourceNameEndPos - ResourceNameStartPos + CounterPrefix.Len());

			// Remove current "RWByteAddressBuffer counter_var_*;" resource declaration line
			HlslSourceString.RemoveAt(NextReadPos, ResourceNameEndPos - NextReadPos + 1);

			// Remove all "counter_var_" prefixes for the current resource
			HlslSourceString.ReplaceInline(*ResourceCounterName, *ResourceName, ESearchCase::CaseSensitive);
		}
	}

	if (Frequency == SF_Vertex)
	{
		// Ensure order of output variables remains the same as declared in original shader source
		PatchHlslWithReorderedIOVariables(HlslSourceString, OriginalShaderSource, OriginalEntryPoint, UE::HlslParser::EShaderParameterStorageClass::Output, OutErrors);
	}
	else if (Frequency == SF_Pixel)
	{
		// Patch internal error when SV_DepthLessEqual or SV_DepthGreaterEqual is specified in a pixel shader output. This is to prevent the following internal error:
		//	error X8000 : D3D11 Internal Compiler Error : Invalid Bytecode : Interpolation mode for PS input position must be
		//				  linear_noperspective_centroid or linear_noperspective_sample when outputting oDepthGE or oDepthLE and
		//				  not running at sample frequency(which is forced by inputting SV_SampleIndex or declaring an input linear_sample or linear_noperspective_sample).
		if (HlslSourceString.Find(TEXT("SV_DepthLessEqual"), ESearchCase::CaseSensitive) != INDEX_NONE ||
			HlslSourceString.Find(TEXT("SV_DepthGreaterEqual"), ESearchCase::CaseSensitive) != INDEX_NONE)
		{
			// Ensure the interpolation mode is linear_noperspective_sample by adding "sample" specifier to one of the input-interpolators that have a floating-point type
			const int32 FragCoordStringPosition = HlslSourceString.Find(TEXT("float4 gl_FragCoord : SV_Position"), ESearchCase::CaseSensitive);
			if (FragCoordStringPosition != INDEX_NONE)
			{
				HlslSourceString.InsertAt(FragCoordStringPosition, TEXT("sample "));
			}
		}

		// Ensure order of input variables remains the same as declared in original shader source
		PatchHlslWithReorderedIOVariables(HlslSourceString, OriginalShaderSource, OriginalEntryPoint, UE::HlslParser::EShaderParameterStorageClass::Input, OutErrors);
	}

	// Return new HLSL source
	HlslSource.SetNum(HlslSourceString.Len() + 1);
	FMemory::Memcpy(HlslSource.GetData(), TCHAR_TO_ANSI(*HlslSourceString), HlslSourceString.Len());
	HlslSource[HlslSourceString.Len()] = '\0';
}

// Returns whether the specified D3D compiler error buffer contains any internal error messages, e.g. "internal error: out of memory"
static bool CompileErrorsContainInternalError(ID3DBlob* Errors)
{
	if (Errors)
	{
		if (void* ErrorBuffer = Errors->GetBufferPointer())
		{
			const ANSICHAR* ErrorString = reinterpret_cast<const ANSICHAR*>(ErrorBuffer);
			return
				FCStringAnsi::Strstr(ErrorString, "internal error:") != nullptr ||
				FCStringAnsi::Strstr(ErrorString, "Internal Compiler Error:") != nullptr;
		}
	}
	return false;
}

static bool D3DCompileErrorContainsValidationErrors(ID3DBlob* ErrorBlob)
{
	if (ErrorBlob != nullptr)
	{
		const FAnsiStringView ErrorString((const ANSICHAR*)ErrorBlob->GetBufferPointer(), (int32)ErrorBlob->GetBufferSize());
		return (ErrorString.Find(ANSITEXTVIEW("error X8000: Validation Error:")) != INDEX_NONE);
	}
	return false;
}

// Generate the dumped usf file; call the D3D compiler, gather reflection information and generate the output data
static bool CompileAndProcessD3DShaderFXCExt(
	uint32 CompileFlags,
	const FShaderCompilerInput& Input,
	const FString& PreprocessedShaderSource,
	const FString& EntryPointName,
	const FShaderParameterParser& ShaderParameterParser,
	const TCHAR* ShaderProfile,
	const ED3DShaderModel ShaderModel, 
	const bool bSecondPassAferUnusedInputRemoval,
	FShaderCompilerOutput& Output)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CompileAndProcessD3DShaderFXCExt);

	auto AnsiSourceFile = StringCast<ANSICHAR>(*PreprocessedShaderSource);

	bool bDumpDebugInfo = Input.DumpDebugInfoEnabled();
	if (bDumpDebugInfo)
	{
		FString BatchFileContents;
		FString Filename = Input.GetSourceFilename();
		BatchFileContents = D3D11CreateShaderCompileCommandLine(Filename, *EntryPointName, ShaderProfile, CompileFlags, Output);

		if (GD3DDumpAMDCodeXLFile)
		{
			const FString BatchFileContents2 = CreateAMDCodeXLCommandLine(Filename, *EntryPointName, ShaderProfile, CompileFlags);
			FFileHelper::SaveStringToFile(BatchFileContents2, *(Input.DumpDebugInfoPath / TEXT("CompileAMD.bat")));
		}

		FFileHelper::SaveStringToFile(BatchFileContents, *(Input.DumpDebugInfoPath / TEXT("CompileFXC.bat")));
	}

	TRefCountPtr<ID3DBlob> Shader;

	HRESULT Result = S_OK;
	pD3DCompile D3DCompileFunc = FxcCompilerFunctions::GetCompile();
	pD3DReflect D3DReflectFunc = FxcCompilerFunctions::GetReflect();
	pD3DDisassemble D3DDisassembleFunc = FxcCompilerFunctions::GetDisassemble();
	pD3DStripShader D3DStripShaderFunc = FxcCompilerFunctions::GetStripShader();
	pD3DGetBlobPart D3DGetBlobPartFunc = FxcCompilerFunctions::GetGetBlobPart();
	pD3DGetDebugInfo D3DGetDebugInfoFunc = FxcCompilerFunctions::GetGetDebugInfo();

	TRefCountPtr<ID3DBlob> Errors;

	if (D3DCompileFunc)
	{
		TArray<FString> InitialFXCRunFilteredErrors;
		const bool bHlslVersion2021 = Input.Environment.CompilerFlags.Contains(CFLAG_HLSL2021);
		const bool bPrecompileWithDXC = bHlslVersion2021 || Input.Environment.CompilerFlags.Contains(CFLAG_PrecompileWithDXC);
		if (!bPrecompileWithDXC)
		{
			Result = D3DCompileWrapper(
				D3DCompileFunc,
				AnsiSourceFile.Get(),
				AnsiSourceFile.Length(),
				TCHAR_TO_ANSI(*Input.VirtualSourceFilePath),
				/*pDefines=*/ NULL,
				/*pInclude=*/ NULL,
				TCHAR_TO_ANSI(*EntryPointName),
				TCHAR_TO_ANSI(ShaderProfile),
				CompileFlags,
				0,
				Shader.GetInitReference(),
				Errors.GetInitReference(),
				// We only want to catch the exception on initial FXC compiles so we can retry with a 
				// DXC precompilation step. If it fails again on the second attempt then we let
				// ShaderCompileWorker handle the exception and log an error.
				/* bCatchException */ true
			);

			if (Result == E_FAIL)
			{
				// We might have failed compiling with FXC but then we might actually manage to compile through DXC
				if (void* ErrorBuffer = Errors ? Errors->GetBufferPointer() : nullptr)
				{
					D3D11FilterShaderCompileWarnings(ANSI_TO_TCHAR(ErrorBuffer), InitialFXCRunFilteredErrors);
				}
			}
		}

		// Some materials give FXC a hard time to optimize and the compiler fails with an internal error.
		const bool bFxcCompilerFailed = Result == HRESULT_FROM_WIN32(ERROR_ARITHMETIC_OVERFLOW) || Result == E_OUTOFMEMORY || Result == E_FAIL || (Result != S_OK && CompileErrorsContainInternalError(Errors.GetReference()));
		if (bPrecompileWithDXC || bFxcCompilerFailed)
		{
			// If we're cross-compiling HLSL to simplified HLSL due to an FXC error, force diagnostics output and dump debug information.
			// This is meant to help track down potential issues that only occur at PSO creation time.
			if (bFxcCompilerFailed)
			{
				Output.bForceDumpDebugInfo = true;
				Output.bForceLogDiagnosticMessages = true;
			}

			// If we ran out of memory, it's likely the next attempt will crash, too.
			// Report the error now in case CompileHlslToSpirv throws an exception.
			if (Result == E_OUTOFMEMORY)
			{
				FSCWErrorCode::Report(FSCWErrorCode::OutOfMemory);
			}

			CrossCompiler::FShaderConductorContext CompilerContext;

			auto FlushInitialFxcAndNewDxcErrors = [&CompilerContext, &InitialFXCRunFilteredErrors, &Output]() -> void
			{
				// Flush new errors from DXC
				CompilerContext.FlushErrors(Output.Errors);

				// Also flush initial errors from FXC
				Algo::Transform(InitialFXCRunFilteredErrors, Output.Errors, [](const FString& InInitialError) { return FShaderCompilerError(*InInitialError); });
			};

			// Load shader source into compiler context
			const EShaderFrequency Frequency = (EShaderFrequency)Input.Target.Frequency;
			CompilerContext.LoadSource(PreprocessedShaderSource, Input.VirtualSourceFilePath, EntryPointName, Frequency);

			// Compile HLSL source to SPIR-V binary
			CrossCompiler::FShaderConductorOptions Options;
			Options.bWarningsAsErrors = Input.Environment.CompilerFlags.Contains(CFLAG_WarningsAsErrors);
			Options.bPreserveStorageInput = true; // Input/output stage variables must match
			if (bHlslVersion2021)
			{
				Options.HlslVersion = 2021;
			}

			FSpirv Spirv;
			if (!CompilerContext.CompileHlslToSpirv(Options, Spirv.Data))
			{
				FlushInitialFxcAndNewDxcErrors();
				return false;
			}

			// Cross-compile back to HLSL
			CrossCompiler::FShaderConductorTarget TargetDesc;
			TargetDesc.Language = CrossCompiler::EShaderConductorLanguage::Hlsl;
			TargetDesc.Version = 50;

			TargetDesc.CompileFlags->SetDefine(TEXT("implicit_resource_binding"), 1);
			TargetDesc.CompileFlags->SetDefine(TEXT("reconstruct_global_uniforms"), 1);
			TargetDesc.CompileFlags->SetDefine(TEXT("reconstruct_cbuffer_names"), 1);
			TargetDesc.CompileFlags->SetDefine(TEXT("reconstruct_semantics"), 1);
			TargetDesc.CompileFlags->SetDefine(TEXT("force_zero_initialized_variables"), 1);
			TargetDesc.CompileFlags->SetDefine(TEXT("relax_nan_checks"), 1);
			TargetDesc.CompileFlags->SetDefine(TEXT("preserve_structured_buffers"), 1);

			// Patch SPIR-V for workarounds to prevent potential additional FXC failures
			PatchSpirvForPrecompilation(Spirv);

			TArray<ANSICHAR> CrossCompiledSource;
			if (!CompilerContext.CompileSpirvToSourceAnsi(Options, TargetDesc, Spirv.GetByteData(), Spirv.GetByteSize(), CrossCompiledSource))
			{
				FlushInitialFxcAndNewDxcErrors();
				return false;
			}

			// Patch HLSL for workarounds to prevent potential additional FXC failures
			PatchHlslForPrecompilation(CrossCompiledSource, Frequency, PreprocessedShaderSource, EntryPointName, Output.Errors);

			if (bDumpDebugInfo && CrossCompiledSource.Num() > 1)
			{
				DumpDebugShaderDisassembledSpirv(Input, Spirv.GetByteData(), Spirv.GetByteSize(), TEXT("intermediate.spvasm"));
				DumpDebugShaderText(Input, CrossCompiledSource.GetData(), CrossCompiledSource.Num() - 1, TEXT("intermediate.hlsl"));
			}

			// Generates an virtual source file path with the ".intermediate." suffix injected.
			auto MakeIntermediateVirtualSourceFilePath = [](const FString& InVirtualSourceFilePath) -> FString
			{
				FString PathPart, FilenamePart, ExtensionPart;
				FPaths::Split(InVirtualSourceFilePath, PathPart, FilenamePart, ExtensionPart);
				return FPaths::Combine(PathPart, FilenamePart) + TEXT(".intermediate.") + ExtensionPart;
			};

			const FString CrossCompiledSourceFilename = MakeIntermediateVirtualSourceFilePath(Input.VirtualSourceFilePath);
			auto ShaderProfileAnsi = StringCast<ANSICHAR>(ShaderProfile);
			auto CrossCompiledSourceFilenameAnsi = StringCast<ANSICHAR>(*CrossCompiledSourceFilename);

			// SPIRV-Cross will have generated the new shader with "main" as the new entry point.
			auto CompileCrossCompiledHlsl = [&D3DCompileFunc, &CrossCompiledSourceFilenameAnsi, &Shader, &Errors, &ShaderProfileAnsi](const TArray<ANSICHAR>& Source, uint32 CompileFlags, const ANSICHAR* EntryPoint = "main") -> HRESULT
			{
				checkf(Source.Num() > 0, TEXT("TArray<ANSICHAR> of cross-compiled HLSL source must have at least one element including the NUL-terminator"));
				return D3DCompileWrapper(
					D3DCompileFunc,
					Source.GetData(),
					static_cast<SIZE_T>(Source.Num() - 1),
					CrossCompiledSourceFilenameAnsi.Get(),
					/*pDefines=*/ NULL,
					/*pInclude=*/ NULL,
					EntryPoint,
					ShaderProfileAnsi.Get(),
					CompileFlags,
					0,
					Shader.GetInitReference(),
					Errors.GetInitReference()
				);
			};

			if (Input.Environment.CompilerFlags.Contains(CFLAG_OutputAnalysisArtifacts) && !bSecondPassAferUnusedInputRemoval)
			{
				// Add disassembly string to reflection output
				Output.AddStatistic<FString>(TEXT("Optimized HLSL-2018"), ANSI_TO_TCHAR(CrossCompiledSource.GetData()), FGenericShaderStat::EFlags::Hidden, FShaderStatTagNames::AnalysisArtifactsName);
			}

			// Compile again with FXC - 1st try
			const uint32 CompileFlagsNoWarningsAsErrors = CompileFlags & (~D3DCOMPILE_WARNINGS_ARE_ERRORS);
			Result = CompileCrossCompiledHlsl(CrossCompiledSource, CompileFlagsNoWarningsAsErrors);

			// If FXC compilation failed with a validation error, assume bug in FXC's optimization passes
			// Compile again with FXC and disable special compiler rule to simplify control flow - 2nd try
			if (Result == E_FAIL && D3DCompileErrorContainsValidationErrors(Errors.GetReference()))
			{
				Output.AddErrorf(TEXT("Validation error in FXC encountered: Compiling intermediate HLSL a second time with simplified control flow"));

				// Rule 0x08024065 is described as "simplify flow control that writes the same value in each flow control path"
				const FAnsiStringView PragmaDirectiveCode = "#pragma ruledisable 0x08024065\n";
				CrossCompiledSource.Insert(PragmaDirectiveCode.GetData(), PragmaDirectiveCode.Len(), 0);

				Result = CompileCrossCompiledHlsl(CrossCompiledSource, CompileFlagsNoWarningsAsErrors);

				// If FXC compilation still fails with a validation error, compile again and skip optimizations entirely as a last resort - 3rd try
				if (Result == E_FAIL && D3DCompileErrorContainsValidationErrors(Errors.GetReference()))
				{
					Output.AddErrorf(TEXT("Validation error in FXC encountered: Compiling intermediate HLSL a third time without optimization (D3DCOMPILE_SKIP_OPTIMIZATION)"));

					const uint32 CompileFlagsSkipOptimizations = CompileFlagsNoWarningsAsErrors | D3DCOMPILE_SKIP_OPTIMIZATION;
					Result = CompileCrossCompiledHlsl(CrossCompiledSource, CompileFlagsSkipOptimizations);
				}
			}

			if (bFxcCompilerFailed && SUCCEEDED(Result))
			{
				// Reset our previously set error code
				FSCWErrorCode::Reset();

				// Let the user know this shader had to be cross-compiled due to a crash in FXC. Only shows up if CVar 'r.ShaderDevelopmentMode' is enabled.
				Output.AddErrorf(TEXT("Cross-compiled shader to intermediate HLSL after first attempt crashed FXC"));

				// Output the errors from the initial run so that the user can know what failed and eventually correct it or at least make an informed decision about whether to bypass the error using DXC pre-compilation:
				Algo::Transform(InitialFXCRunFilteredErrors, Output.Errors, [](const FString& InInitialError) { return FShaderCompilerError(*InInitialError); });
			}
		}
	}
	else
	{
		Output.AddErrorf(TEXT("Couldn't find D3D shader compiler DLL"));
		Result = E_FAIL;
	}

	// Filter any errors.
	TArray<FString> FilteredErrors;
	void* ErrorBuffer = Errors ? Errors->GetBufferPointer() : NULL;
	if (ErrorBuffer)
	{
		D3D11FilterShaderCompileWarnings(ANSI_TO_TCHAR(ErrorBuffer), FilteredErrors);

		// Process errors
		for (int32 ErrorIndex = 0; ErrorIndex < FilteredErrors.Num(); ErrorIndex++)
		{
			const FString& CurrentError = FilteredErrors[ErrorIndex];
			FShaderCompilerError NewError;

			// Extract filename and line number from FXC output with format:
			// "d:\Project\Binaries\BasePassPixelShader(30,7): error X3000: invalid target or usage string"
			int32 FirstParenIndex = CurrentError.Find(TEXT("("));
			int32 LastParenIndex = CurrentError.Find(TEXT("):"));
			if (FirstParenIndex != INDEX_NONE &&
				LastParenIndex != INDEX_NONE &&
				LastParenIndex > FirstParenIndex)
			{
				// Extract and store error message with source filename
				NewError.ErrorVirtualFilePath = CurrentError.Left(FirstParenIndex);
				NewError.ErrorLineString = CurrentError.Mid(FirstParenIndex + 1, LastParenIndex - FirstParenIndex - FCString::Strlen(TEXT("(")));
				NewError.StrippedErrorMessage = CurrentError.Right(CurrentError.Len() - LastParenIndex - FCString::Strlen(TEXT("):")));
			}
			else
			{
				NewError.StrippedErrorMessage = CurrentError;
			}
			Output.Errors.Emplace(MoveTemp(NewError));
		}
	}

	// Fail the compilation if certain extended features are being used, since those are not supported on all D3D11 cards.
	if (SUCCEEDED(Result) && D3DDisassembleFunc)
	{
		const bool bCheckForTypedUAVs = !Input.Environment.CompilerFlags.Contains(CFLAG_AllowTypedUAVLoads);
		if (GD3DCheckForDoubles || bCheckForTypedUAVs || bDumpDebugInfo)
		{
			TRefCountPtr<ID3DBlob> Disassembly;
			if (SUCCEEDED(D3DDisassembleFunc(Shader->GetBufferPointer(), Shader->GetBufferSize(), 0, "", Disassembly.GetInitReference())))
			{
				ANSICHAR* DisassemblyString = new ANSICHAR[Disassembly->GetBufferSize() + 1];
				FMemory::Memcpy(DisassemblyString, Disassembly->GetBufferPointer(), Disassembly->GetBufferSize());
				DisassemblyString[Disassembly->GetBufferSize()] = 0;
				FString DisassemblyStringW(DisassemblyString);
				delete[] DisassemblyString;

				if (bDumpDebugInfo)
				{
					FFileHelper::SaveStringToFile(DisassemblyStringW, *(Input.DumpDebugInfoPath / TEXT("Output.d3dasm")));
				}

				if (GD3DCheckForDoubles)
				{
					// dcl_globalFlags will contain enableDoublePrecisionFloatOps when the shader uses doubles, even though the docs on dcl_globalFlags don't say anything about this
					if (DisassemblyStringW.Contains(TEXT("enableDoublePrecisionFloatOps")))
					{
						Output.AddErrorf(TEXT("Shader uses double precision floats, which are not supported on all D3D11 hardware!"));
						return false;
					}
				}
					
				if (bCheckForTypedUAVs)
				{
					// Disassembly will contain this text with typed loads from UAVs are used where the format and dimension are not fully supported
					// across all versions of Windows (like Windows 7/8.1).
					// https://microsoft.github.io/DirectX-Specs/d3d/UAVTypedLoad.html
					// https://docs.microsoft.com/en-us/windows/win32/direct3d12/typed-unordered-access-view-loads
					// https://docs.microsoft.com/en-us/windows/win32/direct3ddxgi/format-support-for-direct3d-11-0-feature-level-hardware
					if (DisassemblyStringW.Contains(TEXT("Typed UAV Load Additional Formats")))
					{
						Output.AddErrorf(TEXT("Shader uses UAV loads from additional typed formats, which are not supported on all D3D11 hardware! Set r.D3D.CheckedForTypedUAVs=0 if you want to allow typed UAV loads for your project, or individual shaders can opt-in by specifying CFLAG_AllowTypedUAVLoads."));
						return false;
					}
				}
			}
		}
	}

	// Gather reflection information
	TArray<FString> ShaderInputs;

	if (SUCCEEDED(Result))
	{
		FD3DShaderCompileData CompileData;
		// D3D11RHI uses the D3D11_ defines, but we want to enforce the engine limits as well.
		CompileData.MaxSamplers = FMath::Min(D3D11_COMMONSHADER_SAMPLER_SLOT_COUNT,             MAX_SAMPLERS);
		CompileData.MaxSRVs     = FMath::Min(D3D11_COMMONSHADER_INPUT_RESOURCE_SLOT_COUNT,      MAX_SRVS);
		CompileData.MaxCBs      = FMath::Min(D3D11_COMMONSHADER_CONSTANT_BUFFER_API_SLOT_COUNT, MAX_CBS);
		CompileData.MaxUAVs     = FMath::Min(D3D11_PS_CS_UAV_REGISTER_COUNT,                    MAX_UAVS);

		if (D3DReflectFunc)
		{
			Output.bSucceeded = true;
			TRefCountPtr<ID3D11ShaderReflection> Reflector;
			
			Result = D3DReflectFunc(Shader->GetBufferPointer(), Shader->GetBufferSize(), IID_ID3D11ShaderReflectionForCurrentCompiler, (void**)Reflector.GetInitReference());
			if (FAILED(Result))
			{
				UE_LOGF(LogD3DShaderCompiler, Fatal, "D3DReflect failed: Result=%08x", Result);
			}

			// Read the constant table description.
			D3D11_SHADER_DESC ShaderDesc;
			Reflector->GetDesc(&ShaderDesc);

			// Remove unused interpolators from pixel shader 
			// (propagated to corresponding VS from pipeline by later setting Output.bSupportsQueryingUsedAttributes and Output.UsedAttributes)
			{
				ShaderCompileLambdaType ShaderCompileLambda = [CompileFlags](
					const FShaderCompilerInput& Input,
					const FString& PreprocessedShaderSource,
					const FString& EntryPointName,
					const FShaderParameterParser& ShaderParameterParser,
					const TCHAR* ShaderProfile,
					const ED3DShaderModel ShaderModel,
					const bool bProcessingSecondTime,
					FShaderCompilerOutput& Output)
					{
						return CompileAndProcessD3DShaderFXCExt(
							CompileFlags,
							Input,
							PreprocessedShaderSource,
							EntryPointName,
							ShaderParameterParser,
							ShaderProfile,
							ShaderModel,
							bProcessingSecondTime,
							Output);
					};

				bool CompileResult = false;
				const bool RemovedUnusedInterpolatorsApplied =
					RemoveUnusedInterpolators<ShaderCompilerType::FXC, ShaderCompileLambdaType,
					ID3D11ShaderReflection, D3D11_SHADER_DESC, D3D11_SIGNATURE_PARAMETER_DESC>(
						Input,
						PreprocessedShaderSource,
						EntryPointName,
						ShaderParameterParser,
						ShaderProfile,
						ShaderModel,
						bSecondPassAferUnusedInputRemoval,
						CompileData,
						Reflector,
						ShaderCompileLambda,
						Output,
						CompileResult);
				if (RemovedUnusedInterpolatorsApplied)
				{
					return CompileResult;
				}
			}

			const uint32 BindingSpace = 0; // Default binding space for D3D11 shaders
			ExtractParameterMapFromD3DShader<
				ID3D11ShaderReflection, D3D11_SHADER_DESC, D3D11_SHADER_INPUT_BIND_DESC,
				ID3D11ShaderReflectionConstantBuffer, D3D11_SHADER_BUFFER_DESC,
				ID3D11ShaderReflectionVariable, D3D11_SHADER_VARIABLE_DESC>(
					Input,
					ShaderParameterParser,
					BindingSpace,
					Reflector,
					ShaderDesc,
					CompileData,
					Output
				);
		}
		else
		{
			Output.AddErrorf(TEXT("Couldn't find shader reflection function in D3D Compiler DLL"));
			Result = E_FAIL;
			Output.bSucceeded = false;
		}
		
		if (!ValidateResourceCounts(CompileData, Output.Errors))
		{
			Result = E_FAIL;
			Output.bSucceeded = false;
		}

		// Check for resource limits for feature level 11.0
		if (CompileData.NumUAVs > GD3DMaximumNumUAVs)
		{
			Output.AddErrorf(TEXT("Number of UAVs exceeded limit: %d slots used, but limit is %d due to maximum feature level 11.0"), CompileData.NumUAVs, GD3DMaximumNumUAVs);
			Result = E_FAIL;
			Output.bSucceeded = false;
		}

		// Save results if compilation and reflection succeeded
		if (Output.bSucceeded)
		{
			if (Input.Environment.CompilerFlags.Contains(CFLAG_OutputAnalysisArtifacts) && !bSecondPassAferUnusedInputRemoval)
			{
				TRefCountPtr<ID3DBlob> Disassembly;
				if (SUCCEEDED(D3DDisassembleFunc(Shader->GetBufferPointer(), Shader->GetBufferSize(), 0, "", Disassembly.GetInitReference())))
				{
					// Convert disassembly from ID3DBlob to FString
					TArray<ANSICHAR> DisassemblyAnsiString;
					DisassemblyAnsiString.SetNumUninitialized(Disassembly->GetBufferSize() + 1);
					FMemory::Memcpy(DisassemblyAnsiString.GetData(), Disassembly->GetBufferPointer(), Disassembly->GetBufferSize());
					DisassemblyAnsiString[Disassembly->GetBufferSize()] = '\0';
					FString DisassemblyString(DisassemblyAnsiString);

					// Add disassembly string to reflection output
					Output.AddStatistic<FString>(TEXT("DXBC"), MoveTemp(DisassemblyString), FGenericShaderStat::EFlags::Hidden, FShaderStatTagNames::AnalysisArtifactsName);
				}
			}

			TRefCountPtr<ID3DBlob> CompressedData;
			TRefCountPtr<ID3DBlob> DebugDataBlob;
			TRefCountPtr<ID3DBlob> DebugNameBlob;
			
			bool bGenerateSymbolsInfo = Input.Environment.CompilerFlags.Contains(CFLAG_GenerateSymbolsInfo);
			bool bGenerateSymbols = Input.Environment.CompilerFlags.Contains(CFLAG_GenerateSymbols);
			if ((bGenerateSymbols || bGenerateSymbolsInfo) && D3DGetBlobPartFunc && D3DGetDebugInfoFunc)
			{
				Result = D3DGetBlobPartFunc(Shader->GetBufferPointer(), Shader->GetBufferSize(), D3D_BLOB_DEBUG_NAME, 0u, DebugNameBlob.GetInitReference());

				if (SUCCEEDED(Result))
				{
					// copypasta from https://devblogs.microsoft.com/pix/using-automatic-shader-pdb-resolution-in-pix/
					struct ShaderDebugName
					{
						uint16_t Flags;       // Reserved, must be set to zero.
						uint16_t NameLength;  // Length of the debug name, without null terminator.
						// Followed by NameLength bytes of the UTF-8-encoded name.
						// Followed by a null terminator.
						// Followed by [0-3] zero bytes to align to a 4-byte boundary.
					};

					const ShaderDebugName* DebugNameData = reinterpret_cast<const ShaderDebugName*>(DebugNameBlob->GetBufferPointer());
					const char* Name = reinterpret_cast<const char*>(DebugNameData + 1);

					FD3DShaderDebugData DebugData;
					FD3DShaderDebugData::FFile& PdbFile = DebugData.Files.AddDefaulted_GetRef();
					PdbFile.Name = Name;

					if (bGenerateSymbols)
					{
						Result = D3DGetBlobPartFunc(Shader->GetBufferPointer(), Shader->GetBufferSize(), D3D_BLOB_PDB, 0u, DebugDataBlob.GetInitReference());
						if (SUCCEEDED(Result) && DebugDataBlob->GetBufferSize() > 0)
						{
							PdbFile.Contents = MakeArrayViewFromBlob(DebugDataBlob);
						}
						else
						{
							Output.AddErrorf(TEXT("Symbol generation was requested, but no PDB blob exists in the compiler output."));
							Result = E_FAIL;
							Output.bSucceeded = false;
						}
					}

					FMemoryWriter Ar(Output.ShaderCode.GetSymbolWriteAccess());
					Ar << DebugData;
				}
				else
				{
					Output.AddErrorf(TEXT("Symbol or symbols info generation was requested, but no debug name blob exists in the compiler output."));
					Output.bSucceeded = false;
				}
			}

			if (D3DStripShaderFunc)
			{
				// Strip shader reflection and debug info
				D3D_SHADER_DATA ShaderData;
				ShaderData.pBytecode = Shader->GetBufferPointer();
				ShaderData.BytecodeLength = Shader->GetBufferSize();
				Result = D3DStripShaderFunc(Shader->GetBufferPointer(),
					Shader->GetBufferSize(),
					D3DCOMPILER_STRIP_REFLECTION_DATA | D3DCOMPILER_STRIP_DEBUG_INFO | D3DCOMPILER_STRIP_TEST_BLOBS,
					CompressedData.GetInitReference());

				if (FAILED(Result))
				{
					UE_LOGF(LogD3DShaderCompiler, Fatal, "D3DStripShader failed: Result=%08x", Result);
				}
			}
			else
			{
				// D3DStripShader is not guaranteed to exist
				// e.g. the open-source DXIL shader compiler does not currently implement it
				CompressedData = Shader;
			}

			// Add resource masks before the parameters are pulled for the uniform buffers
			FShaderCodeResourceMasks ResourceMasks{};
			for (const auto& Param : Output.ParameterMap.GetParameterMap())
			{
				const FParameterAllocation& ParamAlloc = Param.Value;
				if (ParamAlloc.Type == EShaderParameterType::UAV)
				{
					ResourceMasks.UAVMask |= 1u << ParamAlloc.BaseIndex;
				}
			}

			auto AddOptionalDataCallback = [&](FShaderCode& ShaderCode)
			{
				Output.ShaderCode.AddOptionalData(ResourceMasks);
			};

			FShaderCodePackedResourceCounts PackedResourceCounts = InitPackedResourceCounts(CompileData);

			GenerateFinalOutput(
				CompressedData,
				Input,
				ED3DShaderModel::SM5_0,
				bSecondPassAferUnusedInputRemoval,
				CompileData,
				PackedResourceCounts,
				Output,
				[](FMemoryWriter&){},
				AddOptionalDataCallback
			);
		}
	}

	return SUCCEEDED(Result);
}

bool CompileAndProcessD3DShaderFXC(
	const FShaderCompilerInput& Input,
	const FString& InPreprocessedSource,
	const FString& InEntryPointName,
	const FShaderParameterParser& ShaderParameterParser,
	const TCHAR* ShaderProfile,
	const ED3DShaderModel ShaderModel,
	const bool bSecondPassAferUnusedInputRemoval,
	FShaderCompilerOutput& Output)
{
	// @TODO - implement different material path to allow us to remove backwards compatibility flag on sm5 shaders
	uint32 CompileFlags = D3DCOMPILE_ENABLE_BACKWARDS_COMPATIBILITY
		// Unpack uniform matrices as row-major to match the CPU layout.
		| D3DCOMPILE_PACK_MATRIX_ROW_MAJOR;

	bool bGenerateSymbols = Input.Environment.CompilerFlags.Contains(CFLAG_GenerateSymbols);
	bool bGenerateSymbolsInfo = Input.Environment.CompilerFlags.Contains(CFLAG_GenerateSymbolsInfo);

	if (bGenerateSymbols || bGenerateSymbolsInfo)
	{
		CompileFlags |= D3DCOMPILE_DEBUG;

		if (Input.Environment.CompilerFlags.Contains(CFLAG_AllowUniqueSymbols))
		{
			CompileFlags |= D3DCOMPILE_DEBUG_NAME_FOR_SOURCE;
		}
		else
		{
			CompileFlags |= D3DCOMPILE_DEBUG_NAME_FOR_BINARY;
		}
	}

	if (Input.Environment.CompilerFlags.Contains(CFLAG_Debug))
	{
		CompileFlags |= D3DCOMPILE_SKIP_OPTIMIZATION;
	}
	else if (Input.Environment.CompilerFlags.Contains(CFLAG_StandardOptimization))
	{
		CompileFlags |= D3DCOMPILE_OPTIMIZATION_LEVEL1;
	}
	else
	{
		CompileFlags |= D3DCOMPILE_OPTIMIZATION_LEVEL3;
	}

	Input.Environment.CompilerFlags.Iterate([&CompileFlags](uint32 Flag)
		{
			CompileFlags |= TranslateCompilerFlagD3D11((ECompilerFlags)Flag);
		});

	const bool bSuccess = CompileAndProcessD3DShaderFXCExt(CompileFlags, Input, InPreprocessedSource, InEntryPointName, ShaderParameterParser, ShaderProfile, ShaderModel, false, Output);

	return bSuccess;
}

struct FD3DShaderParameterParserPlatformConfiguration : public FShaderParameterParser::FPlatformConfiguration
{
	FD3DShaderParameterParserPlatformConfiguration(const FShaderCompilerInput& Input)
		: FShaderParameterParser::FPlatformConfiguration(TEXTVIEW("cbuffer"), EShaderParameterParserConfigurationFlags::UseStableConstantBuffer|EShaderParameterParserConfigurationFlags::SupportsBindless)
		, bIsRayTracingShader(Input.IsRayTracingShader())
		, HitGroupSystemIndexBufferName(FShaderParameterParser::kBindlessSRVPrefix + FString(TEXT("HitGroupSystemIndexBuffer")))
		, HitGroupSystemVertexBufferName(FShaderParameterParser::kBindlessSRVPrefix + FString(TEXT("HitGroupSystemVertexBuffer")))
	{
	}

	virtual FString GenerateBindlessAccess(EBindlessConversionType BindlessType, FStringView FullTypeString, FStringView ArrayNameOverride, FStringView IndexString) const final
	{
		// GetSRVFromHeap(Type, Index) ResourceDescriptorHeap[Index]
		// GetUAVFromHeap(Type, Index) ResourceDescriptorHeap[Index]
		// GetSamplerFromHeap(Type, Index)  SamplerDescriptorHeap[Index]

		const TCHAR* HeapString = BindlessType == EBindlessConversionType::Sampler ? TEXT("SamplerDescriptorHeap") : TEXT("ResourceDescriptorHeap");

		if (bIsRayTracingShader)
		{
			if (BindlessType == EBindlessConversionType::SRV)
			{
				// Patch the HitGroupSystemIndexBuffer/HitGroupSystemVertexBuffer indices to use the ones contained in the shader record
				if (IndexString == HitGroupSystemIndexBufferName)
				{
					IndexString = TEXTVIEW("D3DHitGroupSystemParameters.BindlessHitGroupSystemIndexBuffer");
				}
				else if (IndexString == HitGroupSystemVertexBufferName)
				{
					IndexString = TEXTVIEW("D3DHitGroupSystemParameters.BindlessHitGroupSystemVertexBuffer");
				}
			}

			// Raytracing shaders need NonUniformResourceIndex because bindless index can be divergent in hit/miss/callable shaders
			return FString::Printf(TEXT("%s[NonUniformResourceIndex(%.*s)]"),
				HeapString,
				IndexString.Len(), IndexString.GetData()
			);
		}
			
		return FString::Printf(TEXT("%s[%.*s]"),
			HeapString,
			IndexString.Len(), IndexString.GetData()
		);
	}

	virtual FString GetStableConstantBufferRegisterString(EShaderFrequency ShaderFrequency) const override
	{
		const uint32 Space = GetAutoBindingSpace(ShaderFrequency);
		if (Space != 0)
		{
			return FString::Printf(TEXT(" : register(b0, space%d)"), Space);
		}

		return TEXT(" : register(b0)");
	}
	
	const bool bIsRayTracingShader;
	const FString HitGroupSystemIndexBufferName;
	const FString HitGroupSystemVertexBufferName;
};

void CompileD3DShader(const FShaderCompilerInput& Input, const FShaderPreprocessOutput& InPreprocessOutput, FShaderCompilerOutput& Output, ED3DShaderModel ShaderModel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CompileD3DShader);

	const TCHAR* ShaderProfile = GetShaderProfileName(Input, ShaderModel);

	if (!ShaderProfile)
	{
		Output.AddErrorf(TEXT("Unrecognized shader frequency %s"), GetShaderFrequencyString(Input.Target.GetFrequency()));
		return;
	}

	FString EntryPointName = Input.EntryPointName;
	FString PreprocessedSource(InPreprocessOutput.GetSourceViewWide());

	FD3DShaderParameterParserPlatformConfiguration PlatformConfiguration(Input);
	FShaderParameterParser ShaderParameterParser(PlatformConfiguration);
	if (!ShaderParameterParser.ParseAndModify(Input, Output.Errors, PreprocessedSource))
	{
		// The FShaderParameterParser will add any relevant errors.
		return;
	}

	// Perform symbolic elimination
	bool bSDCEMinified = UE::ShaderMinifier::SDCE::MinifyInPlace(Input, InPreprocessOutput, PreprocessedSource);

	if (ShaderParameterParser.DidModifyShader() || bSDCEMinified)
	{
		Output.ModifiedShaderSource = PreprocessedSource;
	}

	if (Input.Environment.CompilerFlags.Contains(CFLAG_ForceRemoveUnusedInterpolators) && Input.Target.Frequency == SF_Vertex && Input.bCompilingForShaderPipeline)
	{
		// Always add SV_Position
		TArray<FStringView> UsedOutputs;
		for (const FString& UsedOutput : Input.UsedOutputs)
		{
			UsedOutputs.Emplace(UsedOutput);
		}
		UsedOutputs.Emplace(TEXTVIEW("SV_POSITION"));
		UsedOutputs.Emplace(TEXTVIEW("SV_ViewPortArrayIndex"));

		// We can't remove any of the output-only system semantics
		//@todo - there are a bunch of tessellation ones as well
		const FStringView Exceptions[] =
		{
			TEXTVIEW("SV_ClipDistance"),
			TEXTVIEW("SV_ClipDistance0"),
			TEXTVIEW("SV_ClipDistance1"),
			TEXTVIEW("SV_ClipDistance2"),
			TEXTVIEW("SV_ClipDistance3"),
			TEXTVIEW("SV_ClipDistance4"),
			TEXTVIEW("SV_ClipDistance5"),
			TEXTVIEW("SV_ClipDistance6"),
			TEXTVIEW("SV_ClipDistance7"),

			TEXTVIEW("SV_CullDistance"),
			TEXTVIEW("SV_CullDistance0"),
			TEXTVIEW("SV_CullDistance1"),
			TEXTVIEW("SV_CullDistance2"),
			TEXTVIEW("SV_CullDistance3"),
			TEXTVIEW("SV_CullDistance4"),
			TEXTVIEW("SV_CullDistance5"),
			TEXTVIEW("SV_CullDistance6"),
			TEXTVIEW("SV_CullDistance7"),
		};

		TArray<UE::HlslParser::FScopedDeclarations> ScopedDeclarations;
		const FStringView GlobalSymbols[] =
		{
			TEXTVIEW("RayDesc"),
		};
		ScopedDeclarations.Emplace(TConstArrayView<FStringView>(), GlobalSymbols);

		(void)UE::ShaderCompilerCommon::RemoveUnusedOutputs(Input, Output, PreprocessedSource, EntryPointName, UsedOutputs, Exceptions, ScopedDeclarations);
	}

	const bool bSuccess = DoesShaderModelRequireDXC(ShaderModel)
		? CompileAndProcessD3DShaderDXC(Input, PreprocessedSource, EntryPointName, ShaderParameterParser, ShaderProfile, ShaderModel, false, Output)
		: CompileAndProcessD3DShaderFXC(Input, PreprocessedSource, EntryPointName, ShaderParameterParser, ShaderProfile, ShaderModel, false, Output);

	if (!bSuccess && !Output.Errors.Num())
	{
		Output.AddErrorf(TEXT("Compile failed without errors!"));
	}

	ShaderParameterParser.ValidateShaderParameterTypes(Input, Output);

	if (EnumHasAnyFlags(Input.DebugInfoFlags, EShaderDebugInfoFlags::CompileFromDebugUSF))
	{
		for (const FShaderCompilerError& Error : Output.Errors)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%s\n"), *Error.GetErrorStringWithLineMarker());
		}
	}
}