// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHI.h"
#include "RHIShaderBindingLayout.h"
#include <functional>

struct FD3DShaderCompileData
{
	FD3DShaderCompileData()
		: UsedUniformBufferSlots(false, 32)
	{
	}

	TArray<FShaderCodeVendorExtension> VendorExtensions;
	TArray<FString> ShaderInputs;
	TArray<FString> UniformBufferNames;
	TBitArray<> UsedUniformBufferSlots;

	bool bBindlessEnabled = false;
	bool bGlobalUniformBufferUsed = false;
	bool bDiagnosticBufferUsed = false;

	uint32 NumInstructions = 0;
	uint32 NumSamplers = 0;
	uint32 NumSRVs = 0;
	uint32 NumCBs = 0;
	uint32 NumUAVs = 0;

	uint32 MaxSamplers = 0;
	uint32 MaxSRVs = 0;
	uint32 MaxCBs = 0;
	uint32 MaxUAVs = 0;
};

template <typename D3D1x_SHADER_INPUT_BIND_DESC>
EShaderCodeResourceBindingType D3DBindDescToShaderCodeResourceBinding(const D3D1x_SHADER_INPUT_BIND_DESC& Binding)
{
	switch (Binding.Type)
	{
	case D3D_SIT_SAMPLER:		
		return EShaderCodeResourceBindingType::SamplerState;
	case D3D_SIT_TBUFFER:
	case D3D_SIT_CBUFFER:
		return EShaderCodeResourceBindingType::Buffer;
	case D3D_SIT_TEXTURE:
		switch (Binding.Dimension)
		{
		case D3D_SRV_DIMENSION_BUFFER:			return EShaderCodeResourceBindingType::Buffer;
		case D3D_SRV_DIMENSION_TEXTURE2D:		return EShaderCodeResourceBindingType::Texture2D;
		case D3D_SRV_DIMENSION_TEXTURE2DARRAY:	return EShaderCodeResourceBindingType::Texture2DArray;
		case D3D_SRV_DIMENSION_TEXTURE2DMS:		return EShaderCodeResourceBindingType::Texture2DMS;
		case D3D_SRV_DIMENSION_TEXTURE3D:		return EShaderCodeResourceBindingType::Texture3D;
		case D3D_SRV_DIMENSION_TEXTURECUBE:		return EShaderCodeResourceBindingType::TextureCube;
		default:
			return EShaderCodeResourceBindingType::Invalid;
		}
	case D3D_SIT_UAV_RWTYPED:
		switch (Binding.Dimension)
		{
		case D3D_SRV_DIMENSION_BUFFER:			return EShaderCodeResourceBindingType::RWBuffer;
		case D3D_SRV_DIMENSION_TEXTURE2D:		return EShaderCodeResourceBindingType::RWTexture2D;
		case D3D_SRV_DIMENSION_TEXTURE2DARRAY:	return EShaderCodeResourceBindingType::RWTexture2DArray;
		case D3D_SRV_DIMENSION_TEXTURE3D:		return EShaderCodeResourceBindingType::RWTexture3D;
		case D3D_SRV_DIMENSION_TEXTURECUBE:		return EShaderCodeResourceBindingType::RWTextureCube;
		default:
			return EShaderCodeResourceBindingType::Invalid;
		}
	case D3D_SIT_STRUCTURED:		return EShaderCodeResourceBindingType::StructuredBuffer;
	case D3D_SIT_UAV_RWSTRUCTURED:	return EShaderCodeResourceBindingType::RWStructuredBuffer;
	case D3D_SIT_BYTEADDRESS:		return EShaderCodeResourceBindingType::ByteAddressBuffer;
	case D3D_SIT_UAV_RWBYTEADDRESS:	return EShaderCodeResourceBindingType::RWByteAddressBuffer;
	default:
		return EShaderCodeResourceBindingType::Invalid;
	}
}

template <typename ID3D1xShaderReflection, typename D3D1x_SHADER_DESC, typename D3D1x_SHADER_INPUT_BIND_DESC,
	typename ID3D1xShaderReflectionConstantBuffer, typename D3D1x_SHADER_BUFFER_DESC,
	typename ID3D1xShaderReflectionVariable, typename D3D1x_SHADER_VARIABLE_DESC>
	inline void ExtractParameterMapFromD3DShader(
		const FShaderCompilerInput& Input,
		const FShaderParameterParser& ShaderParameterParser,
		uint32 BindingSpace,
		ID3D1xShaderReflection* Reflector, const D3D1x_SHADER_DESC& ShaderDesc,
		FD3DShaderCompileData& CompileData,
		FShaderCompilerOutput& Output
	)
{
	// Add parameters for shader resources (constant buffers, textures, samplers, etc. */
	for (uint32 ResourceIndex = 0; ResourceIndex < ShaderDesc.BoundResources; ResourceIndex++)
	{
		D3D1x_SHADER_INPUT_BIND_DESC BindDesc{};
		Reflector->GetResourceBindingDesc(ResourceIndex, &BindDesc);

		if (!IsCompatibleBinding(BindDesc, BindingSpace))
		{
			continue;
		}

		const FString BindDescName(BindDesc.Name);

		if (BindDesc.Type == D3D_SIT_CBUFFER || BindDesc.Type == D3D_SIT_TBUFFER)
		{
			const uint32 CBIndex = BindDesc.BindPoint;
			ID3D1xShaderReflectionConstantBuffer* ConstantBuffer = Reflector->GetConstantBufferByName(BindDesc.Name);

			D3D1x_SHADER_BUFFER_DESC CBDesc{};
			ConstantBuffer->GetDesc(&CBDesc);

			const FString ConstantBufferName(CBDesc.Name);

			const bool bGlobalCB = (ConstantBufferName == TEXT("$Globals"));
			const bool bRootConstantsCB = (ConstantBufferName == TEXT("UERootConstants"));
			const bool bIsRootCB = (ConstantBufferName == FShaderParametersMetadata::kRootUniformBufferBindingName);

			if (bGlobalCB)
			{
				if (Input.ShouldUseStableConstantBuffer())
				{
					// Each member found in the global constant buffer means it was not in RootParametersStructure or
					// it would have been moved by ShaderParameterParser.ParseAndModify().
					for (uint32 ConstantIndex = 0; ConstantIndex < CBDesc.Variables; ConstantIndex++)
					{
						ID3D1xShaderReflectionVariable* Variable = ConstantBuffer->GetVariableByIndex(ConstantIndex);

						D3D1x_SHADER_VARIABLE_DESC VariableDesc{};
						Variable->GetDesc(&VariableDesc);

						if (VariableDesc.uFlags & D3D_SVF_USED)
						{
							const FString VariableName(VariableDesc.Name);

							AddUnboundShaderParameterError(
								Input,
								ShaderParameterParser,
								VariableName,
								Output);
						}
					}
				}
				else
				{
					// Track all of the variables in this constant buffer.
					for (uint32 ConstantIndex = 0; ConstantIndex < CBDesc.Variables; ConstantIndex++)
					{
						ID3D1xShaderReflectionVariable* Variable = ConstantBuffer->GetVariableByIndex(ConstantIndex);

						D3D1x_SHADER_VARIABLE_DESC VariableDesc{};
						Variable->GetDesc(&VariableDesc);

						if (VariableDesc.uFlags & D3D_SVF_USED)
						{
							const FString VariableName(VariableDesc.Name);

							HandleReflectedGlobalConstantBufferMember(
								Input,
								ShaderParameterParser,
								VariableName,
								CBIndex,
								VariableDesc.StartOffset,
								VariableDesc.Size,
								Output
							);

							CompileData.bGlobalUniformBufferUsed = true;
							CompileData.UsedUniformBufferSlots[CBIndex] = true;
						}
					}
				}
			}
			else if (bRootConstantsCB)
			{
				// For the UERootConstants root constant CB, we want to fully skip adding it to the parameter map, or
				// updating the used slots or num CBs (all those assume space0).
			}
			else if (bIsRootCB && Input.ShouldUseStableConstantBuffer())
			{
				if (CBIndex == FShaderParametersMetadata::kRootCBufferBindingIndex)
				{
					int32 ConstantBufferSize = 0;

					// Track all of the variables in this constant buffer.
					for (uint32 ConstantIndex = 0; ConstantIndex < CBDesc.Variables; ConstantIndex++)
					{
						ID3D1xShaderReflectionVariable* Variable = ConstantBuffer->GetVariableByIndex(ConstantIndex);

						D3D1x_SHADER_VARIABLE_DESC VariableDesc{};
						Variable->GetDesc(&VariableDesc);

						if (VariableDesc.uFlags & D3D_SVF_USED)
						{
							const FString VariableName(VariableDesc.Name);

							HandleReflectedRootConstantBufferMember(
								Input,
								ShaderParameterParser,
								VariableName,
								VariableDesc.StartOffset,
								VariableDesc.Size,
								Output
							);

							ConstantBufferSize = FMath::Max<int32>(ConstantBufferSize, VariableDesc.StartOffset + VariableDesc.Size);
						}
					}

					if (ConstantBufferSize > 0)
					{
						HandleReflectedRootConstantBuffer(ConstantBufferSize, Output);

						CompileData.bGlobalUniformBufferUsed = true;
						CompileData.UsedUniformBufferSlots[CBIndex] = true;
					}
				}
				else
				{
					Output.AddErrorf(
						TEXT("Error: %s is expected to always be in the API slot %d, but is actually in slot %d."),
						FShaderParametersMetadata::kRootUniformBufferBindingName,
						FShaderParametersMetadata::kRootCBufferBindingIndex,
						CBIndex
					);
					Output.bSucceeded = false;
				}
			}
			else
			{
				// Track just the constant buffer itself.
				AddShaderValidationUBSize(CBIndex, CBDesc.Size, Output);
				HandleReflectedUniformBuffer(Input, ConstantBufferName, CBIndex, Output);
				
				CompileData.UsedUniformBufferSlots[CBIndex] = true;

				const EUniformBufferMemberReflectionReason Reason = ShouldReflectUniformBufferMembers(Input, ConstantBufferName);
				if (Reason != EUniformBufferMemberReflectionReason::None)
				{
					for (uint32 ConstantIndex = 0; ConstantIndex < CBDesc.Variables; ConstantIndex++)
					{
						ID3D1xShaderReflectionVariable* Variable = ConstantBuffer->GetVariableByIndex(ConstantIndex);

						D3D1x_SHADER_VARIABLE_DESC VariableDesc{};
						Variable->GetDesc(&VariableDesc);

						if (VariableDesc.uFlags & D3D_SVF_USED)
						{
							const FString VariableName(VariableDesc.Name);

							HandleReflectedUniformBufferConstantBufferMember(
								Input,
								Reason,
								ConstantBufferName,
								CBIndex,
								VariableName,
								VariableDesc.StartOffset,
								VariableDesc.Size,
								Output
							);
						}
					}
				}
			}

			if (CompileData.UniformBufferNames.Num() <= (int32)CBIndex)
			{
				CompileData.UniformBufferNames.AddDefaulted(CBIndex - CompileData.UniformBufferNames.Num() + 1);
			}
			CompileData.UniformBufferNames[CBIndex] = UE::ShaderCompilerCommon::RemoveConstantBufferPrefix(ConstantBufferName);

			CompileData.NumCBs = FMath::Max(CompileData.NumCBs, BindDesc.BindPoint + BindDesc.BindCount);
		}
		else if (BindDesc.Type == D3D_SIT_TEXTURE || BindDesc.Type == D3D_SIT_SAMPLER)
		{
			check(BindDesc.BindCount == 1);

			// https://github.com/GPUOpen-LibrariesAndSDKs/AGS_SDK/blob/master/ags_lib/hlsl/ags_shader_intrinsics_dx11.hlsl
			const bool bIsAMDTexExtension = (FCStringAnsi::Strcmp(BindDesc.Name, "AmdDxExtShaderIntrinsicsResource") == 0);
			const bool bIsAMDSmpExtension = (FCStringAnsi::Strcmp(BindDesc.Name, "AmdDxExtShaderIntrinsicsSamplerState") == 0);
			const bool bIsVendorParameter = bIsAMDTexExtension || bIsAMDSmpExtension;

			const uint32 BindCount = 1;
			const EShaderParameterType ParameterType = (BindDesc.Type == D3D_SIT_SAMPLER) ? EShaderParameterType::Sampler : EShaderParameterType::SRV;

			if (bIsVendorParameter)
			{
				CompileData.VendorExtensions.Emplace(EGpuVendorId::Amd, 0, BindDesc.BindPoint, BindCount, ParameterType);
			}
			else if (ParameterType == EShaderParameterType::Sampler)
			{
				HandleReflectedShaderSampler(BindDescName, BindDesc.BindPoint, Output);
				CompileData.NumSamplers = FMath::Max(CompileData.NumSamplers, BindDesc.BindPoint + BindCount);
			}
			else
			{
				EShaderCodeResourceBindingType ResourceBindingType = D3DBindDescToShaderCodeResourceBinding(BindDesc);
				AddShaderValidationSRVType(BindDesc.BindPoint, ResourceBindingType, Output);

				HandleReflectedShaderResource(BindDescName, BindDesc.BindPoint, Output);
				CompileData.NumSRVs = FMath::Max(CompileData.NumSRVs, BindDesc.BindPoint + BindCount);
			}
		}
		else if (BindDesc.Type == D3D_SIT_UAV_RWTYPED || BindDesc.Type == D3D_SIT_UAV_RWSTRUCTURED ||
			BindDesc.Type == D3D_SIT_UAV_RWBYTEADDRESS || BindDesc.Type == D3D_SIT_UAV_RWSTRUCTURED_WITH_COUNTER ||
			BindDesc.Type == D3D_SIT_UAV_APPEND_STRUCTURED)
		{
			check(BindDesc.BindCount == 1);

			// https://developer.nvidia.com/unlocking-gpu-intrinsics-hlsl
			const bool bIsNVExtension = (FCStringAnsi::Strcmp(BindDesc.Name, "g_NvidiaExt") == 0);

			// https://github.com/intel/intel-graphics-compiler/blob/master/inc/IntelExtensions.hlsl
			const bool bIsIntelExtension = (FCStringAnsi::Strcmp(BindDesc.Name, "g_IntelExt") == 0);

			// https://github.com/GPUOpen-LibrariesAndSDKs/AGS_SDK/blob/master/ags_lib/hlsl/ags_shader_intrinsics_dx11.hlsl
			const bool bIsAMDExtensionDX11 = (FCStringAnsi::Strcmp(BindDesc.Name, "AmdDxExtShaderIntrinsicsUAV") == 0);

			// https://github.com/GPUOpen-LibrariesAndSDKs/AGS_SDK/blob/master/ags_lib/hlsl/ags_shader_intrinsics_dx12.hlsl
			const bool bIsAMDExtensionDX12 = (FCStringAnsi::Strcmp(BindDesc.Name, "AmdExtD3DShaderIntrinsicsUAV") == 0);

			const bool bIsVendorParameter = bIsNVExtension || bIsIntelExtension || bIsAMDExtensionDX11 || bIsAMDExtensionDX12;

			// See D3DCommon.ush
			const bool bIsDiagnosticBufferParameter = (FCStringAnsi::Strcmp(BindDesc.Name, "UEDiagnosticBuffer") == 0);

			const uint32 BindCount = 1;
			if (bIsVendorParameter)
			{
				const EGpuVendorId VendorId =
					bIsNVExtension ? EGpuVendorId::Nvidia :
					(bIsAMDExtensionDX11 || bIsAMDExtensionDX12) ? EGpuVendorId::Amd :
					bIsIntelExtension ? EGpuVendorId::Intel :
					EGpuVendorId::Unknown;

				// g_NvidiaExt can be included in shaders compiled for different vendors due to usage of dynamic branches (e.g. UsingCLAS) to avoid extra shader permutations.
				// Mark it optional so the shader can be created on non-NVIDIA GPUs.
				// TODO: This could be removed once CLAS is part of the DXR spec.
				const bool bIsOptional = bIsNVExtension;

				CompileData.VendorExtensions.Emplace(VendorId, 0, BindDesc.BindPoint, BindCount, EShaderParameterType::UAV, bIsOptional);
			}
			else if (bIsDiagnosticBufferParameter)
			{
				CompileData.bDiagnosticBufferUsed = true;
			}
			else
			{
				EShaderCodeResourceBindingType ResourceBindingType = D3DBindDescToShaderCodeResourceBinding(BindDesc);
				AddShaderValidationUAVType(BindDesc.BindPoint, ResourceBindingType, Output);

				HandleReflectedShaderUAV(BindDescName, BindDesc.BindPoint, Output);
				CompileData.NumUAVs = FMath::Max(CompileData.NumUAVs, BindDesc.BindPoint + BindCount);
			}
		}
		else if (BindDesc.Type == D3D_SIT_STRUCTURED || BindDesc.Type == D3D_SIT_BYTEADDRESS)
		{
			check(BindDesc.BindCount == 1);

			EShaderCodeResourceBindingType ResourceBindingType = D3DBindDescToShaderCodeResourceBinding(BindDesc);
			AddShaderValidationSRVType(BindDesc.BindPoint, ResourceBindingType, Output);

			HandleReflectedShaderResource(BindDescName, BindDesc.BindPoint, Output);

			// https://learn.microsoft.com/en-us/windows/win32/api/d3d12shader/ns-d3d12shader-d3d12_shader_input_bind_desc
			// If the shader resource is a structured buffer, the field contains the stride of the type in bytes
			if (BindDesc.Type == D3D_SIT_STRUCTURED)
			{
				UpdateStructuredBufferStride(Input, BindDescName, BindDesc.BindPoint, BindDesc.NumSamples, Output);
			}

			CompileData.NumSRVs = FMath::Max(CompileData.NumSRVs, BindDesc.BindPoint + 1);
		}
		else if (BindDesc.Type == D3D_SIT_RTACCELERATIONSTRUCTURE)
		{
			// Acceleration structure resources are treated as SRVs.
			check(BindDesc.BindCount == 1);

			EShaderCodeResourceBindingType ResourceBindingType = D3DBindDescToShaderCodeResourceBinding(BindDesc);
			AddShaderValidationSRVType(BindDesc.BindPoint, ResourceBindingType, Output);

			HandleReflectedShaderResource(BindDescName, BindDesc.BindPoint, Output);
			CompileData.NumSRVs = FMath::Max(CompileData.NumSRVs, BindDesc.BindPoint + 1);
		}
	}

	CompileData.NumInstructions = ShaderDesc.InstructionCount;
}

// Validate that we are not going over to maximum amount of resource bindings support by the default root signature on DX12
// Currently limited for hard-coded root signature setup (see: FD3D12Adapter::StaticGraphicsRootSignature)
// In theory this limitation is only required for DX12, but we don't want a shader to compile on DX11 while not working on DX12.
// (DX11 has an API limit on 128 SRVs, 16 Samplers, 8 UAVs and 14 CBs but if you go over these values then the shader won't compile)
inline bool ValidateResourceCounts(const FD3DShaderCompileData& CompileData, TArray<FShaderCompilerError>& OutErrors)
{
	const bool bTooManySRVs     = !CompileData.bBindlessEnabled && CompileData.NumSRVs     > CompileData.MaxSRVs;
	const bool bTooManyUAVs     = !CompileData.bBindlessEnabled && CompileData.NumUAVs     > CompileData.MaxUAVs;
	const bool bTooManySamplers = !CompileData.bBindlessEnabled && CompileData.NumSamplers > CompileData.MaxSamplers;
	const bool bTooManyCBs = CompileData.NumCBs > CompileData.MaxCBs;

	if (bTooManySRVs || bTooManySamplers || bTooManyUAVs || bTooManyCBs)
	{
		if (bTooManySRVs)
		{
			OutErrors.Add(FString::Printf(TEXT("Shader is using too many SRVs: %d (only %d supported)"), CompileData.NumSRVs, CompileData.MaxSRVs));
		}

		if (bTooManySamplers)
		{
			OutErrors.Add(FString::Printf(TEXT("Shader is using too many Samplers: %d (only %d supported)"), CompileData.NumSamplers, CompileData.MaxSamplers));
		}

		if (bTooManyUAVs)
		{
			OutErrors.Add(FString::Printf(TEXT("Shader is using too many UAVs: %d (only %d supported)"), CompileData.NumUAVs, CompileData.MaxUAVs));
		}

		if (bTooManyCBs)
		{
			OutErrors.Add(FString::Printf(TEXT("Shader is using too many Constant Buffers: %d (only %d supported)"), CompileData.NumCBs, CompileData.MaxCBs));
		}

		return false;
	}

	return true;
}

inline FShaderCodePackedResourceCounts InitPackedResourceCounts(const FD3DShaderCompileData& CompileData)
{
	FShaderCodePackedResourceCounts PackedResourceCounts{};

	if (CompileData.bGlobalUniformBufferUsed)
	{
		EnumAddFlags(PackedResourceCounts.UsageFlags, EShaderResourceUsageFlags::GlobalUniformBuffer);
	}

	if (CompileData.bBindlessEnabled)
	{
		EnumAddFlags(PackedResourceCounts.UsageFlags, EShaderResourceUsageFlags::BindlessResources);
		EnumAddFlags(PackedResourceCounts.UsageFlags, EShaderResourceUsageFlags::BindlessSamplers);
	}

	if (CompileData.bDiagnosticBufferUsed)
	{
		EnumAddFlags(PackedResourceCounts.UsageFlags, EShaderResourceUsageFlags::DiagnosticBuffer);
	}

	PackedResourceCounts.NumSamplers = static_cast<uint8>(CompileData.NumSamplers);
	PackedResourceCounts.NumSRVs = static_cast<uint8>(CompileData.NumSRVs);
	PackedResourceCounts.NumCBs = static_cast<uint8>(CompileData.NumCBs);
	PackedResourceCounts.NumUAVs = static_cast<uint8>(CompileData.NumUAVs);

	return PackedResourceCounts;
}

template <typename TBlob>
inline void GenerateFinalOutput(
	TRefCountPtr<TBlob>& CompressedData,
	const FShaderCompilerInput& Input,
	ED3DShaderModel ShaderModel,
	bool bProcessingSecondTime,
	FD3DShaderCompileData& CompileData,
	const FShaderCodePackedResourceCounts& PackedResourceCounts,
	FShaderCompilerOutput& Output,
	TFunction<void(FMemoryWriter&)> PostSRTWriterCallback,
	TFunction<void(FShaderCode&)> AddOptionalDataCallback)
{
	const uint32 NumBindlessResources = CompileData.bBindlessEnabled ? Output.ParameterMap.CountParametersOfType(EShaderParameterType::BindlessSRV) : 0;
	const uint32 NumBindlessSamplers = CompileData.bBindlessEnabled ? Output.ParameterMap.CountParametersOfType(EShaderParameterType::BindlessSampler) : 0;

	// Build the SRT for this shader.
	FShaderResourceTable SRT;

	TArray<uint8> UniformBufferNameBytes;

	{	
		// Build the generic SRT for this shader.
		FShaderCompilerResourceTable GenericSRT;
		BuildResourceTableMapping(Input.Environment.ResourceTableMap, Input.Environment.UniformBufferMap, CompileData.UsedUniformBufferSlots, Output.ParameterMap, GenericSRT);

		// Ray generation shaders rely on a different binding model that aren't compatible with global uniform buffers.
		if (Input.Target.Frequency != SF_RayGen)
		{
			CullGlobalUniformBuffers(Input.Environment.UniformBufferMap, Output.ParameterMap);
		}

		if (CompileData.UniformBufferNames.Num() < GenericSRT.ResourceTableLayoutHashes.Num())
		{
			CompileData.UniformBufferNames.AddDefaulted(GenericSRT.ResourceTableLayoutHashes.Num() - CompileData.UniformBufferNames.Num());
		}

		for (int32 Index = 0; Index < GenericSRT.ResourceTableLayoutHashes.Num(); ++Index)
		{
			if (GenericSRT.ResourceTableLayoutHashes[Index] != 0 && CompileData.UniformBufferNames[Index].Len() == 0)
			{
				for (const auto& KeyValue : Input.Environment.UniformBufferMap)
				{
					const FUniformBufferEntry& UniformBufferEntry = KeyValue.Value;

					if (UniformBufferEntry.LayoutHash == GenericSRT.ResourceTableLayoutHashes[Index])
					{
						CompileData.UniformBufferNames[Index] = KeyValue.Key;
						break;
					}
				}
			}
		}

		FMemoryWriter UniformBufferNameWriter(UniformBufferNameBytes);
		UniformBufferNameWriter << CompileData.UniformBufferNames;

		UE::ShaderCompilerCommon::BuildShaderResourceTable(GenericSRT, SRT);
	}

	if (Input.Environment.CompilerFlags.Contains(CFLAG_ForceRemoveUnusedInterpolators) && Input.Target.Frequency == SF_Pixel && Input.bCompilingForShaderPipeline && bProcessingSecondTime)
	{
		Output.bSupportsQueryingUsedAttributes = true;
		Output.UsedAttributes = CompileData.ShaderInputs;
	}

	// Generate the final Output
	FMemoryWriter Ar(Output.ShaderCode.GetWriteAccess(), true);
	Ar << SRT;

	PostSRTWriterCallback(Ar);

	Ar.Serialize(CompressedData->GetBufferPointer(), CompressedData->GetBufferSize());

	// Append data that is generate from the shader code and assist the usage, mostly needed for DX12 
	{
		Output.ShaderCode.AddOptionalData(PackedResourceCounts);
		Output.ShaderCode.AddOptionalData(FShaderCodeUniformBuffers::Key, UniformBufferNameBytes.GetData(), UniformBufferNameBytes.Num());
		AddOptionalDataCallback(Output.ShaderCode);
	}
	
	// Append the shader binding layout hash used for validation
	{
		uint32 ShaderBindingLayoutHash = Input.Environment.RHIShaderBindingLayout.GetHash();

		TArray<uint8> WriterBytes;
		FMemoryWriter Writer(WriterBytes);
		Writer << ShaderBindingLayoutHash;
		if (WriterBytes.Num() > 0)
		{
			Output.ShaderCode.AddOptionalData(FShaderCodeShaderResourceTableDataDesc::Key, WriterBytes.GetData(), WriterBytes.Num());
		}
	}

	// Append information about optional hardware vendor extensions
	if (CompileData.VendorExtensions.Num() > 0)
	{
		TArray<uint8> WriterBytes;
		FMemoryWriter Writer(WriterBytes);
		Writer << CompileData.VendorExtensions;
		if (WriterBytes.Num() > 0)
		{
			Output.ShaderCode.AddOptionalData(FShaderCodeVendorExtension::Key, WriterBytes.GetData(), WriterBytes.Num());
		}
	}

	if (Input.Environment.CompilerFlags.Contains(CFLAG_ExtraShaderData))
	{
		Output.ShaderCode.AddOptionalData(FShaderCodeName::Key, TCHAR_TO_UTF8(*Input.GenerateShaderName()));
	}

	Output.SerializeShaderCodeValidation();
	Output.SerializeShaderDiagnosticData();

	// Set the number of instructions.
	Output.NumInstructions = CompileData.NumInstructions;

	Output.NumTextureSamplers = PackedResourceCounts.NumSamplers;

	// Pass the target through to the output.
	Output.Target = Input.Target;

	// SRV Limits
	{
		if (CompileData.bBindlessEnabled)
		{
			Output.AddStatistic<uint32>(TEXT("Bindless Resources"), NumBindlessResources);
			Output.AddStatistic<uint32>(TEXT("Bindless Samplers"), NumBindlessSamplers);
		}
		else
		{
			Output.AddStatistic<uint32>(TEXT("Resources Used"), CompileData.NumSRVs);
			Output.AddStatistic<uint32>(TEXT("Resource Limit"), CompileData.MaxSRVs);

			Output.AddStatistic<uint32>(TEXT("RW Resources Used"), CompileData.NumUAVs);
			Output.AddStatistic<uint32>(TEXT("RW Resource Limit"), CompileData.MaxUAVs);

			Output.AddStatistic<uint32>(TEXT("Samplers Used"), CompileData.NumSamplers);
			Output.AddStatistic<uint32>(TEXT("Sampler Limit"), CompileData.MaxSamplers);
		}
	}
}

enum class ShaderCompilerType
{
	FXC,
	DXC,
};

using ShaderCompileLambdaType = std::function<bool(
	const FShaderCompilerInput& Input,
	const FString& PreprocessedShaderSource,
	const FString& EntryPointName,
	const FShaderParameterParser& ShaderParameterParser,
	const TCHAR* ShaderProfile,
	const ED3DShaderModel ShaderModel,
	const bool bProcessingSecondTime,
	FShaderCompilerOutput& Output)>;

template<ShaderCompilerType LambdaPlatform, typename TShaderCompileLambdaType,
	typename ID3D1xShaderReflection, typename D3D1x_SHADER_DESC, typename D3D1x_SIGNATURE_PARAMETER_DESC>
bool RemoveUnusedInterpolators(
	const FShaderCompilerInput& Input,
	const FString& PreprocessedShaderSource,
	const FString& EntryPointName,
	const FShaderParameterParser& ShaderParameterParser,
	const TCHAR* ShaderProfile,
	const ED3DShaderModel ShaderModel,
	const bool bProcessingSecondTime,
	FD3DShaderCompileData& CompileData,
	TRefCountPtr<ID3D1xShaderReflection> Reflector,
	TShaderCompileLambdaType ShaderCompileLambda,
	FShaderCompilerOutput& Output,
	bool& CompileResult)
{
	if (Input.Target.Frequency == SF_Pixel)
	{
		if (Reflector)
		{
			// Read the constant table description.
			D3D1x_SHADER_DESC ShaderDesc;
			Reflector->GetDesc(&ShaderDesc);

			bool bFoundUnused = false;
			for (uint32 Index = 0; Index < ShaderDesc.InputParameters; ++Index)
			{
				// VC++ horrible hack: Runtime ESP checks get confused and fail for some reason calling Reflector->GetInputParameterDesc() (because it comes from another DLL?)
				// so "guard it" using the middle of an array; it's been confirmed NO corruption is really happening.
				D3D1x_SIGNATURE_PARAMETER_DESC ParamDescs[3];
				D3D1x_SIGNATURE_PARAMETER_DESC& ParamDesc = ParamDescs[1];
				Reflector->GetInputParameterDesc(Index, &ParamDesc);
				if (ParamDesc.SystemValueType == D3D_NAME_UNDEFINED)
				{
					if (ParamDesc.ReadWriteMask != 0)
					{
						FString SemanticName(ParamDesc.SemanticName);

						CompileData.ShaderInputs.AddUnique(SemanticName);

						// Add the number (for the case of TEXCOORD)
						FString SemanticIndexName = FString::Printf(TEXT("%s%d"), *SemanticName, ParamDesc.SemanticIndex);
						CompileData.ShaderInputs.AddUnique(SemanticIndexName);

						// Add _centroid
						CompileData.ShaderInputs.AddUnique(SemanticName + TEXT("_centroid"));
						CompileData.ShaderInputs.AddUnique(SemanticIndexName + TEXT("_centroid"));
					}
					else
					{
						bFoundUnused = true;
					}
				}
				else
				{
					// if (ParamDesc.ReadWriteMask != 0)
					{
						// Keep system values
						CompileData.ShaderInputs.AddUnique(FString(ParamDesc.SemanticName));
					}
				}
			}

			if constexpr (LambdaPlatform == ShaderCompilerType::DXC)
			{
				//Seems like there is bug in DXC compiler which prevents SV_Coverage semantic 
				//from being detected as Input Parameter in the shader reflection data. 
				//Other system interpolators are fine... 
				//Can compile GizmoMaterial to repro the issue for example: -run=CookShadersCommandlet -targetPlatform=Windows -material=GizmoMaterial
				//Always adding SV_Coverage as Used interpolator to mitigate this issue.
				CompileData.ShaderInputs.AddUnique(TEXT("SV_Coverage"));
			}

			if (Input.Environment.CompilerFlags.Contains(CFLAG_ForceRemoveUnusedInterpolators) && Input.bCompilingForShaderPipeline && bFoundUnused && !bProcessingSecondTime)
			{
				// Rewrite the source removing the unused inputs so the bindings will match.
				// We may need to do this more than once if unused inputs change after the removal. Ie. for complex shaders, what can happen is:
				// pass1 detects that input A is not used, but input B and C are. Input A is removed, and we recompile (pass2). After the recompilation, we see that Input B is now also unused in pass2
				// (it became simpler and the compiler could see through that).
				// Since unused inputs are passed to the next stage, that will cause us to generate a vertex shader that does not output B, but our pixel shader will still be expecting B on input,
				// as it was rewritten based on the pass1 results.

				const FShaderCompilerOutput OriginalOutput = Output;
				const int kMaxReasonableAttempts = 64;
				for (int32 Attempt = 0; Attempt < kMaxReasonableAttempts; ++Attempt)
				{
					if (UE::ShaderCompilerCommon::RemoveUnusedInputs(Input, Output, PreprocessedShaderSource, Input.EntryPointName, CompileData.ShaderInputs))
					{
						CompileResult = ShaderCompileLambda(Input, Output.ModifiedShaderSource, Output.ModifiedEntryPointName, ShaderParameterParser, ShaderProfile, ShaderModel, true, Output);
						if (!CompileResult)
						{
							// if we failed to compile the shader, propagate the error up
							return true;
						}

						// check if the ShaderInputs changed - if not, we're done here
						if (Output.UsedAttributes.Num() == CompileData.ShaderInputs.Num())
						{
							return true;
						}

						// second pass cannot use more attributes than previously
						if (Output.UsedAttributes.Num() > CompileData.ShaderInputs.Num())
						{
							UE_LOGF(LogD3DShaderCompiler, Warning, "Second pass had more used attributes (%d) than first pass (%d)", Output.UsedAttributes.Num(), CompileData.ShaderInputs.Num());
							FShaderCompilerError NewError(FString::Printf(TEXT("Second pass had more used attributes (%d) than first pass (%d)"), Output.UsedAttributes.Num(), CompileData.ShaderInputs.Num()));
							Output = OriginalOutput;
							Output.Errors.Emplace(MoveTemp(NewError));
							break;
						}

						// if we're about to run out of attempts, report
						if (Attempt >= kMaxReasonableAttempts - 1)
						{
							UE_LOGF(LogD3DShaderCompiler, Warning, "Unable to determine unused inputs after %d attempts (last number of used attributes: %d, previous step:%d)!",
								Attempt + 1,
								Output.UsedAttributes.Num(),
								CompileData.ShaderInputs.Num());
							FShaderCompilerError NewError(FString::Printf(TEXT("Unable to determine unused inputs after %d attempts (last number of used attributes: %d, previous step:%d)!"),
								Attempt + 1,
								Output.UsedAttributes.Num(),
								CompileData.ShaderInputs.Num()));
							Output = OriginalOutput;
							Output.Errors.Emplace(MoveTemp(NewError));
							break;
						}

						CompileData.ShaderInputs = Output.UsedAttributes;
						Output = OriginalOutput;
						// go around to remove newly identified unused inputs
					}
				}
			}
		}
	}

	return false;
}
