// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderFormatD3D.h"
#include "ShaderPreprocessor.h"
#include "ShaderCompilerCommon.h"
#include "ShaderCompilerCommonInternal.h"
#include "ShaderCompileWorkerUtil.h"
#include "ShaderParameterParser.h"
#include "D3D12RHI.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Misc/Fnv.h"
#include "HAL/FileManager.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderPreprocessTypes.h"
#include "RayTracingDefinitions.h"

// D3D doesn't define a mask for this, so we do so here
#define SHADER_OPTIMIZATION_LEVEL_MASK (D3DCOMPILE_OPTIMIZATION_LEVEL0 | D3DCOMPILE_OPTIMIZATION_LEVEL1 | D3DCOMPILE_OPTIMIZATION_LEVEL2 | D3DCOMPILE_OPTIMIZATION_LEVEL3)

// Disable macro redefinition warning for compatibility with Windows SDK 8+
#pragma warning(push)
#pragma warning(disable : 4005)	// macro redefinition

#include "Windows/AllowWindowsPlatformTypes.h"
	#include <D3D12.h>
	#include <D3Dcompiler.h>
	#include <d3d11Shader.h>
	#include "amd_ags.h"
#include "Windows/HideWindowsPlatformTypes.h"
#undef DrawText

#pragma warning(pop)

#include <dxc/dxcapi.h>
#include <dxc/Support/dxcapi.use.h>
#include <dxc/Support/ErrorCodes.h>
#include <dxc/DXIL/DxilConstants.h>
#include <d3d12shader.h>

THIRD_PARTY_INCLUDES_START
	#include <string>
	#include "ShaderConductor/ShaderConductor.hpp"
THIRD_PARTY_INCLUDES_END

#include "DXCUtils.inl"
#include "HlslParserInternal.h"
#include "D3DShaderCompiler.inl"

static bool IsGlobalConstantBufferSupported(const FShaderTarget& Target)
{
	switch (Target.Frequency)
	{
	case SF_RayGen:
	case SF_RayMiss:
	case SF_RayCallable:
		// Global CB is not currently implemented for RayGen, Miss and Callable ray tracing shaders.
		return false;
	default:
		return true;
	}
}

// DXC specific error codes cannot be translated by FPlatformMisc::GetSystemErrorMessage, so do it manually.
// Codes defines in <DXC>/include/dxc/Support/ErrorCodes.h
static const TCHAR* DxcErrorCodeToString(HRESULT Code)
{
#define SWITCHCASE_TO_STRING(VALUE) case VALUE: return TEXT(#VALUE)
	switch (Code)
	{
		SWITCHCASE_TO_STRING( DXC_E_OVERLAPPING_SEMANTICS );
		SWITCHCASE_TO_STRING( DXC_E_MULTIPLE_DEPTH_SEMANTICS );
		SWITCHCASE_TO_STRING( DXC_E_INPUT_FILE_TOO_LARGE );
		SWITCHCASE_TO_STRING( DXC_E_INCORRECT_DXBC );
		SWITCHCASE_TO_STRING( DXC_E_ERROR_PARSING_DXBC_BYTECODE );
		SWITCHCASE_TO_STRING( DXC_E_DATA_TOO_LARGE );
		SWITCHCASE_TO_STRING( DXC_E_INCOMPATIBLE_CONVERTER_OPTIONS);
		SWITCHCASE_TO_STRING( DXC_E_IRREDUCIBLE_CFG );
		SWITCHCASE_TO_STRING( DXC_E_IR_VERIFICATION_FAILED );
		SWITCHCASE_TO_STRING( DXC_E_SCOPE_NESTED_FAILED );
		SWITCHCASE_TO_STRING( DXC_E_NOT_SUPPORTED );
		SWITCHCASE_TO_STRING( DXC_E_STRING_ENCODING_FAILED );
		SWITCHCASE_TO_STRING( DXC_E_CONTAINER_INVALID );
		SWITCHCASE_TO_STRING( DXC_E_CONTAINER_MISSING_DXIL );
		SWITCHCASE_TO_STRING( DXC_E_INCORRECT_DXIL_METADATA );
		SWITCHCASE_TO_STRING( DXC_E_INCORRECT_DDI_SIGNATURE );
		SWITCHCASE_TO_STRING( DXC_E_DUPLICATE_PART );
		SWITCHCASE_TO_STRING( DXC_E_MISSING_PART );
		SWITCHCASE_TO_STRING( DXC_E_MALFORMED_CONTAINER );
		SWITCHCASE_TO_STRING( DXC_E_INCORRECT_ROOT_SIGNATURE );
		SWITCHCASE_TO_STRING( DXC_E_CONTAINER_MISSING_DEBUG );
		SWITCHCASE_TO_STRING( DXC_E_MACRO_EXPANSION_FAILURE );
		SWITCHCASE_TO_STRING( DXC_E_OPTIMIZATION_FAILED );
		SWITCHCASE_TO_STRING( DXC_E_GENERAL_INTERNAL_ERROR );
		SWITCHCASE_TO_STRING( DXC_E_ABORT_COMPILATION_ERROR );
		SWITCHCASE_TO_STRING( DXC_E_EXTENSION_ERROR );
		SWITCHCASE_TO_STRING( DXC_E_LLVM_FATAL_ERROR );
		SWITCHCASE_TO_STRING( DXC_E_LLVM_UNREACHABLE );
		SWITCHCASE_TO_STRING( DXC_E_LLVM_CAST_ERROR );
	}
	return nullptr;
#undef SWITCHCASE_TO_STRING
}

static void LogFailedHRESULT(const TCHAR* FailedExpressionStr, HRESULT Result)
{
	if (Result == E_OUTOFMEMORY)
	{
		const FString ErrorReport = FString::Printf(TEXT("%s failed: Result=0x%08x (E_OUTOFMEMORY)"), FailedExpressionStr, (uint32)Result);
		FSCWErrorCode::Report(FSCWErrorCode::OutOfMemory, ErrorReport);
		UE_LOGF(LogD3DShaderCompiler, Fatal, "%ls", *ErrorReport);
	}
	else if (const TCHAR* ErrorCodeStr = DxcErrorCodeToString(Result))
	{
		UE_LOGF(LogD3DShaderCompiler, Fatal, "%ls failed: Result=0x%08x (%ls)", FailedExpressionStr, Result, ErrorCodeStr);
	}
	else
	{
		// Turn HRESULT into human readable string for error report
		TCHAR ResultStr[4096] = {};
		FPlatformMisc::GetSystemErrorMessage(ResultStr, UE_ARRAY_COUNT(ResultStr), Result);
		UE_LOGF(LogD3DShaderCompiler, Fatal, "%ls failed: Result=0x%08x (%ls)", FailedExpressionStr, Result, ResultStr);
	}
}

#define VERIFYHRESULT(expr)									\
	{														\
		const HRESULT HR##__LINE__ = expr;					\
		if (FAILED(HR##__LINE__))							\
		{													\
			LogFailedHRESULT(TEXT(#expr), HR##__LINE__);	\
		}													\
	}


class FDxcArguments
{
protected:
	FString ShaderProfile;
	FString EntryPoint;
	FString Exports;
	FString DumpDisasmFilename;
	FString BatchBaseFilename;
	FString DumpDebugInfoPath;
	bool bStripReflection = true;
	bool bDump = false;

	TArray<FString> ExtraArguments;

public:
	FDxcArguments(
		const FShaderCompilerInput& Input,
		const FString& InEntryPoint,
		const TCHAR* InShaderProfile,
		ED3DShaderModel ShaderModel,
		const FString& InExports
	)
		: ShaderProfile(InShaderProfile)
		, EntryPoint(InEntryPoint)
		, Exports(InExports)
		, BatchBaseFilename(FPaths::GetBaseFilename(Input.GetSourceFilename()))
		, DumpDebugInfoPath(Input.DumpDebugInfoPath)
		, bDump(Input.DumpDebugInfoEnabled())
	{
		if (bDump)
		{
			DumpDisasmFilename = Input.DumpDebugInfoPath / TEXT("Output.d3dasm");
		}

		const bool bEnable16BitTypes =
			// 16bit types are SM6.2, so their support at runtime is guaranteed in SM6.6.
			(ShaderModel >= ED3DShaderModel::SM6_6 && Input.Environment.CompilerFlags.Contains(CFLAG_AllowRealTypes))

			// Enable 16bit_types to reduce DXIL size (compiler bug - will be fixed)
			|| Input.IsRayTracingShader();

		const bool bHlslVersion2021 = Input.Environment.CompilerFlags.Contains(CFLAG_HLSL2021);
		if (bHlslVersion2021)
		{
			ExtraArguments.Add(TEXT("-HV"));
			ExtraArguments.Add(TEXT("2021"));
		}
		else
		{
			ExtraArguments.Add(TEXT("-HV"));
			ExtraArguments.Add(TEXT("2018"));
		}

		// Unpack uniform matrices as row-major to match the CPU layout.
		ExtraArguments.Add(TEXT("-Zpr"));

		if (Input.Environment.CompilerFlags.Contains(CFLAG_SkipValidation))
		{
			ExtraArguments.Add(TEXT("-Vd"));
		}

		if (Input.Environment.CompilerFlags.Contains(CFLAG_Debug) || Input.Environment.CompilerFlags.Contains(CFLAG_SkipOptimizationsDXC))
		{
			ExtraArguments.Add(TEXT("-Od"));
		}
		else if (Input.Environment.CompilerFlags.Contains(CFLAG_StandardOptimization))
		{
			ExtraArguments.Add(TEXT("-O1"));
		}
		else
		{
			ExtraArguments.Add(TEXT("-O3"));
		}

		if (Input.Environment.CompilerFlags.Contains(CFLAG_PreferFlowControl))
		{
			ExtraArguments.Add(TEXT("-Gfp"));
		}

		if (Input.Environment.CompilerFlags.Contains(CFLAG_AvoidFlowControl))
		{
			ExtraArguments.Add(TEXT("-Gfa"));
		}

		if (Input.Environment.CompilerFlags.Contains(CFLAG_WarningsAsErrors))
		{
			ExtraArguments.Add(TEXT("-WX"));
		}

		const uint32 AutoBindingSpace = GetAutoBindingSpace(Input.Target);
		{
			ExtraArguments.Add(TEXT("-auto-binding-space"));
			ExtraArguments.Add(FString::Printf(TEXT("%d"), AutoBindingSpace));
		}

		if (Exports.Len() > 0)
		{
			// Ensure that only the requested functions exists in the output DXIL.
			// All other functions and their used resources must be eliminated.
			ExtraArguments.Add(TEXT("-exports"));
			ExtraArguments.Add(Exports);
		}

		if (bEnable16BitTypes)
		{
			ExtraArguments.Add(TEXT("-enable-16bit-types"));
		}

		if (Input.Environment.CompilerFlags.Contains(CFLAG_GenerateSymbols))
		{
			if (Input.Environment.CompilerFlags.Contains(CFLAG_AllowUniqueSymbols))
			{
				// -Zss Compute Shader Hash considering source information
				ExtraArguments.Add(TEXT("-Zss"));
			}
			else
			{
				// -Zsb Compute Shader Hash considering only output binary
				ExtraArguments.Add(TEXT("-Zsb"));
			}

			// generate the debug information (PDB)
			ExtraArguments.Add(TEXT("-Zi"));

			// always strip the PDB from the DXIL output blob, we retrieve from the compile result and save it manually in the PlatformDebugData
			ExtraArguments.Add("-Qstrip_debug");
		}

		// disable undesired warnings
		ExtraArguments.Add(TEXT("-Wno-parentheses-equality"));
        ExtraArguments.Add(TEXT("-Wno-for-redefinition"));

		// working around bindless conversion specific issue where globallycoherent on a function return type is flagged as ignored even though it is necessary.
		// github issue: https://github.com/microsoft/DirectXShaderCompiler/issues/4537
		if (Input.IsBindlessEnabled())
		{
			ExtraArguments.Add(TEXT("-Wno-ignored-attributes"));
		}

		// @lh-todo: This fixes a loop unrolling issue that showed up in DOFGatherKernel with cs_6_6 with the latest DXC revision
		ExtraArguments.Add(TEXT("-disable-lifetime-markers"));

		// temporary change until nsight supports reconstructing reflection data from PDB; don't strip reflection if CFLAG_ExtraShaderData is specified
		bStripReflection = !Input.Environment.CompilerFlags.Contains(CFLAG_ExtraShaderData);
	}

	FString GetDumpDebugInfoPath() const
	{
		return DumpDebugInfoPath;
	}

	bool ShouldDump() const
	{
		return bDump;
	}

	bool ShouldStripReflectionData() const
	{
		return bStripReflection;
	}

	FString GetEntryPointName() const
	{
		return Exports.Len() > 0 ? FString(TEXT("")) : EntryPoint;
	}

	const FString& GetShaderProfile() const
	{
		return ShaderProfile;
	}

	const FString& GetDumpDisassemblyFilename() const
	{
		return DumpDisasmFilename;
	}

	void GetCompilerArgsNoEntryNoProfileNoDisasm(TArray<const WCHAR*>& Out) const
	{
		for (const FString& Entry : ExtraArguments)
		{
			Out.Add(*Entry);
		}
	}

	void GetCompilerArgs(TArray<const WCHAR*>& Out) const
	{
		GetCompilerArgsNoEntryNoProfileNoDisasm(Out);
		if (Exports.Len() == 0)
		{
			Out.Add(TEXT("-E"));
			Out.Add(*EntryPoint);
		}

		Out.Add(TEXT("-T"));
		Out.Add(*ShaderProfile);
	}

	const FString& GetBatchBaseFilename() const
	{
		return BatchBaseFilename;
	}

	FString GetBatchCommandLineString() const
	{
		FString DXCCommandline;
		for (const FString& Entry : ExtraArguments)
		{
			DXCCommandline += TEXT(" ");
			DXCCommandline += Entry;
		}

		DXCCommandline += TEXT(" -T ");
		DXCCommandline += ShaderProfile;

		if (Exports.Len() == 0)
		{
			DXCCommandline += TEXT(" -E ");
			DXCCommandline += EntryPoint;
		}

		DXCCommandline += TEXT(" -Fc ");
		DXCCommandline += BatchBaseFilename + TEXT(".d3dasm");

		DXCCommandline += TEXT(" -Fo ");
		DXCCommandline += BatchBaseFilename + TEXT(".dxil");

		return DXCCommandline;
	}
};


static dxc::DxcDllSupport& GetDxcDllHelper()
{
	struct DxcDllHelper
	{
		DxcDllHelper()
		{
			VERIFYHRESULT(DxcDllSupport.Initialize());
		}
		dxc::DxcDllSupport DxcDllSupport;
	};

	static DxcDllHelper DllHelper;
	return DllHelper.DxcDllSupport;
}

static FString DxcBlobEncodingToFString(TRefCountPtr<IDxcBlobEncoding> DxcBlob)
{
	FString OutString;
	if (DxcBlob && DxcBlob->GetBufferSize())
	{
		ANSICHAR* Chars = new ANSICHAR[DxcBlob->GetBufferSize() + 1];
		FMemory::Memcpy(Chars, DxcBlob->GetBufferPointer(), DxcBlob->GetBufferSize());
		Chars[DxcBlob->GetBufferSize()] = 0;
		OutString = Chars;
		delete[] Chars;
	}

	return OutString;
}

#if !PLATFORM_SEH_EXCEPTIONS_DISABLED && PLATFORM_WINDOWS
#include "Windows/WindowsPlatformCrashContext.h"
#include "HAL/PlatformStackWalk.h"
static char GDxcStackTrace[65536] = "";
static int32 HandleException(LPEXCEPTION_POINTERS ExceptionInfo)
{
	constexpr int32 NumStackFramesToIgnore = 1;
	GDxcStackTrace[0] = 0;
	FPlatformStackWalk::StackWalkAndDump(GDxcStackTrace, UE_ARRAY_COUNT(GDxcStackTrace), NumStackFramesToIgnore, nullptr);
	return EXCEPTION_EXECUTE_HANDLER;
}
#else
static const char* GDxcStackTrace = "";
#endif

static HRESULT InnerDXCCompileWrapper(
	TRefCountPtr<IDxcCompiler3>& Compiler,
	TRefCountPtr<IDxcBlobEncoding>& TextBlob,
	LPCWSTR* Arguments,
	uint32 NumArguments,
	bool& bOutExceptionError,
	TRefCountPtr<IDxcResult>& OutCompileResult)
{
	bOutExceptionError = false;
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED && PLATFORM_WINDOWS
	__try
#endif
	{
		DxcBuffer SourceBuffer = { 0 };
		SourceBuffer.Ptr = TextBlob->GetBufferPointer();
		SourceBuffer.Size = TextBlob->GetBufferSize();
		BOOL bKnown = 0;
		uint32 Encoding = 0;
		if (SUCCEEDED(TextBlob->GetEncoding(&bKnown, (uint32*)&Encoding)))
		{
			if (bKnown)
			{
				SourceBuffer.Encoding = Encoding;
			}
		}
		return Compiler->Compile(
			&SourceBuffer,						// source text to compile
			Arguments,							// array of pointers to arguments
			NumArguments,						// number of arguments
			nullptr,							// user-provided interface to handle #include directives (optional)
			IID_PPV_ARGS(OutCompileResult.GetInitReference())	// compiler output status, buffer, and errors
		);
	}
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED && PLATFORM_WINDOWS
	__except (HandleException(GetExceptionInformation()))
	{
		bOutExceptionError = true;
		return E_FAIL;
	}
#endif
}

static HRESULT DXCCompileWrapper(
	TRefCountPtr<IDxcCompiler3>& Compiler,
	TRefCountPtr<IDxcBlobEncoding>& TextBlob,
	const FDxcArguments& Arguments,
	TRefCountPtr<IDxcResult>& OutCompileResult)
{
	bool bExceptionError = false;

	TArray<const WCHAR*> CompilerArgs;
	Arguments.GetCompilerArgs(CompilerArgs);

	// Give a unique name to the d3dasm and dxil outputs (Must have same scope as CompilerArgs so the temporary strings remain valid)
	FString AsmFilename  = Arguments.GetBatchBaseFilename() + TEXT(".d3dasm");
	FString DXILFilename = Arguments.GetBatchBaseFilename() + TEXT(".dxil");
	CompilerArgs.Add(TEXT(" -Fc "));
	CompilerArgs.Add(*AsmFilename);
	CompilerArgs.Add(TEXT(" -Fo "));
	CompilerArgs.Add(*DXILFilename);

	HRESULT Result = InnerDXCCompileWrapper(Compiler, TextBlob,
		CompilerArgs.GetData(), CompilerArgs.Num(), bExceptionError, OutCompileResult);

	if (bExceptionError)
	{
		FSCWErrorCode::Report(FSCWErrorCode::CrashInsidePlatformCompiler);

		FString ErrorMsg = TEXT("Internal error or exception inside dxcompiler.dll\n");
		ErrorMsg += GDxcStackTrace;

		FCString::Strcpy(GErrorExceptionDescription, *ErrorMsg);

#if !PLATFORM_SEH_EXCEPTIONS_DISABLED && PLATFORM_WINDOWS
		// Throw an exception so SCW can send it back in the output file
		FPlatformMisc::RaiseException(EXCEPTION_EXECUTE_HANDLER);
#endif
	}

	return Result;
}

static void SaveDxcBlobToFile(IDxcBlob* Blob, const FString& Filename)
{
	const uint8* DxilData = (const uint8*)Blob->GetBufferPointer();
	uint32 DxilSize = Blob->GetBufferSize();
	TArrayView<const uint8> Contents(DxilData, DxilSize);
	FFileHelper::SaveArrayToFile(Contents, *Filename);
}

static void DisassembleAndSave(TRefCountPtr<IDxcCompiler3>& Compiler, IDxcBlob* Dxil, const FString& DisasmFilename)
{
	TRefCountPtr<IDxcResult> DisasmResult;
	DxcBuffer DisasmBuffer = { 0 };
	DisasmBuffer.Size = Dxil->GetBufferSize();
	DisasmBuffer.Ptr = Dxil->GetBufferPointer();
	if (SUCCEEDED(Compiler->Disassemble(&DisasmBuffer, IID_PPV_ARGS(DisasmResult.GetInitReference()))))
	{
		HRESULT DisasmCodeResult;
		DisasmResult->GetStatus(&DisasmCodeResult);
		if (SUCCEEDED(DisasmCodeResult))
		{
			checkf(DisasmResult->HasOutput(DXC_OUT_DISASSEMBLY), TEXT("Disasm part missing but container said it has one!"));
			TRefCountPtr<IDxcBlobEncoding> DisasmBlob;
			TRefCountPtr<IDxcBlobUtf16> Dummy;
			VERIFYHRESULT(DisasmResult->GetOutput(DXC_OUT_DISASSEMBLY, IID_PPV_ARGS(DisasmBlob.GetInitReference()), Dummy.GetInitReference()));
			FString String = DxcBlobEncodingToFString(DisasmBlob);
			FFileHelper::SaveStringToFile(String, *DisasmFilename);
		}
	}
}

static HRESULT RemoveReflectionData(dxc::DxcDllSupport& DxcDllHelper, TRefCountPtr<IDxcBlob>& Dxil)
{
	TRefCountPtr<IDxcOperationResult> Result;
	TRefCountPtr<IDxcContainerBuilder> Builder;
	TRefCountPtr<IDxcBlob> StrippedDxil;

	VERIFYHRESULT(DxcDllHelper.CreateInstance2(UE::ShaderCompilerCommon::GetDxcMalloc(), CLSID_DxcContainerBuilder, Builder.GetInitReference()));
	VERIFYHRESULT(Builder->Load(Dxil));

	HRESULT Res = Builder->RemovePart(DXC_PART_REFLECTION_DATA);
	if (FAILED(Res))
	{
		return Res;
	}

	Res = Builder->SerializeContainer(Result.GetInitReference());
	if (FAILED(Res))
	{
		return Res;
	}

	Res = Result->GetResult(StrippedDxil.GetInitReference());
	if (SUCCEEDED(Res))
	{
		Dxil.SafeRelease();
		Dxil = StrippedDxil;
	}
	return Res;
}

static HRESULT D3DCompileToDxil(const char* SourceText, const FDxcArguments& Arguments,
	TRefCountPtr<IDxcBlob>& OutDxilBlob, TRefCountPtr<IDxcBlob>& OutReflectionBlob, TRefCountPtr<IDxcBlobEncoding>& OutErrorBlob, TRefCountPtr<IDxcBlob>& OutPdbBlob, FString& OutPdbName, DxcShaderHash& OutHash)
{
	dxc::DxcDllSupport& DxcDllHelper = GetDxcDllHelper();

	TRefCountPtr<IDxcCompiler3> Compiler;
	VERIFYHRESULT(DxcDllHelper.CreateInstance2(UE::ShaderCompilerCommon::GetDxcMalloc(), CLSID_DxcCompiler, Compiler.GetInitReference()));

	TRefCountPtr<IDxcLibrary> Library;
	VERIFYHRESULT(DxcDllHelper.CreateInstance2(UE::ShaderCompilerCommon::GetDxcMalloc(), CLSID_DxcLibrary, Library.GetInitReference()));

	TRefCountPtr<IDxcBlobEncoding> TextBlob;
	VERIFYHRESULT(Library->CreateBlobWithEncodingFromPinned((LPBYTE)SourceText, FCStringAnsi::Strlen(SourceText), CP_UTF8, TextBlob.GetInitReference()));

	TRefCountPtr<IDxcResult> CompileResult;
	VERIFYHRESULT(DXCCompileWrapper(Compiler, TextBlob, Arguments, CompileResult));

	if (!CompileResult.IsValid())
	{
		return E_FAIL;
	}

	HRESULT CompileResultCode;
	CompileResult->GetStatus(&CompileResultCode);
	if (SUCCEEDED(CompileResultCode))
	{
		TRefCountPtr<IDxcBlobUtf16> ObjectCodeNameBlob; // Dummy name blob to silence static analysis warning
		checkf(CompileResult->HasOutput(DXC_OUT_OBJECT), TEXT("No object code found!"));
		VERIFYHRESULT(CompileResult->GetOutput(DXC_OUT_OBJECT, IID_PPV_ARGS(OutDxilBlob.GetInitReference()), ObjectCodeNameBlob.GetInitReference()));

		const bool bPostCompileSign = false;
		if (bPostCompileSign)
		{
			// https://www.wihlidal.com/blog/pipeline/2018-09-16-dxil-signing-post-compile/
			TRefCountPtr<IDxcValidator> Validator;
			VERIFYHRESULT(DxcDllHelper.CreateInstance2(UE::ShaderCompilerCommon::GetDxcMalloc(), CLSID_DxcValidator, Validator.GetInitReference()));

		#if 0
			struct FDxilMinimalHeader
			{
				uint32 FourCC;
				uint32 HashDigest[4];
			};

			FDxilMinimalHeader BeforeSignHeader = *reinterpret_cast<FDxilMinimalHeader*>(OutDxilBlob->GetBufferPointer());
			(void)BeforeSignHeader;
		#endif

			TRefCountPtr<IDxcOperationResult> ValidateResult;
			VERIFYHRESULT(Validator->Validate(OutDxilBlob.GetReference(), DxcValidatorFlags_InPlaceEdit, ValidateResult.GetInitReference()));

		#if 0
			FDxilMinimalHeader AfterSignHeader = *reinterpret_cast<FDxilMinimalHeader*>(OutDxilBlob->GetBufferPointer());
			(void)AfterSignHeader;
		#endif
		}

		TRefCountPtr<IDxcBlobUtf16> ReflectionNameBlob; // Dummy name blob to silence static analysis warning
		checkf(CompileResult->HasOutput(DXC_OUT_REFLECTION), TEXT("No reflection found!"));
		VERIFYHRESULT(CompileResult->GetOutput(DXC_OUT_REFLECTION, IID_PPV_ARGS(OutReflectionBlob.GetInitReference()), ReflectionNameBlob.GetInitReference()));
		RetrieveDebugNameAndBlob(CompileResult, OutPdbName, OutPdbBlob.GetInitReference(), OutHash);
 
		if (Arguments.ShouldDump())
		{
			// Dump disassembly before we strip reflection out
			const FString& DisasmFilename = Arguments.GetDumpDisassemblyFilename();
			check(DisasmFilename.Len() > 0);
			DisassembleAndSave(Compiler, OutDxilBlob, DisasmFilename);
		}

		if (Arguments.ShouldStripReflectionData())
		{
			HRESULT ReflectionStripResult = RemoveReflectionData(DxcDllHelper, OutDxilBlob);
			if (FAILED(ReflectionStripResult))
			{
				return ReflectionStripResult;
			}
		}
	}

	CompileResult->GetErrorBuffer(OutErrorBlob.GetInitReference());

	return CompileResultCode;
}

static FString D3DCreateDXCCompileBatchFile(const FDxcArguments& Args)
{
	FString DxcPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir());

	DxcPath = FPaths::Combine(DxcPath, TEXT("Binaries/ThirdParty/ShaderConductor/Win64"));
	FPaths::MakePlatformFilename(DxcPath);

	FString DxcFilename = FPaths::Combine(DxcPath, TEXT("dxc.exe"));
	FPaths::MakePlatformFilename(DxcFilename);

	const FString& BatchBaseFilename = Args.GetBatchBaseFilename();
	const FString BatchCmdLineArgs = Args.GetBatchCommandLineString();

	return FString::Printf(
		TEXT(
			"@ECHO OFF\n"
			"SET DXC=\"%s\"\n"
			"IF NOT EXIST %%DXC%% (\n"
			"\tECHO Couldn't find dxc.exe under \"%s\"\n"
			"\tGOTO :END\n"
			")\n"
			"%%DXC%%%s %s.usf\n"
			":END\n"
			"PAUSE\n"
		),
		*DxcFilename,
		*DxcPath,
		*BatchCmdLineArgs,
		*BatchBaseFilename
	);
}

inline bool IsCompatibleBinding(const D3D12_SHADER_INPUT_BIND_DESC& BindDesc, uint32 BindingSpace)
{
	bool bIsCompatibleBinding = (BindDesc.Space == BindingSpace);
	if (!bIsCompatibleBinding)
	{
		const bool bIsAMDExtensionDX12 = (FCStringAnsi::Strcmp(BindDesc.Name, "AmdExtD3DShaderIntrinsicsUAV") == 0);
		bIsCompatibleBinding = bIsAMDExtensionDX12 && (BindDesc.Space == AGS_DX12_SHADER_INSTRINSICS_SPACE_ID);
	}
	if (!bIsCompatibleBinding)
	{
		const bool bIsNVExtension = (FCStringAnsi::Strcmp(BindDesc.Name, "g_NvidiaExt") == 0);
		bIsCompatibleBinding = bIsNVExtension && (BindDesc.Space == UE_HLSL_SPACE_NV_SHADER_EXTN);
	}
	if (!bIsCompatibleBinding)
	{
		const bool bIsUEDebugBuffer = (FCStringAnsi::Strcmp(BindDesc.Name, "UEDiagnosticBuffer") == 0);
		bIsCompatibleBinding = bIsUEDebugBuffer && (BindDesc.Space == UE_HLSL_SPACE_DIAGNOSTIC);
	}
	if (!bIsCompatibleBinding)
	{
		const bool bIsUERootConstants = (FCStringAnsi::Strcmp(BindDesc.Name, "UERootConstants") == 0);
		bIsCompatibleBinding = bIsUERootConstants && (BindDesc.Space == UE_HLSL_SPACE_SHADER_ROOT_CONSTANTS);
	}

	return bIsCompatibleBinding;
}

// Generate the dumped usf file; call the D3D compiler, gather reflection information and generate the output data
bool CompileAndProcessD3DShaderDXC(
	const FShaderCompilerInput& Input,
	const FString& PreprocessedShaderSource,
	const FString& EntryPointName,
	const FShaderParameterParser& ShaderParameterParser,
	const TCHAR* ShaderProfile,
	const ED3DShaderModel ShaderModel,
	const bool bProcessingSecondTime,
	FShaderCompilerOutput& Output)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CompileAndProcessD3DShaderDXC);

	auto AnsiSourceFile = StringCast<ANSICHAR>(*PreprocessedShaderSource);

	const bool bIsRayTracingShader = Input.IsRayTracingShader();
	const bool bIsWorkGraphShader = Input.IsWorkGraphShader();

	const uint32 AutoBindingSpace = GetAutoBindingSpace(Input.Target);

	FString RayEntryPoint; // Primary entry point for all ray tracing shaders
	FString RayAnyHitEntryPoint; // Optional for hit group shaders
	FString RayIntersectionEntryPoint; // Optional for hit group shaders
	FString RayTracingExports;

	if (bIsRayTracingShader)
	{
		UE::ShaderCompilerCommon::ParseRayTracingEntryPoint(Input.EntryPointName, RayEntryPoint, RayAnyHitEntryPoint, RayIntersectionEntryPoint);

		RayTracingExports = RayEntryPoint;

		if (!RayAnyHitEntryPoint.IsEmpty())
		{
			RayTracingExports += TEXT(";");
			RayTracingExports += RayAnyHitEntryPoint;
		}

		if (!RayIntersectionEntryPoint.IsEmpty())
		{
			RayTracingExports += TEXT(";");
			RayTracingExports += RayIntersectionEntryPoint;
		}
	}

	FDxcArguments Args
	(
		Input,
		EntryPointName,
		ShaderProfile,
		ShaderModel,
		RayTracingExports
	);

	if (Args.ShouldDump())
	{
		const FString BatchFileContents = D3DCreateDXCCompileBatchFile(Args);
		FFileHelper::SaveStringToFile(BatchFileContents, *(Args.GetDumpDebugInfoPath() / TEXT("CompileDXC.bat")));
	}

	TRefCountPtr<IDxcBlob> ShaderBlob;
	TRefCountPtr<IDxcBlob> ReflectionBlob;
	TRefCountPtr<IDxcBlobEncoding> DxcErrorBlob;
	TRefCountPtr<IDxcBlob> PdbBlob;
	FString PdbName;
	DxcShaderHash ShaderHash;
	const HRESULT D3DCompileToDxilResult = D3DCompileToDxil(AnsiSourceFile.Get(), Args, ShaderBlob, ReflectionBlob, DxcErrorBlob, PdbBlob, PdbName, ShaderHash);

	Output.AddStatistic(UE::ShaderCompilerCommon::kPlatformHashStatName, BytesToHex(ShaderHash.HashDigest, sizeof(ShaderHash.HashDigest)), FGenericShaderStat::EFlags::Hidden);
	
	// Populate the platform-specific debug data with the PDB name and/or data, if available and requested.
	bool bWriteSymbolsInfo = Input.Environment.CompilerFlags.Contains(CFLAG_GenerateSymbolsInfo);
	bool bWriteSymbols = Input.Environment.CompilerFlags.Contains(CFLAG_GenerateSymbols);
	if ((bWriteSymbols || bWriteSymbolsInfo) && !PdbName.IsEmpty())
	{
		check(!bWriteSymbols || (PdbBlob.IsValid() && (PdbBlob->GetBufferSize() > 0)));
		FD3DShaderDebugData DebugData;
		FD3DShaderDebugData::FFile& PdbFile = DebugData.Files.AddDefaulted_GetRef();
		PdbFile.Name = PdbName;

		if (bWriteSymbols)
		{
			PdbFile.Contents = MakeArrayViewFromBlob(PdbBlob);

			// also export the .dxil file alongside the .pdb if symbols are on
			FD3DShaderDebugData::FFile& DxilFile = DebugData.Files.AddDefaulted_GetRef();
			DxilFile.Name = FPaths::ChangeExtension(PdbName, TEXT(".dxil"));
			DxilFile.Contents = MakeArrayViewFromBlob(ShaderBlob);
		}
		
		FMemoryWriter Ar(Output.ShaderCode.GetSymbolWriteAccess());
		Ar << DebugData;
	}

	if (DxcErrorBlob && DxcErrorBlob->GetBufferSize())
	{
		FAnsiStringView ErrorsString(static_cast<ANSICHAR*>(DxcErrorBlob->GetBufferPointer()), DxcErrorBlob->GetBufferSize());
		CrossCompiler::FShaderConductorContext::ConvertCompileErrors(ErrorsString, Output.Errors);
	}

	if (SUCCEEDED(D3DCompileToDxilResult))
	{
		// Gather reflection information
		FD3DShaderCompileData CompileData;
		CompileData.bBindlessEnabled = Input.IsBindlessEnabled();

		if (Input.IsBindlessEnabled())
		{
			CompileData.MaxSamplers = D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE;
		}
		else if (ShaderModel == ED3DShaderModel::SM6_6)
		{
			CompileData.MaxSamplers = 32; // DDSPI: MaxSamplers=32
		}
		else
		{
			CompileData.MaxSamplers = D3D12_COMMONSHADER_SAMPLER_REGISTER_COUNT;
		}

		if (Input.IsBindlessEnabled())
		{
			CompileData.MaxSRVs = D3D12_MAX_SHADER_VISIBLE_DESCRIPTOR_HEAP_SIZE_TIER_2;
		}
		else
		{
			static_assert(MAX_SRVS <= D3D12_COMMONSHADER_INPUT_RESOURCE_SLOT_COUNT);
			CompileData.MaxSRVs = MAX_SRVS; // Max for D3D12RHI bindful
		}
		CompileData.MaxCBs = MAX_CBS; // Max for D3D12RHI
		CompileData.MaxUAVs = MAX_UAVS; // Max for D3D12RHI

		uint64 ShaderRequiresFlags{};

		dxc::DxcDllSupport& DxcDllHelper = GetDxcDllHelper();
		TRefCountPtr<IDxcUtils> Utils;
		VERIFYHRESULT(DxcDllHelper.CreateInstance2(UE::ShaderCompilerCommon::GetDxcMalloc(), CLSID_DxcUtils, Utils.GetInitReference()));
		DxcBuffer ReflBuffer = { 0 };
		ReflBuffer.Ptr = ReflectionBlob->GetBufferPointer();
		ReflBuffer.Size = ReflectionBlob->GetBufferSize();

		bool bHasNoDerivativeOps = false;

		if ((Input.Target.GetFrequency() == SF_Compute || Input.Target.GetFrequency() == SF_WorkGraphComputeNode) && Input.Environment.CompilerFlags.Contains(CFLAG_CheckForDerivativeOps))
		{
			TRefCountPtr<IDxcContainerReflection> ContainerRefl;
			VERIFYHRESULT(DxcDllHelper.CreateInstance2(UE::ShaderCompilerCommon::GetDxcMalloc(), CLSID_DxcContainerReflection, ContainerRefl.GetInitReference()));
			VERIFYHRESULT(ContainerRefl->Load(ShaderBlob));

			uint32 PartCount = 0;
			VERIFYHRESULT(ContainerRefl->GetPartCount(&PartCount));

			for (uint32 PartIndex = 0; PartIndex < PartCount; ++PartIndex)
			{
				uint32 PartKind;
				VERIFYHRESULT(ContainerRefl->GetPartKind(PartIndex, &PartKind));

				if (PartKind == DXC_PART_PRIVATE_DATA)
				{
					struct UE5CustomData
					{
						uint32_t FourCC;
						uint64_t Data;
					};

					TRefCountPtr<IDxcBlob> UserPartBlob;
					ContainerRefl->GetPartContent(PartIndex, UserPartBlob.GetInitReference());
					if (UserPartBlob->GetBufferSize() == sizeof(UE5CustomData))
					{
						const UE5CustomData& CustomData = *(UE5CustomData*)UserPartBlob->GetBufferPointer();
						if (CustomData.FourCC == DXC_PART_FEATURE_INFO)
						{
							bHasNoDerivativeOps = (CustomData.Data & hlsl::DXIL::OptFeatureInfo_UsesDerivatives) == 0;
						}
					}
					break;
				}
			}
		}

		// Remove unused interpolators from pixel shader 
		// (propagated to corresponding VS from pipeline by later setting Output.bSupportsQueryingUsedAttributes and Output.UsedAttributes)
		{
			TRefCountPtr<ID3D12ShaderReflection> Reflector;
			Utils->CreateReflection(&ReflBuffer, IID_PPV_ARGS(Reflector.GetInitReference()));

			ShaderCompileLambdaType ShaderCompileLambda = [](
				const FShaderCompilerInput& Input,
				const FString& PreprocessedShaderSource,
				const FString& EntryPointName,
				const FShaderParameterParser& ShaderParameterParser,
				const TCHAR* ShaderProfile,
				const ED3DShaderModel ShaderModel,
				const bool bProcessingSecondTime,
				FShaderCompilerOutput& Output)
				{
					return CompileAndProcessD3DShaderDXC(
						Input,
						PreprocessedShaderSource,
						EntryPointName,
						ShaderParameterParser,
						ShaderProfile,
						ShaderModel,
						bProcessingSecondTime,
						Output);
				};

			bool CompileResult = false;
			const bool RemovedUnusedInterpolatorsApplied =
				RemoveUnusedInterpolators<ShaderCompilerType::DXC, ShaderCompileLambdaType,
				ID3D12ShaderReflection, D3D12_SHADER_DESC, D3D12_SIGNATURE_PARAMETER_DESC>(
					Input,
					PreprocessedShaderSource,
					EntryPointName,
					ShaderParameterParser,
					ShaderProfile,
					ShaderModel,
					bProcessingSecondTime,
					CompileData,
					Reflector,
					ShaderCompileLambda,
					Output,
					CompileResult);
			if (RemovedUnusedInterpolatorsApplied)
			{
				return CompileResult;
			}
		}

		if (bIsWorkGraphShader || bIsRayTracingShader)
		{
			TRefCountPtr<ID3D12LibraryReflection> LibraryReflection;
			VERIFYHRESULT(Utils->CreateReflection(&ReflBuffer, IID_PPV_ARGS(LibraryReflection.GetInitReference())));

			D3D12_LIBRARY_DESC LibraryDesc = {};
			LibraryReflection->GetDesc(&LibraryDesc);

			ID3D12FunctionReflection* FunctionReflection = nullptr;
			D3D12_FUNCTION_DESC FunctionDesc = {};

			bool bEntryPointsAreMangled = false;
			TArray<FString, TInlineAllocator<3>> EntryPoints;
			if (bIsRayTracingShader)
			{
				// EntryPoints contains partial mangled entry point signatures in a the following form:
				// ?QualifiedName@ (as described here: https://en.wikipedia.org/wiki/Name_mangling)
				// Entry point parameters are currently not included in the partial mangling.
				bEntryPointsAreMangled = true;

				if (!RayEntryPoint.IsEmpty())
				{
					EntryPoints.Add(FString::Printf(TEXT("?%s@"), *RayEntryPoint));
				}
				if (!RayAnyHitEntryPoint.IsEmpty())
				{
					EntryPoints.Add(FString::Printf(TEXT("?%s@"), *RayAnyHitEntryPoint));
				}
				if (!RayIntersectionEntryPoint.IsEmpty())
				{
					EntryPoints.Add(FString::Printf(TEXT("?%s@"), *RayIntersectionEntryPoint));
				}
			}
			else
			{
				EntryPoints.Add(Input.EntryPointName);
			}

			uint32 NumFoundEntryPoints = 0;

			for (uint32 FunctionIndex = 0; FunctionIndex < LibraryDesc.FunctionCount; ++FunctionIndex)
			{
				FunctionReflection = LibraryReflection->GetFunctionByIndex(FunctionIndex);
				FunctionReflection->GetDesc(&FunctionDesc);

				ShaderRequiresFlags |= FunctionDesc.RequiredFeatureFlags;

				bool bAddFunctionEntryPoint = false;
				for (const FString& EntryPoint : EntryPoints)
				{
					// Entry point parameters are currently not included in the partial mangling, therefore partial substring match is used here.
					if (bEntryPointsAreMangled && FCStringAnsi::Strstr(FunctionDesc.Name, TCHAR_TO_ANSI(*EntryPoint)))
					{
						bAddFunctionEntryPoint = true;
						break;
					}
					else if (!bEntryPointsAreMangled && FunctionDesc.Name == EntryPoint)
					{
						bAddFunctionEntryPoint = true;
						break;
					}
				}

				if (bAddFunctionEntryPoint)
				{
					// Note: calling ExtractParameterMapFromD3DShader multiple times merges the reflection data for multiple functions
					ExtractParameterMapFromD3DShader<ID3D12FunctionReflection, D3D12_FUNCTION_DESC, D3D12_SHADER_INPUT_BIND_DESC,
						ID3D12ShaderReflectionConstantBuffer, D3D12_SHADER_BUFFER_DESC,
						ID3D12ShaderReflectionVariable, D3D12_SHADER_VARIABLE_DESC>(
							Input,
							ShaderParameterParser,
							AutoBindingSpace,
							FunctionReflection,
							FunctionDesc, 
							CompileData,
							Output);

					NumFoundEntryPoints++;
				}
			}

			// @todo - working around DXC issue https://github.com/microsoft/DirectXShaderCompiler/issues/4715
			if (LibraryDesc.FunctionCount > 0)
			{
				if (CompileData.bBindlessEnabled)
				{
					ShaderRequiresFlags |= D3D_SHADER_REQUIRES_RESOURCE_DESCRIPTOR_HEAP_INDEXING;
					ShaderRequiresFlags |= D3D_SHADER_REQUIRES_SAMPLER_DESCRIPTOR_HEAP_INDEXING;
				}
			}

			if (NumFoundEntryPoints == EntryPoints.Num())
			{
				Output.bSucceeded = true;

				bool bGlobalUniformBufferAllowed = false;

				if (CompileData.bGlobalUniformBufferUsed && !IsGlobalConstantBufferSupported(Input.Target))
				{
					const TCHAR* ShaderFrequencyString = GetShaderFrequencyString(Input.Target.GetFrequency(), false);
					FString ErrorString = FString::Printf(TEXT("Global uniform buffer cannot be used in a %s shader."), ShaderFrequencyString);

					uint32 NumLooseParameters = 0;
					for (const auto& It : Output.ParameterMap.ParameterMap)
					{
						if (It.Value.Type == EShaderParameterType::LooseData)
						{
							NumLooseParameters++;
						}
					}

					if (NumLooseParameters)
					{
						ErrorString += TEXT(" Global parameters: ");
						uint32 ParameterIndex = 0;
						for (const auto& It : Output.ParameterMap.ParameterMap)
						{
							if (It.Value.Type == EShaderParameterType::LooseData)
							{
								--NumLooseParameters;
								ErrorString += FString::Printf(TEXT("%s%s"), *It.Key, NumLooseParameters ? TEXT(", ") : TEXT("."));
							}
						}
					}

					Output.Errors.Emplace(MoveTemp(ErrorString));
					Output.bSucceeded = false;
				}
			}
			else
			{
				UE_LOGF(LogD3DShaderCompiler, Fatal, "Failed to find required points in the shader library.");
				Output.bSucceeded = false;
			}
		}
		else
		{
			Output.bSucceeded = true;

			TRefCountPtr<ID3D12ShaderReflection> ShaderReflection;
			VERIFYHRESULT(Utils->CreateReflection(&ReflBuffer, IID_PPV_ARGS(ShaderReflection.GetInitReference())));

			D3D12_SHADER_DESC ShaderDesc = {};
			ShaderReflection->GetDesc(&ShaderDesc);

			ShaderRequiresFlags = ShaderReflection->GetRequiresFlags();

			ExtractParameterMapFromD3DShader<ID3D12ShaderReflection, D3D12_SHADER_DESC, D3D12_SHADER_INPUT_BIND_DESC,
				ID3D12ShaderReflectionConstantBuffer, D3D12_SHADER_BUFFER_DESC,
				ID3D12ShaderReflectionVariable, D3D12_SHADER_VARIABLE_DESC>(
					Input,
					ShaderParameterParser,
					AutoBindingSpace,
					ShaderReflection,
					ShaderDesc,
					CompileData,
					Output
				);
		}

		if (!ValidateResourceCounts(CompileData, Output.Errors))
		{
			Output.bSucceeded = false;
		}

		FShaderCodePackedResourceCounts PackedResourceCounts{};

		if (Output.bSucceeded)
		{
			PackedResourceCounts = InitPackedResourceCounts(CompileData);

			if (Input.Environment.CompilerFlags.Contains(CFLAG_RootConstants))
			{
				PackedResourceCounts.UsageFlags |= EShaderResourceUsageFlags::RootConstants;
			}

			if (bHasNoDerivativeOps)
			{
				PackedResourceCounts.UsageFlags |= EShaderResourceUsageFlags::NoDerivativeOps;
			}

			if (Input.Environment.CompilerFlags.Contains(CFLAG_ShaderBundle))
			{
				PackedResourceCounts.UsageFlags |= EShaderResourceUsageFlags::ShaderBundle;
			}

			Output.bSucceeded = UE::ShaderCompilerCommon::ValidatePackedResourceCounts(Output, PackedResourceCounts);

			// Return code reflection if requested for shader analysis
			if (Input.Environment.CompilerFlags.Contains(CFLAG_OutputAnalysisArtifacts))
			{
				FGenericShaderStat ShaderCodeReflection;
				if (CrossCompiler::FShaderConductorContext::Disassemble(CrossCompiler::EShaderConductorIR::Dxil, ShaderBlob->GetBufferPointer(), ShaderBlob->GetBufferSize(), ShaderCodeReflection))
				{
					Output.ShaderStatistics.Add(MoveTemp(ShaderCodeReflection));
				}
			}
		}

		// Save results if compilation and reflection succeeded
		if (Output.bSucceeded)
		{
			uint32 RayTracingPayloadType = 0;
			uint32 RayTracingPayloadSize = 0;
			if (bIsRayTracingShader)
			{
				bool bArgFound = Input.Environment.GetCompileArgument(TEXT("RT_PAYLOAD_TYPE"), RayTracingPayloadType);
				checkf(bArgFound, TEXT("Ray tracing shaders must provide a payload type as this information is required for offline RTPSO compilation. Check that FShaderType::ModifyCompilationEnvironment correctly set this value."));
				bArgFound = Input.Environment.GetCompileArgument(TEXT("RT_PAYLOAD_MAX_SIZE"), RayTracingPayloadSize);
				checkf(bArgFound, TEXT("Ray tracing shaders must provide a payload size as this information is required for offline RTPSO compilation. Check that FShaderType::ModifyCompilationEnvironment correctly set this value."));
			}
			auto PostSRTWriterCallback = [&](FMemoryWriter& Ar)
			{
				if (bIsRayTracingShader)
				{
					Ar << RayEntryPoint;
					Ar << RayAnyHitEntryPoint;
					Ar << RayIntersectionEntryPoint;
					Ar << RayTracingPayloadType;
					Ar << RayTracingPayloadSize;
				}
			};

			auto AddOptionalDataCallback = [&](FShaderCode& ShaderCode)
			{
				FShaderCodeFeatures CodeFeatures;

				if ((ShaderRequiresFlags & D3D_SHADER_REQUIRES_WAVE_OPS) != 0)
				{
					EnumAddFlags(CodeFeatures.CodeFeatures, EShaderCodeFeatures::WaveOps);
				}

				if ((ShaderRequiresFlags & D3D_SHADER_REQUIRES_NATIVE_16BIT_OPS) != 0)
				{
					EnumAddFlags(CodeFeatures.CodeFeatures, EShaderCodeFeatures::SixteenBitTypes);
				}

				if ((ShaderRequiresFlags & D3D_SHADER_REQUIRES_TYPED_UAV_LOAD_ADDITIONAL_FORMATS) != 0)
				{
					EnumAddFlags(CodeFeatures.CodeFeatures, EShaderCodeFeatures::TypedUAVLoadsExtended);
				}

				if ((ShaderRequiresFlags & (D3D_SHADER_REQUIRES_ATOMIC_INT64_ON_TYPED_RESOURCE| D3D_SHADER_REQUIRES_ATOMIC_INT64_ON_GROUP_SHARED)) != 0)
				{
					EnumAddFlags(CodeFeatures.CodeFeatures, EShaderCodeFeatures::Atomic64);
				}

				if ((ShaderRequiresFlags & D3D_SHADER_REQUIRES_RESOURCE_DESCRIPTOR_HEAP_INDEXING) != 0)
				{
					EnumAddFlags(CodeFeatures.CodeFeatures, EShaderCodeFeatures::BindlessResources);
				}

				if ((ShaderRequiresFlags & D3D_SHADER_REQUIRES_SAMPLER_DESCRIPTOR_HEAP_INDEXING) != 0)
				{
					EnumAddFlags(CodeFeatures.CodeFeatures, EShaderCodeFeatures::BindlessSamplers);
				}

				if ((ShaderRequiresFlags & D3D_SHADER_REQUIRES_STENCIL_REF) != 0)
				{
					EnumAddFlags(CodeFeatures.CodeFeatures, EShaderCodeFeatures::StencilRef);
				}

				if ((ShaderRequiresFlags & D3D_SHADER_REQUIRES_BARYCENTRICS) != 0)
				{
					EnumAddFlags(CodeFeatures.CodeFeatures, EShaderCodeFeatures::BarycentricsSemantic);
				}

				// We only need this to appear when using a DXC shader
				ShaderCode.AddOptionalData<FShaderCodeFeatures>(CodeFeatures);

				if (ShaderModel >= ED3DShaderModel::SM6_0)
				{
					uint8 IsSM6 = 1;
					ShaderCode.AddOptionalData(EShaderOptionalDataKey::ShaderModel6, &IsSM6, 1);
				}

				// Store EntryPointName for possible use in work graph state object creation.
				if (bIsWorkGraphShader || Input.Target.GetFrequency() == SF_Pixel)
				{
					TArray<uint8> NameData;
					FMemoryWriter NameWriter(NameData);
					FString Name = Input.EntryPointName;
					NameWriter << Name;
					ShaderCode.AddOptionalData(EShaderOptionalDataKey::EntryPoint, NameData.GetData(), NameData.Num());
				}
			};

			// Return a fraction of the number of instructions as DXIL is more verbose than DXBC.
			// Ratio 119:307 was estimated by gathering average instruction count for D3D11 and D3D12 shaders in ShooterGame with result being ~ 357:921.
			constexpr uint32 DxbcToDxilInstructionRatio[2] = { 119, 307 };
			CompileData.NumInstructions = CompileData.NumInstructions * DxbcToDxilInstructionRatio[0] / DxbcToDxilInstructionRatio[1];

			//#todo-rco: Should compress ShaderCode?

			GenerateFinalOutput(
				ShaderBlob,
				Input,
				ShaderModel,
				bProcessingSecondTime,
				CompileData,
				PackedResourceCounts,
				Output,
				PostSRTWriterCallback,
				AddOptionalDataCallback
			);
		}
	}
	else
	{
		// If we failed and didn't get any error messages back from the compile call try and get a system error message.
		if (Output.Errors.Num() == 0)
		{
			TCHAR ErrorMsg[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, UE_ARRAY_COUNT(ErrorMsg), (int)D3DCompileToDxilResult);
			const bool bKnownError = ErrorMsg[0] != TEXT('\0');

			FString ErrorString = FString::Printf(TEXT("D3DCompileToDxil failed. Error code: %s (0x%08X)."), bKnownError ? ErrorMsg : TEXT("Unknown error"), (int)D3DCompileToDxilResult);
			Output.Errors.Emplace(MoveTemp(ErrorString));
		}
	}


	return Output.bSucceeded;
}

#undef VERIFYHRESULT
