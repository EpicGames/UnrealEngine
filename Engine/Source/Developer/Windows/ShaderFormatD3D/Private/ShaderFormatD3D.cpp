// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderFormatD3D.h"
#include "ShaderCompilerCommon.h"
#include "ShaderPreprocessor.h"
#include "ShaderPreprocessTypes.h"
#include "ShaderSymbolExport.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/IShaderFormatModule.h"
#include "DXCWrapper.h"

DEFINE_LOG_CATEGORY(LogD3DShaderCompiler);

// Shader formats
static FName NAME_PCD3D_SM6(TEXT("PCD3D_SM6"));
static FName NAME_PCD3D_SM5(TEXT("PCD3D_SM5"));
static FName NAME_PCD3D_ES3_1(TEXT("PCD3D_ES31"));

// Preferred shader code formats
static FName NAME_DXIL(TEXT("DXIL"));

static const FGuid UE_SHADER_PCD3D_SHARED_VER = FGuid("9D405D9B-44C0-4747-8E88-1EDB7413F5C3");
static const FGuid UE_SHADER_PCD3D_SM6_VER    = FGuid("123ac6e6-a0a2-4a5f-b8be-4102376788ab");
static const FGuid UE_SHADER_PCD3D_SM5_VER    = FGuid("F30C5AFA-77F7-48DE-8CC1-A075F179443C");
static const FGuid UE_SHADER_PCD3D_ES3_1_VER  = FGuid("952939D9-1156-4347-97E9-9FFEA1A9FE14");

class FShaderFormatD3D : public UE::ShaderCompilerCommon::FBaseShaderFormat 
{
	uint32 DxcVersionHash = 0;

#if WITH_ENGINE
	mutable FShaderSymbolExport ShaderSymbolExportSM6{ NAME_PCD3D_SM6 };
	mutable FShaderSymbolExport ShaderSymbolExportSM5{ NAME_PCD3D_SM5 };
	mutable FShaderSymbolExport ShaderSymbolExportES31{ NAME_PCD3D_ES3_1 };
#endif

public:

	FShaderFormatD3D(uint32 InDxcVersionHash)
		: DxcVersionHash(InDxcVersionHash)
	{
	}

	inline uint32 GetVersionHash(const FGuid& InVersion) const
	{
		const uint32 BaseHash = GetTypeHash(UE_SHADER_PCD3D_SHARED_VER);
		uint32 VersionHash = GetTypeHash(InVersion);

		return HashCombine(BaseHash, VersionHash);
	}

	virtual uint32 GetVersion(FName Format) const override
	{
		if (Format == NAME_PCD3D_SM6)
		{
			uint32 ShaderModelHash = GetVersionHash(UE_SHADER_PCD3D_SM6_VER);

			// Make sure we recompile if EShaderCodeFeatures gets bigger
			ShaderModelHash = HashCombine(ShaderModelHash, GetTypeHash(sizeof(EShaderCodeFeatures)));

			return HashCombine(DxcVersionHash, ShaderModelHash);
		}
		else if (Format == NAME_PCD3D_SM5)
		{
			uint32 ShaderModelHash = GetVersionHash(UE_SHADER_PCD3D_SM6_VER);

			// Make sure we recompile if EShaderCodeFeatures gets bigger
			ShaderModelHash = HashCombine(ShaderModelHash, GetTypeHash(sizeof(EShaderCodeFeatures)));

			// Technically not needed for regular SM5 compiled with legacy compiler,
			// but PCD3D_SM5 currently includes ray tracing shaders that are compiled with new compiler stack.
			return HashCombine(DxcVersionHash, ShaderModelHash);
		}
		else if (Format == NAME_PCD3D_ES3_1) 
		{
			// Shader DXC signature is intentionally not included, as ES3_1 target always uses legacy compiler.
			return GetVersionHash(UE_SHADER_PCD3D_ES3_1_VER);
		}
		checkf(0, TEXT("Unknown Format %s"), *Format.ToString());
		return 0;
	}

	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const
	{
		OutFormats.Add(NAME_PCD3D_SM6);
		OutFormats.Add(NAME_PCD3D_SM5);
		OutFormats.Add(NAME_PCD3D_ES3_1);
	}

	static bool IsValidD3DShaderFormat(FName Format)
	{
		return
		(
			Format == NAME_PCD3D_SM6 ||
			Format == NAME_PCD3D_SM5 ||
			Format == NAME_PCD3D_ES3_1
		);
	}

	static bool IsSM68(const FShaderCompilerInput& Input)
	{
		return IsWorkGraphShaderFrequency(Input.Target.GetFrequency());
	}

	static bool IsSM66(const FShaderCompilerInput& Input)
	{
		return Input.ShaderFormat == NAME_PCD3D_SM6 || IsRayTracingShaderFrequency(Input.Target.GetFrequency());
	}

	// Do we need any SM6.0 features?
	static bool RequiresSM6Features(const FShaderCompilerInput& Input)
	{
		return IsSM68(Input)
			|| IsSM66(Input)
			|| Input.Environment.CompilerFlags.Contains(CFLAG_WaveOperations)
			// TODO: Forcing DXC should not change platform flags, this needs to be moved to IsSM60 once existing uses are accounted for.
			|| Input.Environment.CompilerFlags.Contains(CFLAG_ForceDXC)
			|| Input.PreferredShaderCodeFormat == NAME_DXIL
			;
	}

	// Do we need to compile with SM6.0 at all? We can compile with 6.0 but not require the language features
	static bool IsSM60(const FShaderCompilerInput& Input)
	{
		return RequiresSM6Features(Input)
			|| Input.Environment.GetCompileArgument(TEXT("PLATFORM_MAX_SAMPLERS"), 0) > 16
			;
	}

	static ED3DShaderModel DetermineShaderModel(const FShaderCompilerInput& Input)
	{
		if (IsSM68(Input))
		{
			return ED3DShaderModel::SM6_8;
		}

		if (IsSM66(Input))
		{
			return ED3DShaderModel::SM6_6;
		}

		if (IsSM60(Input))
		{
			return ED3DShaderModel::SM6_0;
		}

		return ED3DShaderModel::SM5_0;
	}

#if WITH_ENGINE
	virtual void NotifyShaderCompiled(const FCompressedBuffer& SymbolData, FName Format, const FString& DebugInfo) const override
	{
		if (Format == NAME_PCD3D_SM6)
		{
			ShaderSymbolExportSM6.NotifyShaderCompiled<FD3DShaderDebugData>(SymbolData, DebugInfo);
		}
		else if (Format == NAME_PCD3D_SM5)
		{
			ShaderSymbolExportSM5.NotifyShaderCompiled<FD3DShaderDebugData>(SymbolData, DebugInfo);
		}
		else if (Format == NAME_PCD3D_ES3_1)
		{
			ShaderSymbolExportES31.NotifyShaderCompiled<FD3DShaderDebugData>(SymbolData, DebugInfo);
		}
	}

	virtual void NotifyShaderCompilersShutdown(FName Format) const override
	{
		if (Format == NAME_PCD3D_SM6)
		{
			ShaderSymbolExportSM6.NotifyShaderCompilersShutdown();
		}
		else if (Format == NAME_PCD3D_SM5)
		{
			ShaderSymbolExportSM5.NotifyShaderCompilersShutdown();
		}
		else if (Format == NAME_PCD3D_ES3_1)
		{
			ShaderSymbolExportES31.NotifyShaderCompilersShutdown();
		}
	}
#endif

	virtual void CompilePreprocessedShader(
		const FShaderCompilerInput& Input, 
		const FShaderPreprocessOutput& PreprocessOutput, 
		FShaderCompilerOutput& Output) const
	{
		// Fill in ShaderDiagnosticDatas into the output result before calling CompileD3DShader as it will use and serialize its content
		Output.ShaderDiagnosticDatas = PreprocessOutput.GetDiagnosticDatas();

		CompileD3DShader(Input, PreprocessOutput, Output, DetermineShaderModel(Input));
	}

	static void AddShaderTargetDefines(FShaderCompilerInput& Input, uint32 ShaderTargetMajor, uint32 ShaderTargetMinor)
	{
		// Inserting our own versions of these defines since we preprocess our shader source before we actually use something that defines them.
		Input.Environment.SetDefine(TEXT("__SHADER_TARGET_MAJOR"), ShaderTargetMajor);
		Input.Environment.SetDefine(TEXT("__SHADER_TARGET_MINOR"), ShaderTargetMinor);
	}

	void ModifyShaderCompilerInput(FShaderCompilerInput& Input) const final
	{
		checkf(IsValidD3DShaderFormat(Input.ShaderFormat), TEXT("Unknown D3D shader format %s"), *Input.ShaderFormat.ToString());
		const ED3DShaderModel ShaderModel = DetermineShaderModel(Input);

		// Our compilers only support HLSL
		Input.Environment.SetDefine(TEXT("COMPILER_HLSL"), true);

		// Assume min. spec HW supports with DX12/SM5
		Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_ROV"), true);

		const bool bSM6Features = RequiresSM6Features(Input);
		const bool bDXC         = DoesShaderModelRequireDXC(ShaderModel);

		// Compiler specific defines
		Input.Environment.SetDefine(TEXT("COMPILER_FXC"), !bDXC);
		Input.Environment.SetDefine(TEXT("COMPILER_DXC"),  bDXC);

		// Do we need SM6.0+ features enabled? This is intentionally disconnected from the ED3DShaderModel to allow SM6.0 to be used without new language features.
		Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_CONSTANTBUFFER_OBJECT"), bSM6Features);
		Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS"), bSM6Features);
		Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_DIAGNOSTIC_BUFFER"),     bSM6Features);

		const bool bSM6Profile = ShaderModel >= ED3DShaderModel::SM6_6;
		const bool bSM5Profile = Input.ShaderFormat == NAME_PCD3D_SM5;

		// "profiles" are almost analogous to ERHIFeatureLevel but RT shaders are forcing themselves to SM6
		Input.Environment.SetDefine(TEXT("SM6_PROFILE"), bSM6Profile);
		Input.Environment.SetDefine(TEXT("SM5_PROFILE"), bSM5Profile);
		Input.Environment.SetDefine(TEXT("ES3_1_PROFILE"), (Input.ShaderFormat == NAME_PCD3D_ES3_1));

		if (!bSM6Profile && !bSM5Profile)
		{
			Input.Environment.SetDefine(TEXT("COMPILER_SUPPORTS_ATTRIBUTES"), true);
			Input.Environment.SetDefine(TEXT("PLATFORM_PREFER_SRGB_BRANCH"), true);
		}

		// Add SM6.6+ specific defines. None of these are intended to be enabled in lower SM's
		if (ShaderModel >= ED3DShaderModel::SM6_6)
		{
			Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_REAL_TYPES"),         Input.Environment.CompilerFlags.Contains(CFLAG_AllowRealTypes));
			Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_INLINE_RAY_TRACING"), Input.Environment.CompilerFlags.Contains(CFLAG_InlineRayTracing));

			Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_STATIC_SAMPLERS"),  true);
			Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_CALLABLE_SHADERS"), true);
			Input.Environment.SetDefine(TEXT("COMPILER_SUPPORTS_NOINLINE"),         true);
		}

		// Add defines for our specific shader model (5.0, 6.0, 6.6)
		switch (ShaderModel)
		{
		case ED3DShaderModel::SM5_0:
			AddShaderTargetDefines(Input, 5, 0);
			break;
		case ED3DShaderModel::SM6_0:
			AddShaderTargetDefines(Input, 6, 0);
			break;
		case ED3DShaderModel::SM6_6:
			AddShaderTargetDefines(Input, 6, 6);
			break;
		case ED3DShaderModel::SM6_8:
			AddShaderTargetDefines(Input, 6, 8);
			break;
		}

		// For mobile emulation
		if (Input.Environment.FullPrecisionInPS || (Input.SharedEnvironment.IsValid() && Input.SharedEnvironment->FullPrecisionInPS))
		{
			Input.Environment.SetDefine(TEXT("FORCE_FLOATS"), (uint32)1);
		}
	}

	virtual const TCHAR* GetPlatformIncludeDirectory() const
	{
		return TEXT("D3D");
	}
};


/**
 * Module for D3D shaders
 */

static IShaderFormat* Singleton = nullptr;

class FShaderFormatD3DModule : public IShaderFormatModule, public FDxcModuleWrapper
{
public:
	virtual ~FShaderFormatD3DModule()
	{
		delete Singleton;
		Singleton = nullptr;
	}

	virtual IShaderFormat* GetShaderFormat()
	{
		if (!Singleton)
		{
			Singleton = new FShaderFormatD3D(FDxcModuleWrapper::GetModuleVersionHash());
		}
		return Singleton;
	}
};

IMPLEMENT_MODULE( FShaderFormatD3DModule, ShaderFormatD3D);
