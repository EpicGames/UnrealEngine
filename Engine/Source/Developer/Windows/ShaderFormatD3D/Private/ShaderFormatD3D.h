// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HLSLReservedSpaces.h"
#include "Windows/WindowsHWrapper.h"
#include "ShaderCompilerCommon.h"

DECLARE_LOG_CATEGORY_EXTERN(LogD3DShaderCompiler, Log, All)

struct FShaderTarget;

enum class ED3DShaderModel
{
	Invalid,
	SM5_0,
	SM6_0,
	SM6_6,
	SM6_8,
};

inline bool DoesShaderModelRequireDXC(ED3DShaderModel ShaderModel)
{
	return ShaderModel >= ED3DShaderModel::SM6_0;
}

inline uint32 GetAutoBindingSpace(EShaderFrequency ShaderFrequency)
{
	switch (ShaderFrequency)
	{
	case SF_RayGen:
		return UE_HLSL_SPACE_RAY_TRACING_GLOBAL;
	case SF_RayMiss:
	case SF_RayHitGroup:
	case SF_RayCallable:
		return UE_HLSL_SPACE_RAY_TRACING_LOCAL;
	case SF_WorkGraphRoot:
		return UE_HLSL_SPACE_WORK_GRAPH_GLOBAL;
	case SF_WorkGraphComputeNode:
		return UE_HLSL_SPACE_WORK_GRAPH_LOCAL;
	default:
		return UE_HLSL_SPACE_DEFAULT;
	}
}

inline uint32 GetAutoBindingSpace(const FShaderTarget& Target)
{
	return GetAutoBindingSpace(Target.GetFrequency());
}

bool PreprocessD3DShader(
	const FShaderCompilerInput& Input,
	const FShaderCompilerEnvironment& MergedEnvironment,
	FShaderPreprocessOutput& PreprocessOutput);

void CompileD3DShader(
	const FShaderCompilerInput& Input,
	const FShaderPreprocessOutput& InPreprocessOutput,
	FShaderCompilerOutput& Output,
	ED3DShaderModel ShaderModel);

/**
 * @param bSecondPassAferUnusedInputRemoval whether we're compiling the shader second time, after having removed the unused inputs discovered in the first pass
 */
bool CompileAndProcessD3DShaderFXC(
	const FShaderCompilerInput& Input,
	const FString& InPreprocessedSource,
	const FString& InEntryPointName,
	const FShaderParameterParser& ShaderParameterParser,
	const TCHAR* ShaderProfile,
	bool bSecondPassAferUnusedInputRemoval,
	FShaderCompilerOutput& Output);

bool CompileAndProcessD3DShaderDXC(
	const FShaderCompilerInput& Input,
	const FString& InPreprocessedSource,
	const FString& InEntryPointName,
	const FShaderParameterParser& ShaderParameterParser,
	const TCHAR* ShaderProfile,
	ED3DShaderModel ShaderModel,
	bool bProcessingSecondTime,
	FShaderCompilerOutput& Output);

struct FD3DShaderCompileData;
bool ValidateResourceCounts(const FD3DShaderCompileData& CompiledData, TArray<FShaderCompilerError>& OutFilteredErrors);

template <typename TBlobType>
TConstArrayView<uint8> MakeArrayViewFromBlob(const TRefCountPtr<TBlobType> Blob)
{
	return MakeArrayView<const uint8>(reinterpret_cast<const uint8*>(Blob->GetBufferPointer()), Blob->GetBufferSize());
}

struct FD3DShaderDebugData
{
	struct FFile
	{
		FString Name;
		TArray<uint8> Contents;

		inline friend FArchive& operator<<(FArchive& Ar, FFile& DebugData)
		{
			Ar << DebugData.Name;
			Ar << DebugData.Contents;
			return Ar;
		}

		inline FString GetFilename() const
		{
			return Name;
		}

		inline TConstArrayView<uint8> GetContents() const
		{
			return TConstArrayView<uint8>(Contents);
		}
	};

	TArray<FFile> Files;

	inline friend FArchive& operator<<(FArchive& Ar, FD3DShaderDebugData& DebugData)
	{
		Ar << DebugData.Files;
		return Ar;
	}

	TConstArrayView<FFile> GetAllSymbolData() const
	{
		return TConstArrayView<FFile>(Files);
	}
};