// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class ShaderFormatD3D : ModuleRules
{
	public ShaderFormatD3D(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"D3D11RHI",
				"D3D12RHI",
				"Shaders",
				"TargetPlatform",
			});

        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"RenderCore",
				"ShaderPreprocessor",
				"ShaderCompilerCommon",
				"HlslParser",
				}
			);

		// DXC
        if (Target.Platform == UnrealTargetPlatform.Win64)
        {
			string AmdAgsPath = Target.UEThirdPartySourceDirectory + "AMD/AMD_AGS/";
			PublicSystemIncludePaths.Add(AmdAgsPath + "inc/");  // For amd_ags.h, to get AGS_DX12_SHADER_INSTRINSICS_SPACE_ID

			RuntimeDependencies.Add(Path.Combine(Target.WindowsPlatform.DirectXDllDir, "d3dcompiler_47.dll"));

			string ShaderConductorArchDir = Target.Architecture == UnrealArch.Arm64 ? "WinArm64" : "Win64";
			string ShaderConductorDllsPath = Path.Combine(Target.UEThirdPartyBinariesDirectory, "ShaderConductor", ShaderConductorArchDir);
			RuntimeDependencies.Add(Path.Combine(ShaderConductorDllsPath, "dxcompiler.dll"));
			RuntimeDependencies.Add(Path.Combine(ShaderConductorDllsPath, "ShaderConductor.dll"));
			RuntimeDependencies.Add(Path.Combine(ShaderConductorDllsPath, "dxil.dll"));

			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"ShaderConductor"
			);
        }

		AddEngineThirdPartyPrivateStaticDependencies(Target, "AgilitySDK");

		// DXC API headers
		PublicSystemIncludePaths.Add(Path.Combine(Target.UEThirdPartySourceDirectory, "ShaderConductor", "ShaderConductor", "External", "DirectXShaderCompiler", "include"));
	}
}
