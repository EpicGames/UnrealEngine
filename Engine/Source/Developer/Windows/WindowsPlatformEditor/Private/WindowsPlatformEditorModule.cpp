// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "WindowsTargetSettings.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/Class.h"

#define LOCTEXT_NAMESPACE "WindowsPlatformEditorModule"


/**
 * Module for Windows project settings
 */
class FWindowsPlatformEditorModule
	: public IModuleInterface
{
	// IModuleInterface interface

	virtual void StartupModule() override
	{
		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Platforms", "Windows",
				LOCTEXT("TargetSettingsName", "Windows"),
				LOCTEXT("TargetSettingsDescription", "Settings for Windows target platform"),
				GetMutableDefault<UWindowsTargetSettings>()
			);
		}
	}

	virtual void ShutdownModule() override
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Platforms", "Windows");
		}
	}
};


IMPLEMENT_MODULE(FWindowsPlatformEditorModule, WindowsPlatformEditor);

#undef LOCTEXT_NAMESPACE
