// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WindowsPlatformEditor : ModuleRules
{
	public WindowsPlatformEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				
				"WindowsTargetPlatformSettings",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Settings",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"GameProjectGeneration",
				}
		);
	}
}
