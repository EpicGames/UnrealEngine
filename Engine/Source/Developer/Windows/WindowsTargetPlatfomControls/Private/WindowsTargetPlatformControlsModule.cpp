// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ISettingsModule.h"
#include "Interfaces/ITargetPlatformControlsModule.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "GenericWindowsTargetPlatformControls.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"
#if WITH_ENGINE
#include "CookedEditorTargetPlatformControls.h"
#endif
#include "IWindowsTargetPlatformSettingsModule.h"

#define LOCTEXT_NAMESPACE "FWindowsTargetPlatformControlsModule"

/**
 * Implements the Windows target platform module.
 */
class FWindowsTargetPlatformControlsModule
	: public ITargetPlatformControlsModule
{
public:

	/** Destructor. */
	~FWindowsTargetPlatformControlsModule( )
	{

	}

public:
	virtual void GetTargetPlatformControls(TArray<ITargetPlatformControls*>& TargetPlatforms, FName& PlatformSettingsModuleName)
	{
		TMap<FString, ITargetPlatformSettings*> PlatformNameToPlatformSettings;
		IWindowsTargetPlatformSettingsModule* ModuleSettings = FModuleManager::GetModulePtr<IWindowsTargetPlatformSettingsModule>(PlatformSettingsModuleName);
		if (ModuleSettings != nullptr)
		{
			TMap<FString, ITargetPlatformSettings*> OutMap;
			ModuleSettings->GetPlatformSettingsMaps(OutMap);
			ITargetPlatformControls* GameTP = new TGenericWindowsTargetPlatformControls<FWindowsPlatformProperties<false, false, false>>(OutMap[FWindowsPlatformProperties<false, false, false>::PlatformName()]);
			ITargetPlatformControls* EditorTP = new TGenericWindowsTargetPlatformControls<FWindowsPlatformProperties<true, false, false>>(OutMap[FWindowsPlatformProperties<true, false, false>::PlatformName()]);
			ITargetPlatformControls* ServerTP = new TGenericWindowsTargetPlatformControls<FWindowsPlatformProperties<false, true, false>>(OutMap[FWindowsPlatformProperties<false, true, false>::PlatformName()]);
			ITargetPlatformControls* ClientTP = new TGenericWindowsTargetPlatformControls<FWindowsPlatformProperties<false, false, true>>(OutMap[FWindowsPlatformProperties<false, false, true>::PlatformName()]);
			TargetPlatforms.Add(GameTP);
			TargetPlatforms.Add(EditorTP);
			TargetPlatforms.Add(ServerTP);
			TargetPlatforms.Add(ClientTP);
#if WITH_ENGINE
			// currently this TP requires the engine for allowing GameDelegates usage
			bool bSupportCookedEditor;
			if (GConfig->GetBool(TEXT("CookedEditorSettings"), TEXT("bSupportCookedEditor"), bSupportCookedEditor, GGameIni) && bSupportCookedEditor)
			{
				ITargetPlatformControls* CookedEditorTP = new TCookedEditorTargetPlatformControls<FWindowsEditorTargetPlatformControlsParent>(ModuleSettings->GetCookedEditorPlatformSettings());
				ITargetPlatformControls* CookedCookerTP = new TCookedCookerTargetPlatformControls<FWindowsEditorTargetPlatformControlsParent>(ModuleSettings->GetCookedCookerPlatformSettings());
				TargetPlatforms.Add(CookedEditorTP);
				TargetPlatforms.Add(CookedCookerTP);
			}
#endif
		}
	}

public:
};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE(FWindowsTargetPlatformControlsModule, WindowsTargetPlatformControls);
