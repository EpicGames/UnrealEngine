// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports

#include "CoreMinimal.h"
#include "Common/TargetPlatformControlsBase.h"
#include "AnalyticsEventAttribute.h"
#if PLATFORM_WINDOWS
#include "LocalPcTargetDevice.h"
#endif
#if WITH_ENGINE
#include "TextureResource.h"
#endif
#include "SteamDeck/SteamDeckDevice.h"
#include "GenericWindowsTargetPlatformSettings.h"
#include "RemoteWindowsDevice.h"

// IWYU pragma: end_exports

#define LOCTEXT_NAMESPACE "TGenericWindowsTargetPlatformControls"

/**
 * Template for Windows target platforms controls
 */
#if PLATFORM_WINDOWS
template<typename TProperties, typename TTargetDevice = FLocalPcTargetDevice>
#else
template<typename TProperties>
#endif

class TGenericWindowsTargetPlatformControls : public TTargetPlatformControlsBase<TProperties>
{
public:
	typedef TTargetPlatformControlsBase<TProperties> TSuper;
	typedef TTargetPlatformSettingsBase<TProperties> TSuperSettings;

	/**
	O * Default constructor.
	 */
	TGenericWindowsTargetPlatformControls(ITargetPlatformSettings* TargetPlatformSettings)
		: TSuper(TargetPlatformSettings)
	{
#if PLATFORM_WINDOWS
		// only add local device if actually running on Windows
		LocalDevice = MakeShareable(new TTargetDevice(*this));

		// quick solution to not having WinGDK devices
		if (this->PlatformName().StartsWith(TEXT("Windows")))
		{
			// Check if we have any SteamDeck or other remote devices around
			SteamDevices = TSteamDeckDevice<FLocalPcTargetDevice>::DiscoverDevices(*this, TEXT("Proton"));
			RemoteDevices = FRemoteWindowsDevice::DiscoverDevices(*this);
		}
#endif
		GenericWindowsTargetPlatformSettings = (TSuperSettings*)TargetPlatformSettings;
	}

public:

	//~ Begin ITargetPlatform Interface

	virtual void EnableDeviceCheck(bool OnOff) override {}

	virtual void GetAllDevices( TArray<ITargetDevicePtr>& OutDevices ) const override
	{
		OutDevices.Reset();
		if (LocalDevice.IsValid())
		{
			OutDevices.Add(LocalDevice);
		}

		for (const ITargetDevicePtr& SteamDeck : SteamDevices)
		{
			if (SteamDeck.IsValid())
			{
				OutDevices.Add(SteamDeck);
			}
		}

		for (const ITargetDevicePtr& RemoteDevice : RemoteDevices)
		{
			if (RemoteDevice.IsValid())
			{
				OutDevices.Add(RemoteDevice);
			}
		}
	}

	virtual bool GenerateStreamingInstallManifest(const TMultiMap<FString, int32>& PakchunkMap, const TSet<int32>& PakchunkIndicesInUse) const override
	{
		return true;
	}

	virtual ITargetDevicePtr GetDefaultDevice( ) const override
	{
		if (LocalDevice.IsValid())
		{
			return LocalDevice;
		}

		return nullptr;
	}

	virtual ITargetDevicePtr GetDevice( const FTargetDeviceId& DeviceId )
	{
		if (LocalDevice.IsValid() && (DeviceId == LocalDevice->GetId()))
		{
			return LocalDevice;
		}

		for (const ITargetDevicePtr& SteamDeck : SteamDevices)
		{
			if (SteamDeck.IsValid() && DeviceId == SteamDeck->GetId())
			{
				return SteamDeck;
			}
		}

		for (const ITargetDevicePtr& RemoteDevice : RemoteDevices)
		{
			if (RemoteDevice.IsValid() && DeviceId == RemoteDevice->GetId())
			{
				return RemoteDevice;
			}
		}

		return nullptr;
	}

	virtual void GetShaderCompilerDependencies(TArray<FString>& OutDependencies) const override
	{
#if  PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
		FTargetPlatformControlsBase::AddDependencySCArrayHelper(OutDependencies, TEXT("Binaries/ThirdParty/Windows/DirectX/arm64/d3dcompiler_47.dll"));
		FTargetPlatformControlsBase::AddDependencySCArrayHelper(OutDependencies, TEXT("Binaries/ThirdParty/ShaderConductor/WinArm64/ShaderConductor.dll"));
		FTargetPlatformControlsBase::AddDependencySCArrayHelper(OutDependencies, TEXT("Binaries/ThirdParty/ShaderConductor/WinArm64/dxcompiler.dll"));
		FTargetPlatformControlsBase::AddDependencySCArrayHelper(OutDependencies, TEXT("Binaries/ThirdParty/ShaderConductor/WinArm64/dxil.dll"));
#else
		FTargetPlatformControlsBase::AddDependencySCArrayHelper(OutDependencies, TEXT("Binaries/ThirdParty/Windows/DirectX/x64/d3dcompiler_47.dll"));
		FTargetPlatformControlsBase::AddDependencySCArrayHelper(OutDependencies, TEXT("Binaries/ThirdParty/ShaderConductor/Win64/ShaderConductor.dll"));
		FTargetPlatformControlsBase::AddDependencySCArrayHelper(OutDependencies, TEXT("Binaries/ThirdParty/ShaderConductor/Win64/dxcompiler.dll"));
		FTargetPlatformControlsBase::AddDependencySCArrayHelper(OutDependencies, TEXT("Binaries/ThirdParty/ShaderConductor/Win64/dxil.dll"));
#endif
	}

	virtual void GetBuildProjectSettingKeys(FString& OutSection, TArray<FString>& InBoolKeys, TArray<FString>& InIntKeys, TArray<FString>& InStringKeys) const override
	{
		OutSection = TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings");
		InStringKeys.Add(TEXT("MinimumOSVersion"));
	}


	virtual void GetPlatformSpecificProjectAnalytics( TArray<FAnalyticsEventAttribute>& AnalyticsParamArray ) const override
	{
		TSuper::GetPlatformSpecificProjectAnalytics(AnalyticsParamArray);

		AppendAnalyticsEventAttributeArray(AnalyticsParamArray,
			TEXT("UsesRayTracing"), GenericWindowsTargetPlatformSettings->UsesRayTracing()
		);
	
		TSuper::AppendAnalyticsEventConfigString(AnalyticsParamArray, TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("DefaultGraphicsRHI"), GEngineIni);

		TSuper::AppendAnalyticsEventConfigArray(AnalyticsParamArray, TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("D3D12TargetedShaderFormats"), GEngineIni);
		TSuper::AppendAnalyticsEventConfigArray(AnalyticsParamArray, TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("D3D11TargetedShaderFormats"), GEngineIni);
		TSuper::AppendAnalyticsEventConfigArray(AnalyticsParamArray, TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("VulkanTargetedShaderFormats"), GEngineIni);
		TSuper::AppendAnalyticsEventConfigArray(AnalyticsParamArray, TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("TargetedRHIs"), GEngineIni, TEXT("TargetedRHIs_Deprecated") );
	}

#if WITH_ENGINE

	virtual void GetTextureFormats( const UTexture* InTexture, TArray< TArray<FName> >& OutFormats) const override
	{
		if (!TProperties::IsServerOnly())
		{
			GetDefaultTextureFormatNamePerLayer(OutFormats.AddDefaulted_GetRef(), GenericWindowsTargetPlatformSettings, this, InTexture, true, 4, true);
		}
	}

#endif //WITH_ENGINE

	virtual bool SupportsVariants() const override
	{
		return true;
	}

	virtual float GetVariantPriority() const override
	{
		return TProperties::GetVariantPriority();
	}
	//~ End ITargetPlatform Interface

private:

	// Holds the local device.
	ITargetDevicePtr LocalDevice;

	TArray<ITargetDevicePtr> SteamDevices;
	TArray<ITargetDevicePtr> RemoteDevices;

	TSuperSettings* GenericWindowsTargetPlatformSettings;
};

#undef LOCTEXT_NAMESPACE
