// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/ITargetDevice.h"
#include "Interfaces/ITargetPlatformControls.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <TlHelp32.h>

/**
 * Template for local PC target devices.
 *
 */
class FLocalPcTargetDevice
	: public ITargetDevice
{
public:

	/**
	 * Creates and initializes a new device for the specified target platform.
	 *
	 * @param InTargetPlatform - The target platform.
	 */
	FLocalPcTargetDevice( const ITargetPlatformControls& InTargetPlatform )
		: TargetPlatformControls(InTargetPlatform)
	{ }

public:

	// ITargetDevice interface

	virtual bool Connect( ) override
	{
		return true;
	}

	virtual void Disconnect( )
	{ }

	virtual ETargetDeviceTypes GetDeviceType( ) const override
	{
		if (::GetSystemMetrics(SM_TABLETPC) != 0)
		{
			return ETargetDeviceTypes::Tablet;
		}

		return ETargetDeviceTypes::Desktop;
	}

	virtual FTargetDeviceId GetId( ) const override
	{
		return FTargetDeviceId(TargetPlatformControls.PlatformName(), GetName());
	}

	virtual FString GetName( ) const override
	{
		return FPlatformProcess::ComputerName();
	}

	virtual FString GetOperatingSystemName( ) override
	{
		return TEXT("Windows (64-bit)");
	}

	virtual int32 GetProcessSnapshot( TArray<FTargetDeviceProcessInfo>& OutProcessInfos ) override
	{
		// enumerate processes
		HANDLE ProcessSnapshot = ::CreateToolhelp32Snapshot(TH32CS_SNAPPROCESS, 0);

		if (ProcessSnapshot != INVALID_HANDLE_VALUE)
		{
			PROCESSENTRY32 ProcessEntry;
			ProcessEntry.dwSize = sizeof(PROCESSENTRY32);

			if (::Process32First(ProcessSnapshot, &ProcessEntry))
			{
				do 
				{
					// only include processes that the user has permission to query
					HANDLE ProcessHandle = ::OpenProcess(PROCESS_QUERY_INFORMATION, false, ProcessEntry.th32ProcessID);

					if (ProcessHandle != NULL)
					{
						FTargetDeviceProcessInfo ProcessInfo;

						// get process details
						ProcessInfo.Id = ProcessEntry.th32ProcessID;
						ProcessInfo.Name = ProcessEntry.szExeFile;
						ProcessInfo.ParentId = ProcessEntry.th32ParentProcessID;
						ProcessInfo.UserName = TEXT("-");

						// get process user name
						HANDLE TokenHandle;

						if (::OpenProcessToken(ProcessHandle, TOKEN_QUERY, &TokenHandle))
						{
							DWORD UserTokenSize;

							::GetTokenInformation(TokenHandle, TokenUser, NULL, 0, &UserTokenSize);

							if (::GetLastError() == ERROR_INSUFFICIENT_BUFFER)
							{
								PTOKEN_USER UserToken = reinterpret_cast<PTOKEN_USER>(new BYTE[UserTokenSize]);

								if (UserToken != NULL)
								{
									if (::GetTokenInformation(TokenHandle, TokenUser, UserToken, UserTokenSize, &UserTokenSize))
									{
										WCHAR DomainName[256];
										DWORD DomainNameLength = 256;

										WCHAR UserName[256];
										DWORD UserNameLength = 256;

										SID_NAME_USE SidType;

										if (::LookupAccountSid(NULL, UserToken->User.Sid, UserName, &UserNameLength, DomainName, &DomainNameLength, &SidType))
										{
											ProcessInfo.UserName = UserName;
										}
										else
										{
											ProcessInfo.UserName = TEXT("SYSTEM");
										}
									}
								}
								else
								{
									DWORD LastError = ::GetLastError();
								}
							}

							::CloseHandle(TokenHandle);
						}

						::CloseHandle(ProcessHandle);

						OutProcessInfos.Add(ProcessInfo);
					}
				}
				while (::Process32Next(ProcessSnapshot, &ProcessEntry));
			}

			::CloseHandle(ProcessSnapshot);
		}

		// get thread details
		HANDLE ThreadSnapshot = ::CreateToolhelp32Snapshot(TH32CS_SNAPTHREAD, 0);

		if (ThreadSnapshot != INVALID_HANDLE_VALUE)
		{
			THREADENTRY32 ThreadEntry;
			ThreadEntry.dwSize = sizeof(THREADENTRY32);

			if (::Thread32First(ThreadSnapshot, &ThreadEntry))
			{
				do 
				{
					for (int32 ProcessInfoIndex = 0; ProcessInfoIndex < OutProcessInfos.Num(); ++ProcessInfoIndex)
					{
						FTargetDeviceProcessInfo& ProcessInfo = OutProcessInfos[ProcessInfoIndex];

						if (ProcessInfo.Id == ThreadEntry.th32OwnerProcessID)
						{
							FTargetDeviceThreadInfo ThreadInfo;

							ThreadInfo.ExitCode = 0;
							ThreadInfo.Id = ThreadEntry.th32ThreadID;
							ThreadInfo.StackSize = 0;
							ThreadInfo.State = ETargetDeviceThreadStates::Unknown;
							ThreadInfo.WaitState = ETargetDeviceThreadWaitStates::Unknown;

							ProcessInfo.Threads.Add(ThreadInfo);

							break;
						}
					}
				}
				while (::Thread32Next(ThreadSnapshot, &ThreadEntry));

				::CloseHandle(ThreadSnapshot);
			}
		}

		return OutProcessInfos.Num();
	}

	virtual const class ITargetPlatformSettings& GetPlatformSettings() const override
	{
		return *(TargetPlatformControls.GetTargetPlatformSettings());
	}
	virtual const class ITargetPlatformControls& GetPlatformControls() const override
	{
		return TargetPlatformControls;
	}

	virtual bool IsConnected( )
	{
		return true;
	}

	virtual bool IsDefault( ) const override
	{
		return true;
	}

	virtual bool PowerOff( bool Force ) override
	{
		if (!AdjustShutdownPrivileges())
		{
			return false;
		}

		CA_SUPPRESS(28159);
		return (::ExitWindowsEx(EWX_POWEROFF | EWX_FORCE, SHTDN_REASON_MINOR_MAINTENANCE | SHTDN_REASON_FLAG_PLANNED) != 0);
	}

	virtual bool PowerOn( ) override
	{
		return false;
	}

	virtual bool Reboot( bool bReconnect = false ) override
	{
		if (!AdjustShutdownPrivileges())
		{
			return false;
		}

		CA_SUPPRESS(28159);
		return (::ExitWindowsEx(EWX_REBOOT | EWX_FORCE, SHTDN_REASON_MINOR_MAINTENANCE | SHTDN_REASON_FLAG_PLANNED) != 0);
	}

	virtual bool SupportsFeature( ETargetDeviceFeatures Feature ) const override
	{
		switch (Feature)
		{
		case ETargetDeviceFeatures::MultiLaunch:
			return true;

		case ETargetDeviceFeatures::PowerOff:
			return true;

		// @todo gmp: implement turning on remote PCs (wake on LAN)
		case ETargetDeviceFeatures::PowerOn:
			return false;

		case ETargetDeviceFeatures::ProcessSnapshot:
			return true;

		case ETargetDeviceFeatures::Reboot:
			return true;
		}

		return false;
	}

	virtual void SetUserCredentials( const FString& UserName, const FString& UserPassword ) override
	{
	}

	virtual bool GetUserCredentials( FString& OutUserName, FString& OutUserPassword ) override
	{
		return false;
	}

	virtual bool TerminateProcess( const int64 ProcessId ) override
	{
		HANDLE ProcessHandle = OpenProcess(PROCESS_TERMINATE, false, (DWORD)ProcessId );

		if (ProcessHandle != NULL)
		{
			bool Result = (::TerminateProcess(ProcessHandle, 0) != 0);

			CloseHandle(ProcessHandle);

			return Result;
		}

		return false;
	}

protected:

	/**
	 * Adjusts shutdown privileges for the local host PC.
	 *
	 * @return true on success, false otherwise.
	 */
	bool AdjustShutdownPrivileges( )
	{
		HANDLE TokenHandle;
		TOKEN_PRIVILEGES Token;

		if (!::OpenProcessToken(GetCurrentProcess(), TOKEN_ADJUST_PRIVILEGES | TOKEN_QUERY, &TokenHandle))
		{
			return false;
		}

		::LookupPrivilegeValue(NULL, SE_SHUTDOWN_NAME, &Token.Privileges[0].Luid);

		if (Token.Privileges[0].Attributes == SE_PRIVILEGE_ENABLED)
		{
			return true;
		}

		Token.PrivilegeCount = 1;
		Token.Privileges[0].Attributes = SE_PRIVILEGE_ENABLED;

		::AdjustTokenPrivileges(TokenHandle, 0, &Token, 0, (PTOKEN_PRIVILEGES)NULL, 0);
      
		return (GetLastError() == ERROR_SUCCESS);
	}

protected:

	// Holds the collection of processes that were started using the Run() method.
	TMap<FGuid, void*> Processes;

	// Holds a reference to the device's target platform.
	const ITargetPlatformControls& TargetPlatformControls;
};

#include "Windows/HideWindowsPlatformTypes.h"
#undef PROCESSENTRY32
