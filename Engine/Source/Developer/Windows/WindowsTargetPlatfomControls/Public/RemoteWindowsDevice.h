// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_UNSUPPORTED - Common/TargetPlatformBase not part of additional include paths

#include "CoreMinimal.h"
#include "Common/TargetPlatformBase.h"
#include "Logging/LogMacros.h"
#include "Misc/Paths.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformMisc.h"

DEFINE_LOG_CATEGORY_STATIC(LogRemoteWindowsTargetDevice, Log, All);

/* Remote Windows device support currently requires the Microsoft.Gaming.RemoteIterationClient_Microsoft WinGet package. 
*  (The remote devices need the Microsoft.Gaming.RemoteIterationEndpoint_Microsoft WinGetpackage)
* 
* It is recommended to use the Xbox PC Toolbox installer to configure host and remote devices : https://aka.ms/ToolboxInstaller
* See https://aka.ms/GameRemoteDevtools for more details
* 
* At present, the list of devices is read from the engine ini. These devices should be paired already.
* 
*  %LOCALAPPDATA%\Unreal Engine\Engine\Config\UserEngine.ini  can be used for convenience
*  
*  [RemoteWin]
*  +DeviceNames=myfirstpc
*  +DeviceNames=mysecondpc
*/


class FRemoteWindowsDevice : public ITargetDevice
{
public:
	FRemoteWindowsDevice(const ITargetPlatformControls& InTargetPlatformControls, const FString& InDeviceName)
		: TargetPlatformControls(InTargetPlatformControls)
		, DeviceName(InDeviceName)
	{
	}

	virtual bool Connect() override
	{
		return true;
	}

	virtual void Disconnect() override
	{
	}

	virtual ETargetDeviceTypes GetDeviceType() const override
	{
		return ETargetDeviceTypes::Desktop;
	}

	virtual FTargetDeviceId GetId() const override
	{
		return FTargetDeviceId(TargetPlatformControls.PlatformName(), GetName());
	}

	virtual FString GetName() const override
	{
		return DeviceName;
	}

	virtual FString GetOperatingSystemName() override
	{
		// This may not always be 64-bit in the future but there's no way to query that yet.
		return TEXT("Windows (64-bit)");
	}

	virtual int32 GetProcessSnapshot(TArray<FTargetDeviceProcessInfo>& OutProcessInfos) override
	{
		return 0;
	}

	virtual const class ITargetPlatformSettings& GetPlatformSettings() const override
	{
		return *(TargetPlatformControls.GetTargetPlatformSettings());
	}

	virtual const class ITargetPlatformControls& GetPlatformControls() const override
	{
		return TargetPlatformControls;
	}

	virtual bool IsConnected() override
	{
		return true;
	}

	virtual bool IsDefault() const override
	{
		return true;
	}

	virtual bool PowerOff(bool Force) override
	{
		return false;
	}

	virtual bool PowerOn() override
	{
		return false;
	}

	virtual bool Reboot(bool bReconnect = false) override
	{
		return false;
	}

	virtual bool SupportsFeature(ETargetDeviceFeatures Feature) const override
	{
		return false;
	}

	virtual bool TerminateProcess(const int64 ProcessId) override
	{
		return RunWdRemote( FString::Printf(TEXT("/action:terminate /processid:%lld"), ProcessId));
	}

	virtual bool TerminateLaunchedProcess(const FString& ProcessIdentifier)
	{
		int64 ProcessId = FCString::Atoi64(*ProcessIdentifier);
		return (ProcessId > 0) && TerminateProcess(ProcessId);
	}

	virtual void SetUserCredentials(const FString& UserName, const FString& UserPassword) override {}

	virtual bool GetUserCredentials(FString& OutUserName, FString& OutUserPassword) override
	{
		return false;
	}

	static TArray<FString> GetAllRemoteDeviceNames()
	{
		TArray<FString> Result;

		// read device names from the engine ini. must already be paired
		// best place is %LOCALAPPDATA%\Unreal Engine\Engine\Config\UserEngine.ini
		TArray<FString> DeviceNames;
		GConfig->GetArray(TEXT("RemoteWin"), TEXT("DeviceNames"), DeviceNames, GEngineIni);
		Result.Append(DeviceNames);

		return MoveTemp(Result);
	}

	static TArray<ITargetDevicePtr> DiscoverDevices(const ITargetPlatformControls& TargetPlatformControls)
	{
		TArray<ITargetDevicePtr> Devices;
		for (const FString& DeviceName : GetAllRemoteDeviceNames())
		{
			Devices.Add(MakeShared<FRemoteWindowsDevice>(TargetPlatformControls, DeviceName));
		}

		return Devices;
	}

private:
	bool RunWdRemote( const FString& Parameters ) const
	{
		return RunWdRemote( Parameters, DeviceName );
	}

	static bool RunWdRemote( const FString& Parameters, const FString& DeviceName )
	{
		FString CommandLineArgs = Parameters;
		if (!DeviceName.IsEmpty())
		{
			CommandLineArgs += FString::Printf(TEXT(" /device:%s"), *DeviceName);
		}

		FString LocalAppData = FPlatformMisc::GetEnvironmentVariable(TEXT("LOCALAPPDATA"));
		FString WdRemotePath = FPaths::Combine(LocalAppData, TEXT("Microsoft\\WinGet\\Links\\wdRemote.exe") );
		if (!FPaths::FileExists(WdRemotePath))
		{
			UE_LOGF( LogRemoteWindowsTargetDevice, Error, "%ls not found!", *WdRemotePath);
			return false;
		}

		int32 ReturnCode = 0;
		FString StdOut;
		FString StdErr;
		FPlatformProcess::ExecProcess(*WdRemotePath, *CommandLineArgs, &ReturnCode, &StdOut, &StdErr);

		UE_CLOGF( (ReturnCode != 0), LogRemoteWindowsTargetDevice, Error, "wdremote %ls  - failed (0x%X)\n%ls\n%ls", *CommandLineArgs, ReturnCode, *StdOut, *StdErr);
		return (ReturnCode == 0);
	}

	const ITargetPlatformControls& TargetPlatformControls;
	FString DeviceName;
};
