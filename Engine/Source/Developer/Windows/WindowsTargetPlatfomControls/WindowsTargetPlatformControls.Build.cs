// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WindowsTargetPlatformControls : ModuleRules
{
	public WindowsTargetPlatformControls(ReadOnlyTargetRules Target) : base(Target)
	{
		SDKVersionRelevantPlatforms.Add(UnrealTargetPlatform.Win64);

		// We need a short name here since this can run afoul very easily of the `MAX_PATH` limit when
		// combined with building other targets that make use of this.
		ShortName = "WinTPCon";

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"TargetPlatform",
				"DesktopPlatform",
                "AudioPlatformConfiguration",
				"WindowsTargetPlatformSettings",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Settings",
				"WindowsTargetPlatformSettings"
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"AudioPlatformConfiguration"
			}
		);

		// compile with Engine
		if (Target.bCompileAgainstEngine)
		{
			PublicIncludePathModuleNames.Add("Engine");
			PrivateDependencyModuleNames.AddRange( new string[] {
				"Engine", 
				"RHI",
				"CookedEditor",
				}
			);
            PrivateIncludePathModuleNames.Add("TextureCompressor");
        }
    }
}
