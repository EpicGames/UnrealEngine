// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Modules/ModuleManager.h"
#include "Common/TargetPlatformBase.h"

#define LOCTEXT_NAMESPACE "FWindowsTargetPlatformModule"


/**
 * Implements the Windows target platform module.
 */
class FWindowsTargetPlatformModule
	: public ITargetPlatformModule
{
public:

	/** Destructor. */
	~FWindowsTargetPlatformModule( )
	{

	}

public:

	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms) override
	{
	}

	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms, TArray<ITargetPlatformSettings*> TargetPlatformSettings, TArray<ITargetPlatformControls*> TargetPlatformControls) override
	{
		for (ITargetPlatformControls* TargetPlatformControlsIt : TargetPlatformControls)
		{
			TargetPlatforms.Add(new FTargetPlatformMerged(TargetPlatformControlsIt->GetTargetPlatformSettings(), TargetPlatformControlsIt));
		}
	}
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FWindowsTargetPlatformModule, WindowsTargetPlatform);
