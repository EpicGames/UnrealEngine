// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WindowsTargetPlatform : ModuleRules
{
	public WindowsTargetPlatform(ReadOnlyTargetRules Target) : base(Target)
	{
		SDKVersionRelevantPlatforms.Add(UnrealTargetPlatform.Win64);
		
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"TargetPlatform",
				"DesktopPlatform",
				"WindowsTargetPlatformSettings",
				"WindowsTargetPlatformControls",
			}
		);
    }
}
