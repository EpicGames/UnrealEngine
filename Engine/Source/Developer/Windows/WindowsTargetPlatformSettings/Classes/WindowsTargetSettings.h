// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WindowsTargetSettings.h: Declares the UWindowsTargetSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "AudioCompressionSettings.h"
#include "Misc/TargetSettingsDefinitions.h"
#include "WindowsTargetSettings.generated.h"

#define UE_API WINDOWSTARGETPLATFORMSETTINGS_API

UENUM()
enum class ECompilerVersion : uint8
{
	Default = 0,
	VisualStudio2015 = 1 UMETA(DisplayName = "Visual Studio 2015 (deprecated)"),
	VisualStudio2017 = 2 UMETA(DisplayName = "Visual Studio 2017 (deprecated)"),
	VisualStudio2019 = 3 UMETA(DisplayName = "Visual Studio 2019 (deprecated)"),
	VisualStudio2022 = 4 UMETA(DisplayName = "Visual Studio 2022"),
	VisualStudio2026 = 5 UMETA(DisplayName = "Visual Studio 2026"),
	Clang = 6 UMETA(DisplayName = "Clang"),
	Intel = 7 UMETA(DisplayName = "Intel oneAPI"),
};

UENUM()
enum class EDefaultGraphicsRHI : uint8
{
	DefaultGraphicsRHI_Default = 0 UMETA(DisplayName = "Default"),
	DefaultGraphicsRHI_DX11 = 1 UMETA(DisplayName = "DirectX 11"),
	DefaultGraphicsRHI_DX12 = 2 UMETA(DisplayName = "DirectX 12"),
	DefaultGraphicsRHI_Vulkan = 3 UMETA(DisplayName = "Vulkan"),
};

/**
 * Implements the settings for the Windows target platform. The first instance of this class is initialized in
 * WindowsTargetPlatform, really early during the startup sequence before the CDO has been constructed, so its config 
 * settings are read manually from there.
 */
UCLASS(MinimalAPI, config=Engine, defaultconfig)
class UWindowsTargetSettings
	: public UObject
{
public:
	GENERATED_UCLASS_BODY()

	UE_API virtual void PostInitProperties() override;

	/** Select which RHI to use. Make sure its also selected as a Targeted RHI. Requires Editor restart. */
	UPROPERTY(EditAnywhere, config, Category="Targeted RHIs", Meta = (DisplayName = "Default RHI", ConfigRestartRequired = true))
	EDefaultGraphicsRHI DefaultGraphicsRHI;

	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "Use one of the RHI specific lists."))
	TArray<FString> TargetedRHIs_DEPRECATED;

	UPROPERTY(EditAnywhere, config, Category = "Rendering", Meta = (ConfigRestartRequired = true))
	TArray<FString> D3D12TargetedShaderFormats;

	UPROPERTY(EditAnywhere, config, Category = "Rendering", Meta = (ConfigRestartRequired = true))
	TArray<FString> D3D11TargetedShaderFormats;

	UPROPERTY(EditAnywhere, config, Category = "Rendering", Meta = (ConfigRestartRequired = true))
	TArray<FString> VulkanTargetedShaderFormats;

	/** Whether to include Nanite Fallback Meshes in cooked builds for Windows. Can be overriden for specific assets in Static Mesh Editor. */
	UPROPERTY(config, EditAnywhere, Category = "Renderer", meta = (DisplayName = "Generate Nanite Fallback Meshes", ConfigRestartRequired = true))
	bool bGenerateNaniteFallbackMeshes = true;

	/** Deprecated use RayTracingMode instead. */
	UPROPERTY(config, meta = (DeprecatedProperty))
	bool bEnableRayTracing_DEPRECATED;

	/** Controls the level of ray tracing support for Windows. r.RayTracing cvar also needs to be enabled. */
	UPROPERTY(config, EditAnywhere, Category = "Renderer", meta = (DisplayName = "Hardware Ray Tracing Mode"))
	ETargetSettingsRayTracingRuntimeMode RayTracingMode;
	
	/** The compiler version to use for this project. May be different to the chosen IDE. */
	UPROPERTY(EditAnywhere, config, Category = "Toolchain", Meta = (DisplayName = "Compiler Version"))
	ECompilerVersion Compiler;

	/** Sample rate to run the audio mixer with. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", Meta = (DisplayName = "Audio Mixer Sample Rate"))
	int32 AudioSampleRate;

	/** The amount of audio to compute each callback block. Lower values decrease latency but may increase CPU cost. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "64", ClampMax = "4096", DisplayName = "Callback Buffer Size", ConfigRestartRequired = true))
	int32 AudioCallbackBufferFrameSize;

	/** The number of buffers to keep enqueued. More buffers increases latency, but can compensate for variable compute availability in audio callbacks on some platforms. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "1", UIMin = "1", DisplayName = "Number of Buffers To Enqueue", ConfigRestartRequired = true))
	int32 AudioNumBuffersToEnqueue;

	/** The max number of channels (voices) to limit for this platform. The max channels used will be the minimum of this value and the global audio quality settings. A value of 0 will not apply a platform channel count max. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "0", UIMin = "0", DisplayName = "Max Channels"))
	int32 AudioMaxChannels;

	/** The number of workers to use to compute source audio. Will only use up to the max number of sources. Will evenly divide sources to each source worker. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "0", UIMin = "0", DisplayName = "Number of Source Workers"))
	int32 AudioNumSourceWorkers;

	/** Which of the currently enabled spatialization plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString SpatializationPlugin;

	/** Which of the currently enabled source data override plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString SourceDataOverridePlugin;

	/** Which of the currently enabled reverb plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString ReverbPlugin;

	/** Which of the currently enabled occlusion plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString OcclusionPlugin;

	/** Audio Cook Settings */

	/** Various overrides for how this platform should handle compression and decompression */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio")
	FPlatformRuntimeAudioCompressionOverrides CompressionOverrides;

	/** This determines the max amount of memory that should be used for the cache at any given time. If set low (<= 8 MB), it lowers the size of individual chunks of audio during cook. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides|Stream Caching", meta = (DisplayName = "Max Cache Size (KB)"))
	int32 CacheSizeKB;

	/** This overrides the default max chunk size used when chunking audio for stream caching (ignored if < 0) */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides|Stream Caching", meta = (DisplayName = "Max Chunk Size Override (KB)"))
	int32 MaxChunkSizeOverrideKB;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides")
	bool bResampleForDevice;

	/** Mapping of which sample rates are used for each sample rate quality for a specific platform. */

	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "Max"))
	float MaxSampleRate;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "High"))
	float HighSampleRate;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "Medium"))
	float MedSampleRate;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "Low"))
	float LowSampleRate;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "Min"))
	float MinSampleRate;

	/** Scales all compression qualities when cooking to this platform. For example, 0.5 will halve all compression qualities, and 1.0 will leave them unchanged. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides")
	float CompressionQualityModifier;

	/** When set to anything beyond 0, this will ensure any SoundWaves longer than this value, in seconds, to stream directly off of the disk. */
	UPROPERTY(GlobalConfig)
	float AutoStreamingThreshold;

	/** Quality Level to COOK SoundCues at (if set, all other levels will be stripped by the cooker). */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides", meta = (DisplayName = "Sound Cue Cook Quality"))
	int32 SoundCueCookQualityIndex = INDEX_NONE;

	void virtual OverrideConfigSection(FString& InOutSectionName) override
	{
		InOutSectionName = TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings");
	}

};

#undef UE_API
