// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsTargetSettings.h"
#include "Misc/ConfigCacheIni.h"

/* UWindowsTargetSettings structors
 *****************************************************************************/

UWindowsTargetSettings::UWindowsTargetSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	// Default values are defined in BaseEngine.ini
}

static bool FilterShaderPlatform_D3D12(const FString& InShaderPlatform)
{
	return InShaderPlatform == TEXT("PCD3D_SM6") || InShaderPlatform == TEXT("PCD3D_SM5") || InShaderPlatform == TEXT("PCD3D_ES31");
}

static bool FilterShaderPlatform_D3D11(const FString& InShaderPlatform)
{
	return InShaderPlatform == TEXT("PCD3D_SM5") || InShaderPlatform == TEXT("PCD3D_ES31");
}

static bool FilterShaderPlatform_Vulkan(const FString& InShaderPlatform)
{
	return InShaderPlatform == TEXT("SF_VULKAN_SM5") || InShaderPlatform == TEXT("SF_VULKAN_SM6");
}

template<typename TFilterFunc>
static void AddToShaderFormatList(TArray<FString>& Dest, const TArray<FString>& Source, TFilterFunc FilterFunc)
{
	for (const FString& ShaderFormat : Source)
	{
		if (FilterFunc(ShaderFormat))
		{
			Dest.AddUnique(ShaderFormat);
		}
	}
}

void UWindowsTargetSettings::PostInitProperties()
{
	Super::PostInitProperties();
	
	FConfigFile LocalEngineIni;
	FConfigFile* EngineIni = FConfigCacheIni::FindOrLoadPlatformConfig(LocalEngineIni, TEXT("Engine"), TEXT("Win64"));
	
	// Convert and delete deprecated properties
	// Deprecated properties are automatically deleted from .inis when updated via UpdateSinglePropertyInConfigFile()
	bool bOldEnableRayTracing = false;
	if (EngineIni->GetBool(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("bEnableRayTracing"), bOldEnableRayTracing))
	{
		RayTracingMode = bOldEnableRayTracing ? ETargetSettingsRayTracingRuntimeMode::Full : ETargetSettingsRayTracingRuntimeMode::Disabled;
		UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UWindowsTargetSettings, RayTracingMode)), GetDefaultConfigFilename());
		UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(FName(TEXT("bEnableRayTracing"))), GetDefaultConfigFilename());
	};

	if (!TargetedRHIs_DEPRECATED.IsEmpty())
	{
		AddToShaderFormatList(D3D12TargetedShaderFormats, TargetedRHIs_DEPRECATED, &FilterShaderPlatform_D3D12);
		AddToShaderFormatList(D3D11TargetedShaderFormats, TargetedRHIs_DEPRECATED, &FilterShaderPlatform_D3D11);
		AddToShaderFormatList(VulkanTargetedShaderFormats, TargetedRHIs_DEPRECATED, &FilterShaderPlatform_Vulkan);

		TargetedRHIs_DEPRECATED.Empty();
	}
}
