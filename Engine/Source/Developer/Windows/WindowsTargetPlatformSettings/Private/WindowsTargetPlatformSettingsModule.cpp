// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ISettingsModule.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "GenericWindowsTargetPlatformSettings.h"
#include "IWindowsTargetPlatformSettingsModule.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"
#if WITH_ENGINE
#include "CookedEditorTargetPlatformSettings.h"
#endif

#define LOCTEXT_NAMESPACE "FWindowsTargetPlatformSettingsModule"


/**
 * Implements the Windows target platform module.
 */
class FWindowsTargetPlatformSettingsModule
	: public IWindowsTargetPlatformSettingsModule
{
public:

	/** Destructor. */
	~FWindowsTargetPlatformSettingsModule( )
	{

	}

	virtual void GetPlatformSettingsMaps(TMap<FString, ITargetPlatformSettings*>& OutMap) override
	{
		OutMap = PlatformNameToPlatformSettings;
	}

	virtual ITargetPlatformSettings* GetCookedEditorPlatformSettings() override
	{
		return PlatformSettingsCookedEditor;
	}

	virtual ITargetPlatformSettings* GetCookedCookerPlatformSettings() override
	{
		return PlatformSettingsCookedCooker;
	}

public:

	// this is an example of a hotfix, declared here for no particular reason. Once we have other examples, it can be deleted.
#if 0
	void HotfixTest( void *InPayload, int PayloadSize )
	{
		check(sizeof(FTestHotFixPayload) == PayloadSize);
		
		FTestHotFixPayload* Payload = (FTestHotFixPayload*)InPayload;
		UE_LOGF(LogTemp, Log, "Hotfix Test %ls", *Payload->Message);
		Payload->Result = Payload->ValueToReturn;
	}
#endif

public:

	virtual void GetTargetPlatformSettings(TArray<ITargetPlatformSettings*>& TargetPlatforms)
	{
		// Game TP
		ITargetPlatformSettings* GameTP = new TGenericWindowsTargetPlatformSettings<FWindowsPlatformProperties<false, false, false>>();
		TargetPlatforms.Add(GameTP);
		PlatformNameToPlatformSettings.Add(FWindowsPlatformProperties<false, false, false>::PlatformName(), GameTP);
		// Editor TP
		ITargetPlatformSettings* EditorTP = new TGenericWindowsTargetPlatformSettings<FWindowsPlatformProperties<true, false, false>>();
		TargetPlatforms.Add(new TGenericWindowsTargetPlatformSettings<FWindowsPlatformProperties<true, false, false>>());
		PlatformNameToPlatformSettings.Add(FWindowsPlatformProperties<true, false, false>::PlatformName(), EditorTP);
		// Server TP
		ITargetPlatformSettings* ServerTP = new TGenericWindowsTargetPlatformSettings<FWindowsPlatformProperties<false, true, false>>();
		TargetPlatforms.Add(new TGenericWindowsTargetPlatformSettings<FWindowsPlatformProperties<false, true, false>>());
		PlatformNameToPlatformSettings.Add(FWindowsPlatformProperties<false, true, false>::PlatformName(), ServerTP);
		// Client TP
		ITargetPlatformSettings* ClientTP = new TGenericWindowsTargetPlatformSettings<FWindowsPlatformProperties<false, false, true>>();
		TargetPlatforms.Add(new TGenericWindowsTargetPlatformSettings<FWindowsPlatformProperties<false, false, true>>());
		PlatformNameToPlatformSettings.Add(FWindowsPlatformProperties<false, false, true>::PlatformName(), ClientTP);

#if WITH_ENGINE
		// currently this TP requires the engine for allowing GameDelegates usage
		bool bSupportCookedEditor;
		if (GConfig->GetBool(TEXT("CookedEditorSettings"), TEXT("bSupportCookedEditor"), bSupportCookedEditor, GGameIni) && bSupportCookedEditor)
		{
			ITargetPlatformSettings* CookedEditorTP = new TCookedEditorTargetPlatformSettings<FWindowsEditorTargetPlatformSettingsParent>();
			ITargetPlatformSettings* CookedCookerTP = new TCookedCookerTargetPlatformSettings<FWindowsEditorTargetPlatformSettingsParent>();
			PlatformSettingsCookedEditor = CookedEditorTP;
			PlatformSettingsCookedCooker = CookedCookerTP;
			TargetPlatforms.Add(CookedEditorTP);
			TargetPlatforms.Add(CookedCookerTP);
		}
#endif
	}

public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{
		// this is an example of a hotfix, declared here for no particular reason. Once we have other examples, it can be deleted.
#if 0
		FCoreDelegates::GetHotfixDelegate(EHotfixDelegates::Test).BindRaw(this, &FWindowsTargetPlatformSettingsModule::HotfixTest);
#endif
	}

	virtual void ShutdownModule() override
	{
		// this is an example of a hotfix, declared here for no particular reason. Once we have other examples, it can be deleted.
#if 0
		FCoreDelegates::GetHotfixDelegate(EHotfixDelegates::Test).Unbind();
#endif

		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	}

private:
	TMap<FString, ITargetPlatformSettings*> PlatformNameToPlatformSettings;
	ITargetPlatformSettings* PlatformSettingsCookedEditor;
	ITargetPlatformSettings* PlatformSettingsCookedCooker;
};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE(FWindowsTargetPlatformSettingsModule, WindowsTargetPlatformSettings);
