// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Common/TargetPlatformSettingsBase.h"
#include "Misc/ConfigCacheIni.h"
#if WITH_ENGINE
	#include "StaticMeshResources.h"
	#include "RHI.h"
	#include "DataDrivenShaderPlatformInfo.h"
#endif // WITH_ENGINE
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMisc.h"

#define LOCTEXT_NAMESPACE "TGenericWindowsTargetPlatformSettings"
#define WIN_SECTION_TEXT TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings")

/**
 * Template for Windows target platforms settings
 */

template<typename TProperties>
class TGenericWindowsTargetPlatformSettings : public TTargetPlatformSettingsBase<TProperties>
{
public:
	typedef TTargetPlatformSettingsBase<TProperties> TSuper;

	/**
	 * Default constructor.
	 */
	TGenericWindowsTargetPlatformSettings( )
	{

	#if WITH_ENGINE
		TextureLODSettings = nullptr; // These are registered by the device profile system.
		StaticMeshLODSettings.Initialize(this);


		// Get the Target RHIs for this platform, we do not always want all those that are supported.
		TArray<FName> TargetedShaderFormats;
		TGenericWindowsTargetPlatformSettings::GetAllTargetedShaderFormats(TargetedShaderFormats);

		static FName NAME_PCD3D_SM6(TEXT("PCD3D_SM6"));
		static FName NAME_PCD3D_SM5(TEXT("PCD3D_SM5"));
		static FName NAME_VULKAN_SM5(TEXT("SF_VULKAN_SM5"));
		static FName NAME_VULKAN_SM6(TEXT("SF_VULKAN_SM6"));
		static FName NAME_PCD3D_ES3_1(TEXT("PCD3D_ES31"));
		static FName NAME_SF_VULKAN_ES31(TEXT("SF_VULKAN_ES31"));
		static FName NAME_OPENGL_150_ES3_1(TEXT("GLSL_150_ES31"));

		// If we are targeting ES3.1, we also must cook encoded HDR reflection captures
		bRequiresEncodedHDRReflectionCaptures =	TargetedShaderFormats.Contains(NAME_SF_VULKAN_ES31)
												|| TargetedShaderFormats.Contains(NAME_OPENGL_150_ES3_1)
												|| TargetedShaderFormats.Contains(NAME_PCD3D_ES3_1);

	#endif
	}

public:

	//~ Begin ITargetPlatform Interface

	virtual bool SupportsFeature( ETargetPlatformFeatures Feature ) const override
	{
		// we currently do not have a build target for WindowsServer
		if (Feature == ETargetPlatformFeatures::Packaging)
		{
			return (TProperties::HasEditorOnlyData() || !TProperties::IsServerOnly());
		}

		if ( Feature == ETargetPlatformFeatures::ShouldSplitPaksIntoSmallerSizes )
		{
			return TProperties::IsClientOnly();
		}

		if (Feature == ETargetPlatformFeatures::CanCookPackages)
		{
			return TProperties::HasEditorOnlyData();
		}

		if (Feature == ETargetPlatformFeatures::MobileRendering)
		{
			static bool bCachedSupportsMobileRendering = false;
#if WITH_ENGINE
			static bool bHasCachedValue = false;
			if (!bHasCachedValue)
			{
				TArray<FName> TargetedShaderFormats;
				GetAllTargetedShaderFormats(TargetedShaderFormats);

				for (const FName& Format : TargetedShaderFormats)
				{
					if (IsMobilePlatform(ShaderFormatToLegacyShaderPlatform(Format)))
					{
						bCachedSupportsMobileRendering = true;
						break;
					}
				}
				bHasCachedValue = true;
			}
#endif

			return bCachedSupportsMobileRendering;
		}

		return TSuper::SupportsFeature(Feature);
	}

	virtual bool IsRunningPlatform() const override
	{
		// Must be Windows platform as editor for this to be considered a running platform
		return PLATFORM_WINDOWS && !UE_SERVER && !UE_GAME && WITH_EDITOR && TProperties::HasEditorOnlyData();
	}

	virtual void GetAllPossibleShaderFormats( TArray<FName>& OutFormats ) const override
	{
		// no shaders needed for dedicated server target
		if (!TProperties::IsServerOnly())
		{
			static FName NAME_PCD3D_SM6(TEXT("PCD3D_SM6"));
			static FName NAME_PCD3D_SM5(TEXT("PCD3D_SM5"));
			static FName NAME_VULKAN_ES31(TEXT("SF_VULKAN_ES31"));
			static FName NAME_OPENGL_150_ES3_1(TEXT("GLSL_150_ES31"));
			static FName NAME_VULKAN_SM5(TEXT("SF_VULKAN_SM5"));
			static FName NAME_VULKAN_SM6(TEXT("SF_VULKAN_SM6"));
			static FName NAME_PCD3D_ES3_1(TEXT("PCD3D_ES31"));

			OutFormats.AddUnique(NAME_PCD3D_SM5);
			OutFormats.AddUnique(NAME_PCD3D_SM6);
			OutFormats.AddUnique(NAME_VULKAN_ES31);
			OutFormats.AddUnique(NAME_OPENGL_150_ES3_1);
			OutFormats.AddUnique(NAME_VULKAN_SM5);
			OutFormats.AddUnique(NAME_VULKAN_SM6);
			OutFormats.AddUnique(NAME_PCD3D_ES3_1);
		}
	}

	TArray<FName> GetAllTargetedShaderFormatsInternal(TConstArrayView<const TCHAR*> RelevantSettings) const
	{
		// no shaders needed for dedicated server target
		if (TProperties::IsServerOnly())
		{
			return TArray<FName>();
		}

		FConfigCacheIni* ConfigSystem = this->GetConfigSystem();

		TArray<FString> TargetedShaderFormats;

		for (const TCHAR* Name : RelevantSettings)
		{
			TArray<FString> NewTargetedShaderFormats;
			ConfigSystem->GetArray(WIN_SECTION_TEXT, Name, NewTargetedShaderFormats, GEngineIni);

			for (const FString& NewShaderFormat : NewTargetedShaderFormats)
			{
				TargetedShaderFormats.AddUnique(NewShaderFormat);
			}
		}
		
		// Gather the list of Target RHIs and filter out any that may be invalid.
		TArray<FName> PossibleShaderFormats;
		GetAllPossibleShaderFormats(PossibleShaderFormats);

		TArray<FName> OutFormats;
		for (const FString& ShaderFormat : TargetedShaderFormats)
		{
			const FName ShaderFormatName(*ShaderFormat, FNAME_Find);

			if (!ShaderFormatName.IsNone() && PossibleShaderFormats.Contains(ShaderFormatName))
			{
				OutFormats.Emplace(ShaderFormatName);
			}
		}

		return OutFormats;
	}

	virtual void GetAllTargetedShaderFormats( TArray<FName>& OutFormats ) const override 
	{
		// Get the Target RHIs for this platform, we do not always want all those that are supported. (reload in case user changed in the editor)

		const TCHAR* const RelevantSettings[] = 
		{
			TEXT("TargetedRHIs"),
			TEXT("D3D12TargetedShaderFormats"),
			TEXT("D3D11TargetedShaderFormats"),
			TEXT("VulkanTargetedShaderFormats")
		};

		static const TArray<FName> CachedFormats = GetAllTargetedShaderFormatsInternal(RelevantSettings);
		OutFormats = CachedFormats;
	}

	virtual void GetRayTracingShaderFormats( TArray<FName>& OutFormats ) const override 
	{
		if (UsesRayTracing())
		{
			const TCHAR* const RelevantSettings[] = 
			{
				TEXT("VulkanTargetedShaderFormats")
			};

			static const TArray<FName> CachedFormats = GetAllTargetedShaderFormatsInternal(RelevantSettings);
			OutFormats = CachedFormats;

			// We always support ray tracing shaders when cooking for D3D12 SM6, however we may skip them for SM5 based on project settings.
			static FName NAME_PCD3D_SM6(TEXT("PCD3D_SM6"));
			OutFormats.Emplace(NAME_PCD3D_SM6);
		}
	}

	virtual void GetPossibleArchitectures(TArray<FString>& OutArchitectures) const override
	{
		// @todo: add support for ini-driven 'project supported architectures' to filter this list
		OutArchitectures.Add(TEXT("x64"));
		OutArchitectures.Add(TEXT("arm64"));
		OutArchitectures.Add(TEXT("arm64ec"));
	}

	virtual FString GetHostArchitecture() const override
	{
		return FPlatformMisc::GetHostArchitecture();
	}


#if WITH_ENGINE
	virtual void GetReflectionCaptureFormats(TArray<FName>& OutFormats) const override
	{
		if (bRequiresEncodedHDRReflectionCaptures)
		{
			OutFormats.Add(FName(TEXT("EncodedHDR")));
		}

		OutFormats.Add(FName(TEXT("FullHDR")));
	}

	virtual void GetShaderFormatModuleHints(TArray<FName>& OutModuleNames) const override
	{
		OutModuleNames.Add(TEXT("ShaderFormatD3D"));
		OutModuleNames.Add(TEXT("ShaderFormatOpenGL"));
		OutModuleNames.Add(TEXT("VulkanShaderFormat"));
	}

	virtual const class FStaticMeshLODSettings& GetStaticMeshLODSettings( ) const override
	{
		return StaticMeshLODSettings;
	}


	virtual const UTextureLODSettings& GetTextureLODSettings() const override
	{
		return *TextureLODSettings;
	}

	virtual void RegisterTextureLODSettings(const UTextureLODSettings* InTextureLODSettings) override
	{
		TextureLODSettings = InTextureLODSettings;
	}

#endif //WITH_ENGINE

	virtual bool ShouldStripNaniteFallbackMeshes() const override
	{
		bool bGenerateNaniteFallbackMeshes = true;
		GConfig->GetBool(WIN_SECTION_TEXT, TEXT("bGenerateNaniteFallbackMeshes"), bGenerateNaniteFallbackMeshes, GEngineIni);

		return !bGenerateNaniteFallbackMeshes;
	}

	virtual bool UsesDistanceFields() const override
	{
		bool bEnableDistanceFields = false;
		GConfig->GetBool(WIN_SECTION_TEXT, TEXT("bEnableDistanceFields"), bEnableDistanceFields, GEngineIni);

		return bEnableDistanceFields && TSuper::UsesDistanceFields();
	}

	virtual bool UsesRayTracing() const override
	{
		return GetRayTracingMode() != ERayTracingRuntimeMode::Disabled;
	}

	virtual ERayTracingRuntimeMode GetRayTracingMode() const override
	{
		if (!TTargetPlatformSettingsBase<TProperties>::UsesRayTracing())
		{
			return ERayTracingRuntimeMode::Disabled;
		}

		return TTargetPlatformSettingsBase<TProperties>::ParseRuntimeRayTracingMode(WIN_SECTION_TEXT);
	}

	//~ End ITargetPlatform Interface

private:

#if WITH_ENGINE
	// Holds the texture LOD settings.
	const UTextureLODSettings* TextureLODSettings;

	// Holds static mesh LOD settings.
	FStaticMeshLODSettings StaticMeshLODSettings;

	// True if the project requires encoded HDR reflection captures
	bool bRequiresEncodedHDRReflectionCaptures;

#endif // WITH_ENGINE

};

#undef LOCTEXT_NAMESPACE
