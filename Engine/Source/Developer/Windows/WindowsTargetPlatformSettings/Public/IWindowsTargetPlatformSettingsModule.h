// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once 

/*------------------------------------------------------------------------------------
IWindowsTargetPlatformSettingsModule interface
------------------------------------------------------------------------------------*/

#include "CoreMinimal.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "GenericWindowsTargetPlatformSettings.h"

class IWindowsTargetPlatformSettingsModule : public ITargetPlatformSettingsModule
{
public:
	virtual void GetPlatformSettingsMaps(TMap<FString, ITargetPlatformSettings*>& OutMap) = 0;
	virtual ITargetPlatformSettings* GetCookedEditorPlatformSettings() = 0;
	virtual ITargetPlatformSettings* GetCookedCookerPlatformSettings() = 0;
};
