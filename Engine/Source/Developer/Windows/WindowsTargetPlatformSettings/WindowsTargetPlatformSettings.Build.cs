// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WindowsTargetPlatformSettings : ModuleRules
{
	public WindowsTargetPlatformSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		// We need a short name here since this can run afoul very easily of the `MAX_PATH` limit when
		// combined with building other targets that make use of this.
		ShortName = "WinTPSet";

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"TargetPlatform",
				"DesktopPlatform",
                "AudioPlatformConfiguration",
            }
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Settings"
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"AudioPlatformConfiguration"
			}
		);

		// compile with Engine
		if (Target.bCompileAgainstEngine)
		{
			PublicIncludePathModuleNames.Add("Engine");
			PrivateDependencyModuleNames.AddRange( new string[] {
				"Engine", 
				"RHI",
				"CookedEditor",
				}
			);
            PrivateIncludePathModuleNames.Add("TextureCompressor");
        }
    }
}
